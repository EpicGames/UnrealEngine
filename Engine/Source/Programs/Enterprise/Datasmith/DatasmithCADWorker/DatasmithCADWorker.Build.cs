// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DatasmithCADWorker : ModuleRules
{
	public DatasmithCADWorker(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Projects",
				"ApplicationCore",
				"Sockets",
				"DatasmithDispatcher",
				"CADInterfaces",
				"CADTools",
			}
		);
	}
}
