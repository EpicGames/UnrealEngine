// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms("Win64")]
public class DatasmithCADWorkerTarget : TargetRules
{
	public DatasmithCADWorkerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "DatasmithCADWorker";
		SolutionDirectory = "Programs/Datasmith";

        // Lean and mean
        bBuildDeveloperTools = false;

		// Editor-only data, however, is needed
		bBuildWithEditorOnlyData = false;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstApplicationCore = true;

		// This is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;

		bLegalToDistributeBinary = true;

		bCompileICU = false;
		bCompileCEF3 = false;

		string EngineDir = @"..\.."; // relative to default destination, which is $(EngineDir)\Binaries\Win64
		ExeBinariesSubFolder = EngineDir + @"\Plugins\Enterprise\DatasmithCADImporter\Binaries\Win64\";
	}
}
