// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithCADWorker.h"
#include "DatasmithCADWorkerImpl.h"

#include "RequiredProgramMainCPPInclude.h"
#include "CADInterfacesModule.h"
#include "CADToolsModule.h"

#ifdef USE_TECHSOFT_SDK
#include "TechSoftInterface.h"
#endif

#include "Modules/ModuleManager.h"

IMPLEMENT_APPLICATION(DatasmithCADWorker, "DatasmithCADWorker");
DEFINE_LOG_CATEGORY(LogDatasmithCADWorker);

#define EXIT_MISSING_CAD_MODULE 2

void GetParameter(int32 Argc, TCHAR* Argv[], const FString& InParam, FString& OutValue)
{
	OutValue.Empty();
	for (int32 Index = 1; Index < Argc; Index++)
	{
		if (! FCString::Strcmp(*InParam, Argv[Index]))
		{
			if (Index + 1 < Argc)
			{
				OutValue = Argv[Index + 1];
			}
		}
	}
}

bool HasParameter(int32 Argc, TCHAR* Argv[], const FString& InParam)
{
	for (int32 Index = 1; Index < Argc; Index++)
	{
		if (!FCString::Strcmp(*InParam, Argv[Index]))
		{
			return true;
		}
	}
	return false;
}

int32 Main(int32 Argc, TCHAR * Argv[])
{
	UE_SET_LOG_VERBOSITY(LogDatasmithCADWorker, Verbose);

#if !defined(USE_TECHSOFT_SDK)
	UE_LOGF(LogDatasmithCADWorker, Error, "Missing CAD module. DatasmithCADWorker is not functional.");
	return EXIT_MISSING_CAD_MODULE;
#else

	FString ServerPID;
	FString ServerPort;
	FString CacheDirectory;
	FString CacheVersion;
	FString EnginePluginsPath;
	GetParameter(Argc, Argv, "-ServerPID", ServerPID);
	GetParameter(Argc, Argv, "-ServerPort", ServerPort);
	GetParameter(Argc, Argv, "-CacheDir", CacheDirectory);
	GetParameter(Argc, Argv, "-CacheVersion", CacheVersion);
	GetParameter(Argc, Argv, "-EnginePluginsDir", EnginePluginsPath);

	if (!CADLibrary::TechSoftInterface::TECHSOFT_InitializeKernel(*EnginePluginsPath))
	{
		UE_LOGF(LogDatasmithCADWorker, Error, "TechSoft interface cannot be initialized. CADInterfaces module is not available.");
		return EXIT_FAILURE;
	}

	if (ICADInterfacesModule::Get().GetAvailability() != ECADInterfaceAvailability::Available)
	{
		UE_LOGF(LogDatasmithCADWorker, Error, "CADInterfaces module is not available.");
		return EXIT_FAILURE;
	}

	int32 EditorCacheVersion = FCString::Atoi(*CacheVersion);
	int32 WorkerCacheVersion = FCADToolsModule::Get().GetCacheVersion();
	if (EditorCacheVersion != 0 && EditorCacheVersion != WorkerCacheVersion)
	{
		UE_LOGF(LogDatasmithCADWorker, Error, "Incompatible cache systems. Please recompile DatasmithCADWorker target.");
		return EXIT_FAILURE;
	}

	FDatasmithCADWorkerImpl Worker(FCString::Atoi(*ServerPID), FCString::Atoi(*ServerPort), EnginePluginsPath, CacheDirectory);
	Worker.Run();

	FDatasmithCADWorkerImpl::bProcessIsRunning = false;

	return EXIT_SUCCESS;
#endif // USE_TECHSOFT_SDK
}

int32 Filter(uint32 Code, struct _EXCEPTION_POINTERS *Ep)
{
	return EXCEPTION_EXECUTE_HANDLER;
}

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	GEngineLoop.PreInit(ArgC, ArgV);

	int32 Result = EXIT_SUCCESS;

	SetErrorMode(SEM_FAILCRITICALERRORS | SEM_NOGPFAULTERRORBOX);
	_set_abort_behavior(0, _WRITE_ABORT_MSG);
	__try
	{
		Result = Main(ArgC, ArgV);
	}
	__except (Filter(GetExceptionCode(), GetExceptionInformation()))
	{
		FDatasmithCADWorkerImpl::bProcessIsRunning = false;
		Result = EXIT_FAILURE;
	}

	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();

	FPlatformMisc::RequestExit(true);

	return Result;
}
