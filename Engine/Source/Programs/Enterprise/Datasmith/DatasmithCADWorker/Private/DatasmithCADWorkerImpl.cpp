// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithCADWorkerImpl.h"

#include "CADFileReader.h"
#include "CADOptions.h"
#include "DatasmithCommands.h"
#include "DatasmithDispatcherConfig.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceRedirector.h"
#include "SocketSubsystem.h"
#include "Sockets.h"
#include "Tasks/Task.h"

using namespace DatasmithDispatcher;

std::atomic<bool> FDatasmithCADWorkerImpl::bProcessIsRunning = false;
std::atomic<bool> FDatasmithCADWorkerImpl::bRequestRestart = false;

FDatasmithCADWorkerImpl::FDatasmithCADWorkerImpl(int32 InServerPID, int32 InServerPort, const FString& InEnginePluginsPath, const FString& InCachePath)
	: ServerPID(InServerPID)
	, ServerPort(InServerPort)
	, EnginePluginsPath(InEnginePluginsPath)
	, CachePath(InCachePath)
	, PingStartCycle(0)
{
	UE_LOGF(LogDatasmithCADWorker, Verbose, "connect to %d...", ServerPort);
	bool bConnected = NetworkInterface.Connect(TEXT("Datasmith CAD Worker"), ServerPort, Config::ConnectTimeout_s);
	UE_LOGF(LogDatasmithCADWorker, Verbose, "connected to %d %ls", ServerPort, bConnected ? TEXT("OK") : TEXT("FAIL"));
	if (bConnected)
	{
		CommandIO.SetNetworkInterface(&NetworkInterface);
	}
	else
	{
		UE_LOGF(LogDatasmithCADWorker, Error, "Server connection failure. exit");
		return;
	}

	InitiatePing();
}

bool FDatasmithCADWorkerImpl::Run()
{
	if (!CommandIO.IsValid())
	{
		return false;
	}

	bool bIsRunning = true;
	while (bIsRunning)
	{
		if (TSharedPtr<ICommand> Command = CommandIO.GetNextCommand(1.0))
		{
			switch (Command->GetType())
			{
			case ECommandId::Ping:
				ProcessCommand(*StaticCast<FPingCommand*>(Command.Get()));
				break;

			case ECommandId::BackPing:
				ProcessCommand(*StaticCast<FBackPingCommand*>(Command.Get()));
				break;

			case ECommandId::RunTask:
				ProcessCommand(*StaticCast<FRunTaskCommand*>(Command.Get()));
				break;

			case ECommandId::ImportParams:
				ProcessCommand(*StaticCast<FImportParametersCommand*>(Command.Get()));
				break;

			case ECommandId::Terminate:
				UE_LOGF(LogDatasmithCADWorker, Verbose, "Terminate command received. Exiting.");
				bIsRunning = false;
				break;

			case ECommandId::NotifyEndTask:
			default:
				break;
			}
		}
		else
		{
			if (bIsRunning)
			{
				bIsRunning = ServerPID == 0 ? true : FPlatformProcess::IsApplicationRunning(ServerPID);
				UE_CLOGF(!bIsRunning, LogDatasmithCADWorker, Error, "Worker failure: server lost");
			}
		}
	}

	UE_CLOGF(!bIsRunning, LogDatasmithCADWorker, Verbose, "Worker loop exit...");
	CommandIO.Disconnect(0);
	return true;
}

void FDatasmithCADWorkerImpl::InitiatePing()
{
	PingStartCycle = FPlatformTime::Cycles64();
	UE_LOGF(LogDatasmithCADWorker, Verbose, "InitiatePing...");
	FPingCommand Ping;
	CommandIO.SendCommand(Ping, Config::SendCommandTimeout_s);
}

void FDatasmithCADWorkerImpl::ProcessCommand(const FPingCommand& PingCommand)
{
	FBackPingCommand BackPing;
	CommandIO.SendCommand(BackPing, Config::SendCommandTimeout_s);
}

void FDatasmithCADWorkerImpl::ProcessCommand(const FBackPingCommand& BackPingCommand)
{
	if (PingStartCycle)
	{
		double ElapsedTime_s = FGenericPlatformTime::ToSeconds(FPlatformTime::Cycles64() - PingStartCycle);
		UE_LOGF(LogDatasmithCADWorker, Verbose, "Ping %f s", ElapsedTime_s);
	}
	PingStartCycle = 0;
}

void FDatasmithCADWorkerImpl::ProcessCommand(const FImportParametersCommand& ImportParametersCommand)
{
	ImportParameters = ImportParametersCommand.ImportParameters;
}

uint64 DefineMaximumAllowedDuration(const CADLibrary::FFileDescriptor& FileDescriptor, bool& bEnableTimeControl)
{
	FFileStatData FileStatData = IFileManager::Get().GetStatData(*FileDescriptor.GetSourcePath());
	double MaxTimePerMb = 5e-6;
	double SafetyCoeficient = 5;

	CADLibrary::ECADFormat Format = FileDescriptor.GetFileFormat();
	switch (Format)
	{
	case CADLibrary::ECADFormat::JT:
	case CADLibrary::ECADFormat::INVENTOR:
		MaxTimePerMb = 1.;
		bEnableTimeControl = false;
		break;
	case CADLibrary::ECADFormat::SOLIDWORKS:
	case CADLibrary::ECADFormat::CATIA_3DXML:
		MaxTimePerMb = 1e-5;
		break;
	case CADLibrary::ECADFormat::CATIA_CGR:
		MaxTimePerMb = 5e-7;
		break;
	case CADLibrary::ECADFormat::IGES:
		MaxTimePerMb = 1e-6;
		break;
	default:
		break;
	}

	constexpr int64 OneKiloBit = 1024;
	UE_LOGF(LogDatasmithCADWorker, Verbose, "    - File size %lld KB", FileStatData.FileSize / OneKiloBit);

	uint64 MaximumDuration = ((double)FileStatData.FileSize) * MaxTimePerMb * SafetyCoeficient;
	return FMath::Max(MaximumDuration, (uint64)30);
}

void FDatasmithCADWorkerImpl::ProcessCommand(const FRunTaskCommand& RunTaskCommand)
{
	using namespace CADLibrary;
	FFileDescriptor FileToProcess = RunTaskCommand.JobFileDescription;
	UE_LOGF(LogDatasmithCADWorker, Verbose, "Process %ls %ls", *FileToProcess.GetFileName(), *FileToProcess.GetConfiguration());

	FCompletedTaskCommand CompletedTask;

	bProcessIsRunning = true;

	bool bEnableTimeControl = CADLibrary::FImportParameters::bGEnableTimeControl;
	int64 MaxDuration = DefineMaximumAllowedDuration(FileToProcess, bEnableTimeControl);

	TArray<UE::Tasks::FTask> Checkers;
	if(bEnableTimeControl)
	{
		Checkers.Emplace(UE::Tasks::Launch(TEXT("TimeChecker"), [&FileToProcess, &MaxDuration]() { CheckDuration(FileToProcess, MaxDuration); }));
	}
	Checkers.Emplace(UE::Tasks::Launch(TEXT("MemoryChecker"), []() { CheckMemory(); }));

	FImportParameters FileImporParameters(ImportParameters, RunTaskCommand.Mesher);

	FCADFileReader FileReader(FileImporParameters, FileToProcess, EnginePluginsPath, CachePath);
	CompletedTask.ProcessResult = FileReader.ProcessFile();

	bProcessIsRunning = false;
	UE::Tasks::Wait(Checkers);

	if (CompletedTask.ProcessResult == ETaskState::ProcessOk)
	{
		if (bRequestRestart)
		{
			CompletedTask.ProcessResult = ETaskState::Unknown;
		}
			
		const FCADFileData& CADFileData = FileReader.GetCADFileData();
		CompletedTask.ExternalReferences = CADFileData.GetExternalRefSet();
		CompletedTask.SceneGraphFileName = CADFileData.GetSceneGraphFileName();
		CompletedTask.GeomFileName = CADFileData.GetMeshFileName();
		CompletedTask.Messages = CADFileData.GetMessages();

		UE_LOGF(LogDatasmithCADWorker, Verbose, "=> Process %ls %ls saved into %ls%ls and %ls%ls.", *FileToProcess.GetFileName(), *FileToProcess.GetConfiguration(), *CompletedTask.SceneGraphFileName, TEXT(".sg"), *CompletedTask.GeomFileName, TEXT(".gm"));
		UE_LOGF(LogDatasmithCADWorker, Verbose, "     It generates %d bodies", CADFileData.GetBodyMeshes().Num());
		for (const FBodyMesh& BodyMesh : CADFileData.GetBodyMeshes())
		{
			FString BodyFileName = FString::Printf(TEXT("UEx%08x"), BodyMesh.MeshActorUId);
			UE_LOGF(LogDatasmithCADWorker, Verbose, "     - Body %ls", *BodyFileName);
		}
	}
	else if(CompletedTask.ProcessResult == ETaskState::FileNotFound)
	{
		UE_LOGF(LogDatasmithCADWorker, Verbose, "=> File not found %ls", *FileToProcess.GetFileName());
	}
	else
	{
		UE_LOGF(LogDatasmithCADWorker, Verbose, "=> Process %ls %ls failed", *FileToProcess.GetFileName(), *FileToProcess.GetConfiguration());
	}
	UE_LOGF(LogDatasmithCADWorker, Verbose, "End of Process %ls %ls", *FileToProcess.GetFileName(), *FileToProcess.GetConfiguration());
	
	if(bRequestRestart)
	{
		GLog->Flush();
	}

	CommandIO.SendCommand(CompletedTask, Config::SendCommandTimeout_s);
}

void FDatasmithCADWorkerImpl::CheckDuration(const CADLibrary::FFileDescriptor& FileToProcess, const int64 MaxDuration)
{
	const uint64 StartTime = FPlatformTime::Cycles64();
	const uint64 MaxCycles = MaxDuration / FPlatformTime::GetSecondsPerCycle64() + StartTime;

	while (bProcessIsRunning)
	{
		FPlatformProcess::Sleep(0.1f);
		if (FPlatformTime::Cycles64() > MaxCycles)
		{
			UE_LOGF(LogDatasmithCADWorker, Verbose, "Time exceeded to process %ls %ls. The maximum allowed duration is %ld s", *FileToProcess.GetFileName(), *FileToProcess.GetConfiguration(), MaxDuration);
			FPlatformMisc::RequestExit(true);
		}
	}
	double Duration = (FPlatformTime::Cycles64() - StartTime) * FPlatformTime::GetSecondsPerCycle64();
	UE_LOGF(LogDatasmithCADWorker, Verbose, "    - Processing Time: %f s", Duration);
}

void FDatasmithCADWorkerImpl::CheckMemory()
{
	constexpr uint64 OneMegaBit = 1024 * 1024;
	constexpr uint64 GigaBit = 1024 * 1024 * 1024;

	uint64 MaxMemoryUsed = FPlatformMemory::GetStats().UsedPhysical;
	UE_LOGF(LogDatasmithCADWorker, Verbose, "    - Start Ram used %llu MB", MaxMemoryUsed / OneMegaBit);

	while (bProcessIsRunning)
	{
		FPlatformProcess::Sleep(0.1f);
		const uint64 MemoryUsed = FPlatformMemory::GetStats().UsedPhysical;
		if (MaxMemoryUsed < MemoryUsed)
		{
			MaxMemoryUsed = MemoryUsed;
		}
	}

	uint64 EndMemoryUsed = FPlatformMemory::GetStats().UsedPhysical;
	UE_LOGF(LogDatasmithCADWorker, Verbose, "    - End Ram used %llu MB", EndMemoryUsed / OneMegaBit);
	UE_LOGF(LogDatasmithCADWorker, Verbose, "    - Max Ram used %llu MB", MaxMemoryUsed / OneMegaBit);
	if(EndMemoryUsed > GigaBit)
	{
		UE_LOGF(LogDatasmithCADWorker, Verbose, "    - Ram used (%llu MB) after cleanup exceeds limit to start new process. CADWorker restart is requested", EndMemoryUsed / OneMegaBit);
		bRequestRestart = true;
	}

}
