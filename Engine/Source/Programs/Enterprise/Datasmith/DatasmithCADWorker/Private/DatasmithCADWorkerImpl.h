// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithCADWorker.h"

#include "CADOptions.h"
#include "DatasmithCommands.h"
#include "DatasmithDispatcherNetworking.h"

#include "HAL/PlatformTime.h"

#include <atomic>

struct FFileStatData;

class FDatasmithCADWorkerImpl
{
public:
	FDatasmithCADWorkerImpl(int32 InServerPID, int32 InServerPort, const FString& EnginePluginsPath, const FString& InCachePath);
	bool Run();

private:
	void InitiatePing();

	void ProcessCommand(const DatasmithDispatcher::FPingCommand& PingCommand);
	void ProcessCommand(const DatasmithDispatcher::FBackPingCommand& BackPingCommand);
	void ProcessCommand(const DatasmithDispatcher::FImportParametersCommand& BackPingCommand);
	void ProcessCommand(const DatasmithDispatcher::FRunTaskCommand& TerminateCommand);

	/**
	 * @param MaxDuration is defined in seconds.
	 */
	static void CheckDuration(const CADLibrary::FFileDescriptor& FileToProcess, const int64 MaxDuration);
	static void CheckMemory();

private:
	DatasmithDispatcher::FNetworkClientNode NetworkInterface;
	DatasmithDispatcher::FCommandQueue CommandIO;

	int32 ServerPID;
	int32 ServerPort;
	FString EnginePluginsPath;
	FString CachePath;
	CADLibrary::FImportParameters ImportParameters;
	uint64 PingStartCycle;

public:
	static std::atomic<bool> bProcessIsRunning;
	static std::atomic<bool> bRequestRestart;
};
