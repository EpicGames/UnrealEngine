// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DatasmithFacadeCSharp : ModuleRules
	{
		public DatasmithFacadeCSharp(ReadOnlyTargetRules Target)
			: base(Target)
		{
			bUseRTTI = true;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"DatasmithCore",
					"DatasmithExporter",
					"DatasmithExporterUI",
					"DatasmithFacade",
					"UdpMessaging", // required for DirectLink networking
				}
			);

			bRequiresImplementModule = false;
		}
	}
}
