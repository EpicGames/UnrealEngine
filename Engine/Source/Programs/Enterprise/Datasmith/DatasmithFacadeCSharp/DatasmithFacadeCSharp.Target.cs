// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;

[SupportedPlatforms("Win64", "Mac")]
public class DatasmithFacadeCSharpTarget : TargetRules
{
	public DatasmithFacadeCSharpTarget(TargetInfo Target)
		: base(Target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		SolutionDirectory = Path.Combine("Programs","Datasmith");
		bBuildInSolutionByDefault = false;

		LaunchModuleName = "DatasmithFacadeCSharp";
		ExeBinariesSubFolder = "DatasmithFacadeCSharp";

		bShouldCompileAsDLL = true;
		LinkType = TargetLinkType.Monolithic;

		bBuildDeveloperTools = false;
		bBuildWithEditorOnlyData = true;
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = true;
		bCompileICU = false;

		bUsesSlate = true;
		bHasExports = true;
		bForceEnableExceptions = true;

		/**
		 * We can't use a external debugger without the necessary DLLs
		 * It cause some issues when initializing a engine loop with a foreign engine dir that is not specified during the static initialization
		 */
		GlobalDefinitions.Add("UE_EXTERNAL_PROFILING_ENABLED=0");

		// Disable CrashReporter
		GlobalDefinitions.Add("NOINITCRASHREPORTER=1");

		AddPreBuildSteps(Target.Platform);
		AddPostBuildSteps(Target.Platform);
	}

	public void AddPreBuildSteps(UnrealTargetPlatform TargetPlatform)
	{
		// Environment variable SWIG_DIR must be set to the Swig third party directory on the developer's workstation to run swig.
		if (string.IsNullOrEmpty(System.Environment.GetEnvironmentVariable("SWIG_DIR")))
		{
			PreBuildSteps.Add("echo Environment variable SWIG_DIR is not defined.");
			return;
		}

		PreBuildSteps.Add("echo Using SWIG_DIR env. variable: %SWIG_DIR%");

		string FacadePath = Path.Combine("$(EngineDir)", "Source", "Developer", "Datasmith", "DatasmithFacade");
		string DatasmithUIPath = Path.Combine("$(EngineDir)", "Source", "Developer", "Datasmith", "DatasmithExporterUI");
		string ProjectPath = Path.Combine("$(EngineDir)", "Source", "Programs", "Enterprise", "Datasmith", "DatasmithFacadeCSharp");
		string SwigCommand = string.Format("\"{0}\" -csharp -c++ -DSWIG_FACADE -DDATASMITHFACADE_API -DDATASMITHEXPORTERUI_API -I\"{1}\" -I\"{2}\" -I\"{3}\" -o \"{4}\" -outdir \"{5}\" \"{6}\"",
			Path.Combine("%SWIG_DIR%", "swig"),
			Path.Combine(FacadePath, "Public"),
			Path.Combine(DatasmithUIPath, "Public"),
			Path.Combine(ProjectPath, "Private"),
			Path.Combine(ProjectPath, "Private", "DatasmithFacadeCSharp.cpp"),
			Path.Combine(ProjectPath, "Public"),
			Path.Combine(ProjectPath, "DatasmithFacadeCSharp.i"));
		string CleanupPath = Path.Combine(ProjectPath, "Public", "*.cs");

		// The 2>nul is to mute the error message if no files where found
		string CleanCommand = string.Format("del \"{0}\" 2>nul", CleanupPath);

		if (TargetPlatform == UnrealTargetPlatform.Mac)
		{
			CleanCommand = string.Format("rm -f \"{0}\"", CleanupPath);
		}

		// Files may disapear as we move or remove classes, it is important to always remove the .cs files to avoid having conflicting leftover files.
		PreBuildSteps.Add(CleanCommand);

		// Generate facade with swig
		PreBuildSteps.Add(SwigCommand);

		// Add copyright headers after generating the new files from swig.
		string PythonDir = Path.Combine("$(EngineDir)", "Binaries", "ThirdParty", "Python3");
		if (TargetPlatform == UnrealTargetPlatform.Mac)
		{
			PreBuildSteps.Add(string.Format("\"{0}\" \"{1}\"", Path.Combine(PythonDir, "Mac", "bin", "python"), Path.Combine(ProjectPath, "FacadeHeaderHelper.py")));
		}
		else
		{
			PreBuildSteps.Add(string.Format("\"{0}\" \"{1}\"", Path.Combine(PythonDir, "Win64", "python.exe"), Path.Combine(ProjectPath, "FacadeHeaderHelper.py")));
		}
	}

	protected void AddPostBuildSteps(UnrealTargetPlatform TargetPlatform)
	{
		string SrcPath = Path.Combine("$(EngineDir)", "Source", "Programs", "Enterprise", "Datasmith", "DatasmithFacadeCSharp", "Public");
		string SrcPathFiles = Path.Combine(SrcPath, "*.cs");
		string DstPath = Path.Combine("$(EngineDir)", "Binaries", "$(TargetPlatform)", "DatasmithFacadeCSharp", "Public");

		// Copy the generated C# files.
		PostBuildSteps.Add(string.Format("echo \"Replace {1} with {0}\"", SrcPathFiles, DstPath));
		if (TargetPlatform == UnrealTargetPlatform.Win64)
		{
			PostBuildSteps.Add(string.Format("del /S /Q \"{0}\"", Path.Combine(DstPath, "*.cs")));
			PostBuildSteps.Add(string.Format("xcopy \"{0}\" \"{1}\" /R /S /Y", SrcPathFiles, DstPath + Path.DirectorySeparatorChar));
		}
		else if (TargetPlatform == UnrealTargetPlatform.Mac)
		{
			PostBuildSteps.Add(string.Format("mkdir -p \"{0}\"", DstPath));
			PostBuildSteps.Add(string.Format("rm -f \"{0}\"*.cs", DstPath + Path.DirectorySeparatorChar));
			PostBuildSteps.Add(string.Format("cp -R \"{0}\"*.cs \"{1}\"", SrcPath + Path.DirectorySeparatorChar, DstPath));
		}
	}
}
