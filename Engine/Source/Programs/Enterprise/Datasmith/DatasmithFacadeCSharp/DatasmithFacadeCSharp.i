// Copyright Epic Games, Inc. All Rights Reserved.

// This is the Swig interface file for the Datasmith C# facade.

%module DatasmithFacadeCSharp

%begin %{
// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFacadeCSharp.h"
%}

%header %{
#include "DatasmithFacadeElement.h"
#include "DatasmithFacadeKeyValueProperty.h"
#include "DatasmithFacadeTexture.h"
#include "DatasmithFacadeMaterial.h"
#include "DatasmithFacadeMaterialID.h"
#include "DatasmithFacadeUEPbrMaterial.h"
#include "DatasmithFacadeMaterialsUtils.h"
#include "DatasmithFacadeMesh.h"
#include "DatasmithFacadeMetaData.h"
#include "DatasmithFacadeActor.h"
#include "DirectLink/DatasmithFacadeEndpointObserver.h"
#include "DirectLink/DatasmithFacadeDirectLink.h"
#include "DatasmithFacadeActorCamera.h"
#include "DatasmithFacadeActorLight.h"
#include "DatasmithFacadeActorMesh.h"
#include "DatasmithFacadeAnimation.h"
#include "DatasmithFacadeScene.h"
#include "DatasmithFacadeVariant.h"
#include "DatasmithFacadeLog.h"
#include "DatasmithFacadeUtils.h"
#include "DatasmithFacadeDecal.h"

#include "IDatasmithExporterUIModule.h"
#include "IDirectLinkUI.h"
%}

%include <arrays_csharp.i>
%include <wchar.i>
%include <swiginterface.i>
%include <typemaps.i>

typedef wchar_t TCHAR;
typedef signed int int32;
typedef unsigned int uint32;
typedef unsigned char uint8;
typedef unsigned long long size_t;

%apply float INPUT[] { const float* InWorldMatrix }
%apply float INPUT[] { const float* InLocalMatrix }

%typemap(cscode) FDatasmithFacadeElement %{
  public static string GetStringHash(string Input)
  {
    System.Text.StringBuilder sb = new System.Text.StringBuilder(32);
    InternalGetStringHash(Input, sb, (ulong)sb.Capacity + 1);
    return sb.ToString();
  }
%}

%typemap(cscode) FDatasmithFacadeTexture %{
  public string GetFileHash()
  {
    System.Text.StringBuilder sb = new System.Text.StringBuilder(32);
    InternalGetFileHash(sb, (ulong)sb.Capacity + 1);
    return sb.ToString();
  }
%}

%typemap(cscode) FDatasmithFacadeMesh %{
  public string CalculateHash()
  {
    System.Text.StringBuilder sb = new System.Text.StringBuilder(32);
    InternalCalculateHash(sb, (ulong)sb.Capacity + 1);
    return sb.ToString();
  }
%}

// Rename functions returning object owned by C# to their corresponding interface name.
%rename (GetChild) GetNewChild;
%rename (GetParentActor) GetNewParentActor;
%rename (GetTexture) GetNewTexture;
%rename (GetActor) GetNewActor;
%rename (GetMaterial) GetNewMaterial;
%rename (GetMetaData) GetNewMetaData;
%rename (GetLightFunctionMaterial) GetNewLightFunctionMaterial;
%rename (GetMaterialOverride) GetNewMaterialOverride;
%rename (GetExpression) GetNewFacadeExpression;
%rename (CreateWeightedMaterialExpression) CreateNewFacadeWeightedMaterialExpression;
%rename (CreateTextureExpression) CreateNewFacadeTextureExpression;
%rename (GetInput) GetNewFacadeInput;
%rename (GetLightFunctionMaterial) GetNewLightFunctionMaterial;
%rename (GetProperty) GetNewProperty;
%rename (GetPropertyByName) GetNewPropertyByName;
%rename (GetLOD) GetNewLOD;
%rename (GetMesh) GetNewMesh;
%rename (GetDestination) GetNewDestination;
%rename (GetSource) GetNewSource;
%rename (GetEndpointInfo) GetNewEndpointInfo;
%rename (GetDataPointsInfo) GetNewDataPointsInfo;
%rename (GetStreamInfo) GetNewStreamInfo;
%rename (GetLevelVariantSets) GetNewLevelVariantSets;
%rename (GetVariantSet) GetNewVariantSet;
%rename (GetVariant) GetNewVariant;
%rename (GetActorBinding) GetNewActorBinding;
%rename (GetPropertyCapture) GetNewPropertyCapture;
%rename (GetTransformAnimation) GetNewTransformAnimation;
%rename (GetVisibilityAnimation) GetNewVisibilityAnimation;
%rename (GetSubsequenceAnimation) GetNewSubsequenceAnimation;
%rename (GetLevelSequence) GetNewLevelSequence;

// Make sure the returned pointer is managed by the C# interface and that the C# type is "downcastable"
%define OWNED_RETURNED_MATERIAL_EXPRESSION(FUNCTIONNAME)
%typemap(csout, excode=SWIGEXCODE) FDatasmithFacadeMaterialExpression* FUNCTIONNAME {
	global::System.IntPtr objectPtr = $imcall;$excode
	if(objectPtr == global::System.IntPtr.Zero)
	{
		return null;
	}
	else
	{
		//Query the type with a temporary wrapper with no memory ownership.
		EDatasmithFacadeMaterialExpressionType ExpressionType = (new FDatasmithFacadeMaterialExpression(objectPtr, false)).GetExpressionType();

		switch(ExpressionType)
		{
		case EDatasmithFacadeMaterialExpressionType.ConstantBool:
			return new FDatasmithFacadeMaterialExpressionBool(objectPtr, true);
		case EDatasmithFacadeMaterialExpressionType.ConstantColor:
			return new FDatasmithFacadeMaterialExpressionColor(objectPtr, true);
		case EDatasmithFacadeMaterialExpressionType.ConstantScalar:
			return new FDatasmithFacadeMaterialExpressionScalar(objectPtr, true);
		case EDatasmithFacadeMaterialExpressionType.FlattenNormal:
			return new FDatasmithFacadeMaterialExpressionFlattenNormal(objectPtr, true);
		case EDatasmithFacadeMaterialExpressionType.FunctionCall:
			return new FDatasmithFacadeMaterialExpressionFunctionCall(objectPtr, true);
		case EDatasmithFacadeMaterialExpressionType.Generic:
			return new FDatasmithFacadeMaterialExpressionGeneric(objectPtr, true);
		case EDatasmithFacadeMaterialExpressionType.Texture:
			return new FDatasmithFacadeMaterialExpressionTexture(objectPtr, true);
		case EDatasmithFacadeMaterialExpressionType.TextureCoordinate:
			return new FDatasmithFacadeMaterialExpressionTextureCoordinate(objectPtr, true);
		default:
			return null;
		}
	}
}
%enddef

OWNED_RETURNED_MATERIAL_EXPRESSION(GetNewFacadeExpression)
OWNED_RETURNED_MATERIAL_EXPRESSION(CreateNewFacadeWeightedMaterialExpression)
#undef OWNED_RETURNED_MATERIAL_EXPRESSION

// Make sure the returned pointer is managed by the C# interface and that the C# type is "downcastable"
%define OWNED_RETURNED_ACTOR(FUNCTIONNAME)
%typemap(csout, excode=SWIGEXCODE) FDatasmithFacadeActor* FUNCTIONNAME {
	global::System.IntPtr objectPtr = $imcall;$excode
	if(objectPtr == global::System.IntPtr.Zero)
	{
		return null;
	}
	else
	{
		FDatasmithFacadeActor.EActorType ActorType = (new FDatasmithFacadeActor(objectPtr, false)).GetActorType();

		switch(ActorType)
		{
		case FDatasmithFacadeActor.EActorType.DirectionalLight:
			return new FDatasmithFacadeDirectionalLight(objectPtr, true);
		case FDatasmithFacadeActor.EActorType.AreaLight:
			return new FDatasmithFacadeAreaLight(objectPtr, true);
		case FDatasmithFacadeActor.EActorType.LightmassPortal:
			return new FDatasmithFacadeLightmassPortal(objectPtr, true);
		case FDatasmithFacadeActor.EActorType.PointLight:
			return new FDatasmithFacadePointLight(objectPtr, true);
		case FDatasmithFacadeActor.EActorType.SpotLight:
			return new FDatasmithFacadeSpotLight(objectPtr, true);
		case FDatasmithFacadeActor.EActorType.StaticMeshActor:
			return new FDatasmithFacadeActorMesh(objectPtr, true);
		case FDatasmithFacadeActor.EActorType.Camera:
			return new FDatasmithFacadeActorCamera(objectPtr, true);
		case FDatasmithFacadeActor.EActorType.Decal:
			return new FDatasmithFacadeActorDecal(objectPtr, true);
		case FDatasmithFacadeActor.EActorType.Actor:
			return new FDatasmithFacadeActor(objectPtr, true);
		case FDatasmithFacadeActor.EActorType.Unsupported:
		default:
			return null;
		}
	}
}
%enddef

OWNED_RETURNED_ACTOR(GetNewChild)
OWNED_RETURNED_ACTOR(GetNewParentActor)
OWNED_RETURNED_ACTOR(GetNewActor)
#undef OWNED_RETURNED_ACTOR

// Make sure the returned pointer is managed by the C# interface and that the C# type is "downcastable"
%define OWNED_RETURNED_MATERIAL(FUNCTIONNAME)
%typemap(csout, excode=SWIGEXCODE) FDatasmithFacadeBaseMaterial* FUNCTIONNAME {
	global::System.IntPtr objectPtr = $imcall;$excode
	if(objectPtr == global::System.IntPtr.Zero)
	{
		return null;
	}
	else
	{
		FDatasmithFacadeBaseMaterial.EDatasmithMaterialType MaterialType = (new FDatasmithFacadeBaseMaterial(objectPtr, false)).GetDatasmithMaterialType();

		switch(MaterialType)
		{
		case FDatasmithFacadeBaseMaterial.EDatasmithMaterialType.UEPbrMaterial:
			return new FDatasmithFacadeUEPbrMaterial(objectPtr, true);
		case FDatasmithFacadeBaseMaterial.EDatasmithMaterialType.MaterialInstance:
			return new FDatasmithFacadeMaterialInstance(objectPtr, true);
		case FDatasmithFacadeBaseMaterial.EDatasmithMaterialType.DecalMaterial:
			return new FDatasmithFacadeDecalMaterial(objectPtr, true);
		case FDatasmithFacadeBaseMaterial.EDatasmithMaterialType.Unsupported:
		default:
			return null;
		}
	}
}
%enddef

OWNED_RETURNED_MATERIAL(GetNewMaterial)
#undef OWNED_RETURNED_MATERIAL

// Make sure we the pointer returned by the given function is managed by the C# interface.
%define OWNED_GENERIC_FACADE_OBJECT(OBJECTTYPE, FUNCTIONNAME)
%typemap(csout, excode=SWIGEXCODE) OBJECTTYPE* FUNCTIONNAME {
	global::System.IntPtr objectPtr = $imcall;$excode
	if(objectPtr == global::System.IntPtr.Zero)
	{
		return null;
	}
	else
	{
		return new OBJECTTYPE(objectPtr, true);
	}
}
%enddef

OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeTexture, GetNewTexture)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeMetaData, GetNewMetaData)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeMaterialID, GetNewLightFunctionMaterial)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeMaterialID, GetNewMaterialOverride)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeMaterialExpressionTexture, CreateNewFacadeTextureExpression)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeExpressionInput, GetNewFacadeInput)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeKeyValueProperty, GetNewProperty)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeKeyValueProperty, GetNewPropertyByName)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeMesh, GetNewLOD)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeMeshElement, GetNewMesh)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeDataPointId, GetNewDestination)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeDataPointId, GetNewSource)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeEndpointInfo, GetNewEndpointInfo)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeDataPointInfo, GetNewDataPointsInfo)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeStreamInfo, GetNewStreamInfo)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeLevelVariantSets, GetNewLevelVariantSets)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeVariantSet, GetNewVariantSet)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeVariant, GetNewVariant)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadeActorBinding, GetNewActorBinding)
OWNED_GENERIC_FACADE_OBJECT(FDatasmithFacadePropertyCapture, GetNewPropertyCapture)
#undef OWNED_GENERIC_FACADE_OBJECT

// Serialize the uint8* returned by FDatasmithFacadeTexture::GetData() into byte[]
%typemap(csout, excode=SWIGEXCODE) uint8* GetData {
	global::System.IntPtr arrayPtr = $imcall;$excode
	if(arrayPtr == global::System.IntPtr.Zero || OutDataSize == 0)
	{
		return null;
	}
	else
	{
		byte[] Data = new byte[OutDataSize];
		global::System.Runtime.InteropServices.Marshal.Copy(arrayPtr, Data, 0, (int)OutDataSize);
		return Data;
	}
}

%typemap(ctype) float[16] "float *"
%typemap(imtype) float[16] "[global::System.Runtime.InteropServices.In, global::System.Runtime.InteropServices.MarshalAs(global::System.Runtime.InteropServices.UnmanagedType.LPArray)] float[]"
%typemap(cstype) float[16] "float[]"
%typemap(csin) float[16] "$csinput"

%typemap(ctype) double[16] "double *"
%typemap(imtype) double[16] "[global::System.Runtime.InteropServices.In, global::System.Runtime.InteropServices.MarshalAs(global::System.Runtime.InteropServices.UnmanagedType.LPArray)] double[]"
%typemap(cstype) double[16] "double[]"
%typemap(csin) double[16] "$csinput"

// Macro used to return custom enum types as C# "out arguments"
%define OUTPUT_CUSTOMENUM_TYPEMAP(ENUMTYPE, ENUMCSTYPE)
%typemap(ctype) ENUMTYPE *OUTPUT, ENUMTYPE &OUTPUT "int *"
%typemap(imtype) ENUMTYPE *OUTPUT, ENUMTYPE &OUTPUT "out ENUMCSTYPE"
%typemap(cstype) ENUMTYPE *OUTPUT, ENUMTYPE &OUTPUT "out ENUMCSTYPE"
%typemap(csin) ENUMTYPE *OUTPUT, ENUMTYPE &OUTPUT "out $csinput"
%enddef

OUTPUT_CUSTOMENUM_TYPEMAP(FDatasmithFacadeTexture::ETextureFormat, FDatasmithFacadeTexture.ETextureFormat)
#undef OUTPUT_CUSTOMENUM_TYPEMAP

%apply FDatasmithFacadeTexture::ETextureFormat& OUTPUT { FDatasmithFacadeTexture::ETextureFormat& OutFormat }

%apply double& OUTPUT { double& };
%apply float& OUTPUT { float& };
%apply uint8& OUTPUT { uint8& }
%apply uint32& OUTPUT { uint32& }
%apply int32& OUTPUT { int32& }
%apply bool& OUTPUT { bool& }

// Implement the proper string serialisation for FDatasmtihFacadeElement::GetStringHash() and FDatasmithFacadeTexture::GetFileHash() with StringBuilder and hide the low-level implementation behind a wraper function.
%rename (InternalGetStringHash) GetStringHash;
%csmethodmodifiers GetStringHash "private";
%rename (InternalGetFileHash) GetFileHash;
%csmethodmodifiers GetFileHash "private";
%rename (InternalCalculateHash) CalculateHash;
%csmethodmodifiers CalculateHash "private";
%typemap(imtype) TCHAR OutBuffer[33] "[global::System.Runtime.InteropServices.MarshalAs(global::System.Runtime.InteropServices.UnmanagedType.LPWStr)] System.Text.StringBuilder"
%typemap(cstype) TCHAR OutBuffer[33]  "System.Text.StringBuilder"
%typemap(csin) TCHAR OutBuffer[33]  "$csinput"
%typemap(imtype) size_t BufferSize "ulong"
%typemap(cstype) size_t BufferSize "ulong"
%typemap(csin) size_t BufferSize  "$csinput"

// Use the correct array types for serializing FDatasmtihFacadeTexture::SetData() and FDatasmtihFacadeTexture::GetData()
%typemap(imtype) uint8* InData "byte[]"
%typemap(cstype) uint8* InData "byte[]"
%typemap(csin) uint8* InData "$csinput"
%typemap(cstype) uint8* GetData "byte[]"

// FDatasmithFacadeEndpointObserver customization:
// Fixing the event registration in C# to make sure no dangling pointer stays registed on the unmanaged side after the wrapper destruction.
%typemap(imtype) FDatasmithFacadeEndpointObserver::OnStateChangedDelegate InOnStateChanged "FDatasmithFacadeEndpointObserver.NotifyStateChangeNativeEvent"
%typemap(cstype) FDatasmithFacadeEndpointObserver::OnStateChangedDelegate InOnStateChanged "NotifyStateChangeNativeEvent"
%typemap(csin) FDatasmithFacadeEndpointObserver::OnStateChangedDelegate InOnStateChanged "$csinput"
%typemap(cscode) FDatasmithFacadeEndpointObserver %{
  public delegate void NotifyStateChangeNativeEvent(global::System.IntPtr NewRawInfoPtr);
  private NotifyStateChangeNativeEvent NotifyStateChangedEvent;
  private System.Object ObjectLock = new System.Object();

  public class FDatasmithFacadeRawInfoEventArgs : System.EventArgs
  {
    public FDatasmithFacadeRawInfo RawInfo;
  }

  private System.EventHandler<FDatasmithFacadeRawInfoEventArgs> OnStateChangeInternal;
  public event System.EventHandler<FDatasmithFacadeRawInfoEventArgs> OnStateChangedEvent
  {
    add
    {
      lock (ObjectLock)
      {
        bool bNeedsRegistration = OnStateChangeInternal == null;
        OnStateChangeInternal += value;
		if (bNeedsRegistration)
		{
          RegisterOnStateChangedDelegateInternal(NotifyStateChangedEvent);
		}
      }
    }
    remove
    {
      lock (ObjectLock)
      {
        OnStateChangeInternal -= value;
		if (OnStateChangeInternal == null)
		{
		  // No need to receive updates if no one is registered to the event
          UnregisterOnStateChangedDelegateInternal(NotifyStateChangedEvent);
		}
      }
    }
  }

  public virtual void OnStateChanged(FDatasmithFacadeRawInfo RawInfo) {}

  private void NotifyStateChange(global::System.IntPtr NewRawInfoPtr)
  {
    // Create C# wrapper around FDatasmithFacadeRawInfo pointer.
    FDatasmithFacadeRawInfo RawInfo = new FDatasmithFacadeRawInfo(NewRawInfoPtr, true);
	FDatasmithFacadeRawInfoEventArgs EventArgs = new FDatasmithFacadeRawInfoEventArgs();
	EventArgs.RawInfo = RawInfo;
    OnStateChangeInternal?.Invoke(this, EventArgs);
	OnStateChanged(RawInfo);
  }
%}
%typemap(csconstruct, excode=SWIGEXCODE,directorconnect="\n    SwigDirectorConnect();") FDatasmithFacadeEndpointObserver %{: this($imcall, true)
  {$excode$directorconnect
    NotifyStateChangedEvent = new NotifyStateChangeNativeEvent(NotifyStateChange);
  }
%}
%typemap(csdisposing, methodname="Dispose", methodmodifiers="protected", parameters="bool disposing") FDatasmithFacadeEndpointObserver %{
  {
    // Disposing of the C# object while the C++ still holds a functions pointer the managed code. Unregister the function ptr.
    if (OnStateChangeInternal != null)
    {
        UnregisterOnStateChangedDelegateInternal(NotifyStateChangedEvent);
    }

    lock(this) {
      if (swigCPtr.Handle != global::System.IntPtr.Zero) {
        if (swigCMemOwn) {
          swigCMemOwn = false;
          $imcall;
        }
        swigCPtr = new global::System.Runtime.InteropServices.HandleRef(null, global::System.IntPtr.Zero);
      }
    }
  }
%}
%csmethodmodifiers RegisterOnStateChangedDelegateInternal "private";
%csmethodmodifiers UnregisterOnStateChangedDelegateInternal "private";


// Customizing the FDatasmithFacadeDirectLink::AddEndpointObserver() and FDatasmithFacadeDirectLink::RemoveEndpointObserver():
// We need to make sure any registed C# observer is kept alive. To achieve this we hold a list of all the registered observers.
%typemap(csout, excode=SWIGEXCODE) void AddEndpointObserver {
    if (!RegisteredObservers.Contains(Observer))
    {
      $imcall;$excode
      RegisteredObservers.Add(Observer);
    }
}
%typemap(csout, excode=SWIGEXCODE) void RemoveEndpointObserver {
	$imcall;$excode
	RegisteredObservers.Remove(Observer);
}
%typemap(cscode) FDatasmithFacadeDirectLink %{
  private System.Collections.Generic.List<FDatasmithFacadeEndpointObserver> RegisteredObservers = new System.Collections.Generic.List<FDatasmithFacadeEndpointObserver>();
%}
%typemap(csdisposing, methodname="Dispose", methodmodifiers="protected", parameters="bool disposing") FDatasmithFacadeDirectLink %{
  {
    // Unregistering all observers in managed code before disposing of the C# object.
	for (int ObserverIndex = RegisteredObservers.Count - 1; ObserverIndex >= 0; --ObserverIndex)
	{
		RemoveEndpointObserver(RegisteredObservers[ObserverIndex]);
	}

    lock(this) {
      if (swigCPtr.Handle != global::System.IntPtr.Zero) {
        if (swigCMemOwn) {
          swigCMemOwn = false;
          $imcall;
        }
        swigCPtr = new global::System.Runtime.InteropServices.HandleRef(null, global::System.IntPtr.Zero);
      }
    }
  }
%}


// Macro used to obfuscate the interface of a struct. This method allows us to obfuscate the type and give it a nice name, instead of being stuck with the default SWIGTYPE_p_<name> syntax.
%define OBFUSCATED_STRUCT(CPP_NAME, CSHARP_NAME)
%rename (CSHARP_NAME) CPP_NAME;
struct CPP_NAME {
private:
	//This prevent swig from generating a default constructor in C# for that type.
	CPP_NAME() = default;
};
%enddef

// Curently, those types are only exposed to be used as handles and keys, the users don't need to access their interface.
OBFUSCATED_STRUCT(FGuid, FDatasmithFacadeGuid)
OBFUSCATED_STRUCT(FMessageAddress, FDatasmithFacadeMessageAddress)
#undef OBFUSCATED_STRUCT


// C# does not support multiple inheritance, declaring FDatasmithFacadeExpressionParameter as a C# interface.
%interface_impl(FDatasmithFacadeExpressionParameter);

%include "DatasmithFacadeElement.h"
%include "DatasmithFacadeKeyValueProperty.h"
%include "DatasmithFacadeTexture.h"
%include "DatasmithFacadeMaterial.h"
%include "DatasmithFacadeMaterialID.h"
%include "DatasmithFacadeUEPbrMaterial.h"
%include "DatasmithFacadeMaterialsUtils.h"
%include "DatasmithFacadeMesh.h"
%include "DatasmithFacadeMetaData.h"
%include "DatasmithFacadeActor.h"
%include "DatasmithFacadeActorCamera.h"
%include "DatasmithFacadeActorLight.h"
%include "DatasmithFacadeActorMesh.h"
%include "DatasmithFacadeAnimation.h"
%include "DatasmithFacadeScene.h"
%include "DatasmithFacadeLog.h"
%include "DirectLink/DatasmithFacadeEndpointObserver.h"
%include "DirectLink/DatasmithFacadeDirectLink.h"
%include "DatasmithFacadeUtils.h"
%include "DatasmithFacadeVariant.h"
%include "DatasmithFacadeDecal.h"

// Datasmith UI
%include "IDatasmithExporterUIModule.h"
%include "IDirectLinkUI.h"

// Specialized templates instantiation
%extend FDatasmithFacadeUEPbrMaterial {
	%template(AddMaterialExpressionBool) AddMaterialExpression<FDatasmithFacadeMaterialExpressionBool>;
	%template(AddMaterialExpressionColor) AddMaterialExpression<FDatasmithFacadeMaterialExpressionColor>;
	%template(AddMaterialExpressionFlattenNormal) AddMaterialExpression<FDatasmithFacadeMaterialExpressionFlattenNormal>;
	%template(AddMaterialExpressionFunctionCall) AddMaterialExpression<FDatasmithFacadeMaterialExpressionFunctionCall>;
	%template(AddMaterialExpressionGeneric) AddMaterialExpression<FDatasmithFacadeMaterialExpressionGeneric>;
	%template(AddMaterialExpressionScalar) AddMaterialExpression<FDatasmithFacadeMaterialExpressionScalar>;
	%template(AddMaterialExpressionTexture) AddMaterialExpression<FDatasmithFacadeMaterialExpressionTexture>;
	%template(AddMaterialExpressionTextureCoordinate) AddMaterialExpression<FDatasmithFacadeMaterialExpressionTextureCoordinate>;
}

%{
#if PLATFORM_MAC
//wchar_t size is platform dependent, we need to make sure the size stays 16-bit, this should cover all our use-cases.
#define wchar_t char16_t
#endif
%}