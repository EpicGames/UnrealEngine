#Copyright Epic Games, Inc. All Rights Reserved.

#This file should be executed from the DatasmithFacadeCSharp folder, it may not work properly otherwise.
import os

license = '// Copyright Epic Games, Inc. All Rights Reserved.\n\n'

def validate_license_on_file(file):
        print('visit %s...'%file)
        with open(file, 'r') as fr:
                content = fr.read()
                if not content.startswith(license[:-5]):
                        with open(file, 'wt') as fw:
                                fw.write(license)
                                fw.write(content)
                                print('\tcopyright added')

def validate_license_on_tree(path):
        for r, folders, files in os.walk(path):
                for file in files:
                        validate_license_on_file(os.path.join(r, file))

basePath = dir_path = os.path.dirname(os.path.realpath(__file__))
rootPrivate = os.path.join(basePath, 'Private')
rootPublic = os.path.join(basePath, 'Public')

validate_license_on_tree(rootPrivate)
validate_license_on_tree(rootPublic)
