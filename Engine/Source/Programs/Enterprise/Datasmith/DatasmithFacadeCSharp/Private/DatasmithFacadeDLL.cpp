// Copyright Epic Games, Inc. All Rights Reserved.

// Datasmith SDK.
#include "DatasmithExporterManager.h"

/**
 * On Windows we can safely use DllMain callback to call FDatasmithExporterManager::Shutdown()however there are no safe alternative on Mac and Linux.
 * We cannot use __attribute__((destructor)) as it is not guaranteed to be called before static variables being freed. 
 * On those platforms we must call FDatasmithFacade::Shutdown() manually on the C# side when the application shutdowns.
 */


#if PLATFORM_WINDOWS
// Begin Datasmith platform include guard.
#include "Windows/AllowWindowsPlatformTypes.h"

BOOL WINAPI DllMain(
	HINSTANCE HinstDLL,
	DWORD     FdwReason,
	LPVOID    LpvReserved
)
{
	if (FdwReason == DLL_PROCESS_DETACH)
	{
		// When a process is unloading the DLL, shut down the Datasmith exporter module.
		FDatasmithExporterManager::Shutdown();
	}
	
	return TRUE;
}

// End Datasmith platform include guard.
#include "Windows/HideWindowsPlatformTypes.h"
#endif