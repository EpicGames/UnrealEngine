// Copyright Epic Games, Inc. All Rights Reserved.

This project requires environment variable SWIG_DIR to be set to the Swig 4.0.1 (www.swig.org) third party directory
on the developer's workstation in order to run swig.exe and regenerate Private\DatasmithFacadeCSharp.cpp and Public\*.cs.
Python3 is also required in order to execute a post-SWIG script adding missing headers to generated files.
