<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi">
	<Fragment>
		<!-- Abort install if hosta pp is not installed -->
		<CustomAction Id="NoHostAppInstaller" Error="!(loc.ErrorNotInstalled)" />
		<!-- Prevent downgrading -->
		<CustomAction Id="PreventDowngrading" Error="!(loc.ErrorOlderVersionInstall)" />
		<!-- Prevent overwriting -->
		<CustomAction Id="PreventSameVersionInstall" Error="!(loc.ErrorSameVersionInstall)" />
		<CustomActionRef Id="WixExitEarlyWithSuccess" />
		<CustomAction Id="InstallDirectX" FileKey="DXSETUPEXE" ExeCommand="/silent" Execute="deferred" Impersonate="no" Return="check"/>
	</Fragment>
</Wix>
