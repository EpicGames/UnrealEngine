<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi">
	<Fragment>
		<Property Id="MsiLogging" Value="v" />
		<UI>
			<UIRef Id="CustomWixUI_InstallDir" />
		</UI>
		<Property Id="WIXUI_EXITDIALOGOPTIONALCHECKBOXTEXT" Value="View Installation Log"></Property>
		<WixVariable Id="WixUILicenseRtf" Value="license.rtf"/>
		<Property Id="WixShellExecTarget" Value="[MsiLogFileLocation]" />
		<CustomAction Id="LaunchApplication" BinaryKey="WixCA" DllEntry="WixShellExec" Impersonate="yes" />
		<UI Id="CustomWixUI_InstallDir">

			<!-- Specify font styles -->
			<TextStyle Id="WixUI_Font_Normal" FaceName="Tahoma" Size="9" />
			<TextStyle Id="WixUI_Font_Normal_White" FaceName="Tahoma" Size="9" Red="255" Green="255" Blue="255" />
			<TextStyle Id="WixUI_Font_Bigger" FaceName="Tahoma" Size="12" />
			<TextStyle Id="WixUI_Font_Title" FaceName="Tahoma" Size="9" Bold="yes" Red="255" Green="255" Blue="255" />
			<TextStyle Id="CP_Font_Desc" FaceName="Tahoma" Size="8" Red="255" Green="255" Blue="255" />

			<Property Id="DefaultUIFont" Value="WixUI_Font_Normal" />
			<Property Id="WixUI_Mode" Value="InstallDir" />
			<DialogRef Id="ErrorDlg" />
			<DialogRef Id="FilesInUse" />
			<DialogRef Id="MsiRMFilesInUse" />
			<DialogRef Id="FilesInUse" />
			<DialogRef Id="PrepareDlg" />
			<DialogRef Id="ProgressDlg" />
			<DialogRef Id="ResumeDlg" />

			<Dialog Id="MyProgressDlg" Width="800" Height="450" Title="!(loc.ProgressDlg_Title)" Modeless="yes">
				<Control Id="Cancel" Type="PushButton" X="720" Y="415" Width="56" Height="17" Default="yes" Cancel="yes" Text="!(loc.WixUICancel)">
					<Publish Event="SpawnDialog" Value="CancelDlg">1</Publish>
				</Control>
				<!-- Header -->
				<Control Id="BannerVoid" Type="Bitmap" X="0" Y="0" Width="800" Height="50" TabSkip="no" Text="BannerFill" />
				<Control Id="BannerLogo" Type="Bitmap" X="750" Y="5" Width="40" Height="40" TabSkip="no" Text="LogoEpicGames" />
				<Control Id="HeaderTitle" Type="Text" X="20" Y="15" Width="600" Height="35"  Transparent="yes" Text="{\TitleBanner}!(loc.InstallMessageEpicBoxTitleDialog)"></Control>
				<!-- / Header -->
				<!-- Left Container -->
				<Control Id="Image1" Type="Bitmap" X="20" Y="75" Width="450" Height="253" TabSkip="no" Text="Background3" />
				<Control Id="Caption1" Type="Text" X="20" Y="330" Width="210" Height="75"  Transparent="yes"  Text="!(loc.BackGroundCredits3)"/>
				<!-- /Left Container -->
				<!-- Right Container -->
				<Control Id="Title" Type="Text" X="490" Y="75" Width="270" Height="60" Transparent="yes" NoPrefix="yes" Text="!(loc.InstallMessageEpicMessageBoxContentLine1)" />
				<!-- /Right Container -->

				<!-- Progress Bottom Container -->
				<Control Id="ActionText" Type="Text" X="70" Y="355" Width="285" Height="25">
					<Subscribe Event="ActionText" Attribute="Text" />
				</Control>
				<Control Id="StatusLabel" Type="Text" X="20" Y="355" Width="50" Height="25" Text="!(loc.ProgressDlgStatusLabel)" />
				<Control Id="ProgressBar" Type="ProgressBar" X="20" Y="380" Width="700" Height="10" ProgressBlocks="yes" Text="!(loc.ProgressDlgProgressBar)">
					<Subscribe Event="SetProgress" Attribute="Progress" />
				</Control>
				<!-- /Progress Bottom Container-->

				<!-- Bottom Container -->
				<Control Id="BottomLine" Type="Line" X="0" Y="400" Width="800" Height="0" />
				<Control Id="VersionNumber" Type="Text" X="20" Y="416" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />
				<!--/Bottom Container -->

			</Dialog>
			<InstallUISequence>
				<Show Dialog="MyProgressDlg" After="ProgressDlg" />
			</InstallUISequence>

			<Dialog Id="MyUserExit"  Width="800" Height="450" Title="!(loc.UserExit_Title)">

				<!-- Header -->
				<Control Id="BannerVoid" Type="Bitmap" X="0" Y="0" Width="800" Height="50" TabSkip="no" Text="BannerFill" />
				<Control Id="BannerLogo" Type="Bitmap" X="750" Y="5" Width="40" Height="40" TabSkip="no" Text="LogoEpicGames" />
				<Control Id="HeaderTitle" Type="Text" X="20" Y="15" Width="600" Height="35"  Transparent="yes" Text="{\TitleBanner}!(loc.InstallMessageEpicBoxTitleDialog)"></Control>
				<!-- / Header -->

				<!-- Left Container -->
				<Control Id="Image1" Type="Bitmap" X="20" Y="75" Width="450" Height="253" TabSkip="no" Text="Background1" />
				<Control Id="Caption1" Type="Text" X="20" Y="330" Width="210" Height="75"  Transparent="yes"  Text="!(loc.BackGroundCredits1)"/>
				<!-- /Left Container -->

				<!-- Right Container -->
				<Control Id="Title" Type="Text" X="490" Y="75" Width="270" Height="250" Transparent="yes" NoPrefix="yes" Text="!(loc.UserExitDescription1) !(loc.UserExitDescription2)" />
				<!-- /Right Container -->

				<!-- Bottom Container -->
				<Control Id="BottomLine" Type="Line" X="0" Y="400" Width="800" Height="0" />
				<Control Id="Finish" Type="PushButton" X="720" Y="415" Width="56" Height="17" Default="yes" Cancel="yes" Text="!(loc.WixUIFinish)">
					<Publish Event="EndDialog" Value="Exit">1</Publish>
				</Control>
				<!-- /Bottom Container -->
			</Dialog>
			<InstallUISequence>
				<Show Dialog="MyUserExit" OnExit="cancel" Overridable="yes">NOT Installed</Show>
			</InstallUISequence>
			<AdminUISequence>
				<Show Dialog="MyUserExit" OnExit="cancel" Overridable="yes">NOT Installed</Show>
			</AdminUISequence>

			<Dialog Id="MyErrorExit"  Width="800" Height="450" Title="!(loc.UserExit_Title)">
				<!-- Header -->
				<Control Id="BannerVoid" Type="Bitmap" X="0" Y="0" Width="800" Height="50" TabSkip="no" Text="BannerFill" />
				<Control Id="BannerLogo" Type="Bitmap" X="736" Y="5" Width="40" Height="42" TabSkip="no" Text="LogoEpicGames" />
				<Control Id="HeaderTitle" Type="Text" X="20" Y="15" Width="600" Height="35"  Transparent="yes" Text="{\TitleBanner}!(loc.InstallMessageEpicBoxTitleDialog)"></Control>
				<!-- / Header -->
				<!-- Left Container -->
				<Control Id="Image1" Type="Bitmap" X="20" Y="75" Width="450" Height="253" TabSkip="no" Text="Background1" />
				<Control Id="Caption1" Type="Text" X="20" Y="330" Width="210" Height="75"  Transparent="yes"  Text="Image courtesy of Michael Mong"/>
				<!-- /Left Container -->

				<!-- Right Container -->
				<Control Id="Title" Type="Text" X="490" Y="75" Width="270" Height="250" Transparent="yes" NoPrefix="yes" Text="!(loc.UserExitDescription1) !(loc.UserExitDescription2)" />
				<!-- /Right Container -->

				<!-- Bottom Container -->
				<Control Id="BottomLine" Type="Line" X="0" Y="400" Width="800" Height="0" />
				<Control Id="Finish" Type="PushButton" X="720" Y="416" Width="56" Height="17" Default="yes" Cancel="yes" Text="!(loc.WixUIFinish)">
					<Publish Event="EndDialog" Value="Exit">1</Publish>
				</Control>
				<!-- /Bottom Container -->
			</Dialog>
			<InstallUISequence>
				<Show Dialog="MyErrorExit" OnExit="error" Overridable="yes">NOT Installed</Show>
			</InstallUISequence>
			<AdminUISequence>
				<Show Dialog="MyErrorExit" OnExit="error" Overridable="yes">NOT Installed</Show>
			</AdminUISequence>

			<Dialog Id="MyExitDialog" Width="800" Height="450" Title="!(loc.ExitDialog_Title)">
				<!-- Header -->
				<Control Id="BannerVoid" Type="Bitmap" X="0" Y="0" Width="800" Height="50" TabSkip="no" Text="BannerFill" />
				<Control Id="BannerLogo" Type="Bitmap" X="736" Y="5" Width="40" Height="42" TabSkip="no" Text="LogoEpicGames" />
				<Control Id="HeaderTitle" Type="Text" X="20" Y="15" Width="600" Height="35"  Transparent="yes" Text="{\TitleBanner}!(loc.ExitMessageEpicBoxTitleDialog)"></Control>
				<!-- / Header -->

				<!-- Left Container -->
				<Control Id="Image1" Type="Bitmap" X="20" Y="75" Width="450" Height="253" TabSkip="no" Text="Background4" />
				<Control Id="Caption1" Type="Text" X="20" Y="330" Width="210" Height="75"  Transparent="yes"  Text="!(loc.BackGroundCredits4)"/>
				<!-- /Left Container -->

				<!-- Right Container -->
				<Control Id="Title" Type="Text" X="490" Y="75" Width="270" Height="30" Transparent="yes" NoPrefix="yes" Text="!(loc.ExitMessageEpicMessageBoxContentLine1)" />
				<Control Id="Description" Type="Text" X="490" Y="90" Width="270" Height="30" Transparent="yes" NoPrefix="yes" Text="!(loc.ExitMessageEpicBoxContentLine2)" >
					<Condition Action="show">NOT Installed AND NOT UPGRADEFOUND</Condition>
					<Condition Action="hide">Installed OR UPGRADEFOUND</Condition>
				</Control>
				<!-- /Right Container -->

				<!--  Bottom Control Button -->
				<Control Id="BottomLine" Type="Line" X="0" Y="400" Width="800" Height="0" />
				<Control Id="VersionNumber" Type="Text" X="20" Y="416" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />
				<Control Id="Finish" Type="PushButton" X="720" Y="415" Width="56" Height="17" Default="yes" Cancel="yes" Text="!(loc.WixUIFinish)">
					<Publish Event="EndDialog" Value="Exit">1</Publish>
				</Control>
				<!--  / Bottom Control Button -->
				<Control Id="OptionalText" Type="Text" X="550" Y="330" Width="220" Height="80" Transparent="yes" NoPrefix="yes" Hidden="yes" Text="[WIXUI_EXITDIALOGOPTIONALTEXT]">
					<Condition Action="show">WIXUI_EXITDIALOGOPTIONALTEXT AND NOT Installed</Condition>
				</Control>
				<Control Id="OptionalCheckBox" Type="CheckBox" X="490" Y="330" Width="220" Height="40" Hidden="yes" Property="WIXUI_EXITDIALOGOPTIONALCHECKBOX" CheckBoxValue="1" Text="[WIXUI_EXITDIALOGOPTIONALCHECKBOXTEXT]">
					<Condition Action="show">WIXUI_EXITDIALOGOPTIONALCHECKBOXTEXT AND NOT Installed</Condition>
				</Control>
			</Dialog>

			<InstallUISequence>
				<Show Dialog="MyExitDialog" OnExit="success" Overridable="yes" >NOT Installed</Show>
			</InstallUISequence>

			<AdminUISequence>
				<Show Dialog="MyExitDialog" OnExit="success" Overridable="yes" >NOT Installed</Show>
			</AdminUISequence>

			<Dialog Id="MyMaintenanceWelcomeDlg" Width="800" Height="450"  Title="!(loc.MaintenanceWelcomeDlg_Title)">
				<!-- Header -->
				<Control Id="BannerVoid" Type="Bitmap" X="0" Y="0" Width="800" Height="50" TabSkip="no" Text="BannerFill" />
				<Control Id="BannerLogo" Type="Bitmap" X="736" Y="5" Width="40" Height="42" TabSkip="no" Text="LogoEpicGames" />
				<Control Id="HeaderTitle" Type="Text" X="20" Y="15" Width="600" Height="35"  Transparent="yes" Text="{\TitleBanner}!(loc.GeneralTitleDialogWizard)"></Control>
				<!-- / Header -->

				<!-- Left Container -->
				<Control Id="Image1" Type="Bitmap" X="20" Y="75" Width="450" Height="253" TabSkip="no" Text="Background1" />
				<Control Id="Caption1" Type="Text" X="20" Y="330" Width="210" Height="75"  Transparent="yes"  Text="!(loc.BackGroundCredits1)"/>
				<!-- /Left Container -->
				<!--  Bottom Control Button -->
				<Control Id="BottomLine" Type="Line" X="0" Y="400" Width="800" Height="0" />
				<Control Id="VersionNumber" Type="Text" X="20" Y="416" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />
				<Control Id="Next" Type="PushButton"  X="640" Y="416" Width="56" Height="17" Default="yes" Text="!(loc.WixUINext)">
					<Publish Event="NewDialog" Value="MyMaintenanceTypeDlg"></Publish>
				</Control>
				<!--  / Bottom Control Button -->
			</Dialog>
			<InstallUISequence>
				<Show Dialog="MyMaintenanceTypeDlg" Before="ProgressDlg" Overridable="yes">Installed AND NOT RESUME AND NOT Preselected AND NOT PATCH</Show>
			</InstallUISequence>
			<Dialog Id="MyMaintenanceTypeDlg" Width="800" Height="450" Title="!(loc.MaintenanceTypeDlg_Title)">
				<!-- Header-->
				<Control Id="BannerVoid" Type="Bitmap" X="0" Y="0" Width="800" Height="50" TabSkip="no" Text="BannerFill" />
				<Control Id="BannerLogo" Type="Bitmap" X="736" Y="5" Width="40" Height="42" TabSkip="no" Text="LogoEpicGames" />
				<Control Id="HeaderTitle" Type="Text" X="20" Y="15" Width="600" Height="35"  Transparent="yes" Text="{\TitleBanner}Unreal Datasmith Exporter for Revit"></Control>
				<!-- /Header-->

				<!-- Left Container -->
				<Control Id="Image1" Type="Bitmap" X="20" Y="75" Width="450" Height="253" TabSkip="no" Text="Background4" />
				<Control Id="Caption1" Type="Text" X="20" Y="330" Width="210" Height="75"  Transparent="yes"  Text="!(loc.BackGroundCredits4)"/>
				<!-- /Left Container-->

				<!-- Right -->
				<Control Id="MaintenanceDialogDescriptionTitle" Type="Text" X="485" Y="75" Width="270" Height="30" Transparent="yes" NoPrefix="yes" Text="!(loc.ManagementPannelMessageAlreadyInstall)" />
				<Control Id="MaintenanceDialogDescriptionText" Type="Text" X="485" Y="95" Width="270" Height="30" Transparent="yes" NoPrefix="yes" Text="!(loc.ManagementPannelMessageActions)" />

				<!--  Bottom Control Button -->
				<Control Id="BottomLine" Type="Line" X="0" Y="400" Width="800" Height="0" />
				<Control Id="VersionNumber" Type="Text" X="20" Y="416" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />

				<Control Id="Finish" Type="PushButton" X="720" Y="415" Width="56" Height="17" Default="yes" Cancel="yes" Text="!(loc.WixUIFinish)">
					<Publish Event="EndDialog" Value="Exit">1</Publish>
				</Control>
				<!--  / Bottom Control Button -->
			</Dialog>

			<Publish Dialog="MyMaintenanceTypeDlg" Control="ChangeButton" Event="NewDialog" Value="CustomInstallDirDlg">1</Publish>
			<Publish Dialog="MyMaintenanceTypeDlg" Control="RepairButton" Event="NewDialog" Value="VerifyReadyDlg">1</Publish>
			<Publish Dialog="MyMaintenanceTypeDlg" Control="RemoveButton" Event="NewDialog" Value="VerifyReadyDlg">1</Publish>
			<Publish Dialog="MyMaintenanceTypeDlg" Control="Back" Event="NewDialog" Value="MyMaintenanceWelcomeDlg">1</Publish>
			<Publish Dialog="MyExitDialog" Control="Finish" Event="DoAction" Value="LaunchApplication">WIXUI_EXITDIALOGOPTIONALCHECKBOX = 1</Publish>
			<Publish Dialog="BrowseDlg" Control="OK" Event="DoAction" Value="WixUIValidatePath" Order="3">1</Publish>
			<Publish Dialog="BrowseDlg" Control="OK" Event="SpawnDialog" Value="InvalidDirDlg" Order="4"><![CDATA[WIXUI_INSTALLDIR_VALID<>"1"]]></Publish>
			<Publish Dialog="CustomWelcomeDialog" Control="Next" Event="NewDialog" Value="CustomInstallDirDlg">NOT Installed</Publish>
			<Publish Dialog="CustomWelcomeDialog" Control="Next" Event="NewDialog" Value="CustomLicenseAgreementDlg">NOT Installed</Publish>
			<Publish Dialog="CustomLicenseAgreementDlg" Control="Next" Event="NewDialog" Value="CustomInstallDirDlg">LicenseAccepted = "1"</Publish>
			<Publish Dialog="CustomLicenseAgreementDlg" Control="Next" Event="NewDialog" Value="CustomInstallDirDlg">Installed</Publish>
			<!-- Even though we don't use the InstallDirDlg, we still need to keep around the control setup to prevent a compile time error. -->
			<Publish Dialog="InstallDirDlg" Control="Back" Event="SpawnDialog" Value="CancelDlg">1</Publish>
			<Publish Dialog="InstallDirDlg" Control="Next" Event="SpawnDialog" Value="CancelDlg" Order="1">1</Publish>
			<Publish Dialog="InstallDirDlg" Control="ChangeFolder" Event="SpawnDialog" Value="CancelDlg" Order="2">1</Publish>
			<Publish Dialog="CustomInstallDirDlg" Control="Next" Event="SetTargetPath" Value="[WIXUI_INSTALLDIR]" Order="1">1</Publish>
			<Publish Dialog="CustomInstallDirDlg" Control="Next" Event="DoAction" Value="WixUIValidatePath" Order="2">NOT WIXUI_DONTVALIDATEPATH</Publish>
			<Publish Dialog="CustomInstallDirDlg" Control="Next" Event="SpawnDialog" Value="InvalidDirDlg" Order="3"><![CDATA[NOT WIXUI_DONTVALIDATEPATH AND WIXUI_INSTALLDIR_VALID<>"1"]]></Publish>
			<Publish Dialog="CustomInstallDirDlg" Control="Next" Event="EndDialog" Value="Return" Order="5"><![CDATA[OutOfDiskSpace <> 1]]></Publish>
			<Publish Dialog="CustomInstallDirDlg" Control="Next" Event="SpawnDialog" Value="OutOfRbDiskDlg" Order="6">OutOfDiskSpace = 1 AND OutOfNoRbDiskSpace = 0 AND (PROMPTROLLBACKCOST="P" OR NOT PROMPTROLLBACKCOST)</Publish>
			<Publish Dialog="CustomInstallDirDlg" Control="Next" Event="EndDialog" Value="Return" Order="7">OutOfDiskSpace = 1 AND OutOfNoRbDiskSpace = 0 AND PROMPTROLLBACKCOST="D"</Publish>
			<Publish Dialog="CustomInstallDirDlg" Control="Next" Event="EnableRollback" Value="False" Order="8">OutOfDiskSpace = 1 AND OutOfNoRbDiskSpace = 0 AND PROMPTROLLBACKCOST="D"</Publish>
			<Publish Dialog="CustomInstallDirDlg" Control="Next" Event="SpawnDialog" Value="OutOfDiskDlg" Order="9">(OutOfDiskSpace = 1 AND OutOfNoRbDiskSpace = 1) OR (OutOfDiskSpace = 1 AND PROMPTROLLBACKCOST="F")</Publish>
			<Publish Dialog="VerifyReadyDlg" Control="Back" Event="NewDialog" Value="CustomInstallDirDlg" Order="1">NOT Installed</Publish>
			<Publish Dialog="VerifyReadyDlg" Control="Back" Event="NewDialog" Value="CustomWelcomeDialog" Order="2">Installed</Publish>
			<Publish Dialog="VerifyReadyDlg" Control="Back" Event="NewDialog" Value="MyMaintenanceTypeDlg" Order="4">WixUI_InstallMode = "Repair" OR WixUI_InstallMode = "Remove"</Publish>
			<Property Id="ARPNOMODIFY" Value="1" />
		</UI>
		<UIRef Id="WixUI_Common" />
	</Fragment>
</Wix>
