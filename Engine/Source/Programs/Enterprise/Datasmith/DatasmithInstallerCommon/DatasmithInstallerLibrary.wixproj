<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="4.0" DefaultTargets="Build" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <PropertyGroup>
    <EngineSourcePath>..\..\..\..\</EngineSourcePath>
    <PrereqPath>$(EngineSourcePath)Programs\PrereqInstaller\</PrereqPath>
    <WixToolPath>$(EngineSourcePath)ThirdParty\WiX\3.8\</WixToolPath>
    <WixTargetsPath>$(WixToolPath)Wix.targets</WixTargetsPath>
    <WixTasksPath>$(WixToolPath)wixtasks.dll</WixTasksPath>
    <SuppressSpecificWarnings>
        1076;1077
    </SuppressSpecificWarnings>
    <DefineConstants>CoreStylePath=..\..\..\..\..\Content\Slate;D3DShadersPath=..\..\..\..\..\Shaders\StandaloneRenderer\D3D</DefineConstants>
  </PropertyGroup>
  <PropertyGroup>
    <Configuration Condition=" '$(Configuration)' == '' ">Debug</Configuration>
    <Platform Condition=" '$(Platform)' == '' ">x86</Platform>
    <ProductVersion>3.10</ProductVersion>
    <ProjectGuid>d3974bd7-9ee2-4821-8384-6389cdad19bc</ProjectGuid>
    <SchemaVersion>2.0</SchemaVersion>
    <OutputName>DatasmithInstallerLibrary</OutputName>
    <OutputType>Library</OutputType>
    <WixTasksPath>wixtasks.dll</WixTasksPath>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|x86' ">
    <OutputPath>bin\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Configuration)\</IntermediateOutputPath>
    <DefineConstants>Debug</DefineConstants>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|x86' ">
    <OutputPath>$(SolutionDir)\lib\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Configuration)\</IntermediateOutputPath>
    <SuppressIces>ICE30;ICE82;ICE03;ICE60</SuppressIces>
    <LeaveTemporaryFiles>False</LeaveTemporaryFiles>
    <SuppressPdbOutput>True</SuppressPdbOutput>
    <VerboseOutput>True</VerboseOutput>
    <LibBindFiles>True</LibBindFiles>
    <DefineConstants>CoreStylePath=..\..\..\..\..\Content\Slate;D3DShadersPath=..\..\..\..\..\Shaders\StandaloneRenderer\D3D</DefineConstants>
    <Cultures>en-us;fr-fr;de-de;es-es;ja-jp;ko-kr;pt-pt;zh-cn</Cultures>
  </PropertyGroup>
  <ItemGroup>
    <Compile Include="CustomActions.wxs" />
    <Compile Include="CustomExitDialog.wxs" />
    <Compile Include="CustomLicenseAgreementDlg.wxs" />
    <Compile Include="CustomWelcomeDialog.wxs" />
    <Compile Include="CustomWixUI_InstallDir.wxs" />
    <Compile Include="GenericUiBrandDialog.wxs" />
    <HarvestDirectory Include="..\..\..\..\..\Content\Slate">
      <DirectoryRefId>CoreStyleFolder</DirectoryRefId>
      <ComponentGroupName>CoreStyle</ComponentGroupName>
      <PreprocessorVariable>var.CoreStylePath</PreprocessorVariable>
      <Link>Slate</Link>
    </HarvestDirectory>
    <HarvestDirectory Include="..\..\..\..\..\Shaders\StandaloneRenderer\D3D">
      <DirectoryRefId>D3DShadersFolder</DirectoryRefId>
      <ComponentGroupName>D3DShaders</ComponentGroupName>
      <PreprocessorVariable>var.D3DShadersPath</PreprocessorVariable>
      <Link>D3D</Link>
    </HarvestDirectory>
  </ItemGroup>
  <ItemGroup>
    <WixExtension Include="WixUtilExtension">
      <HintPath>$(WixToolPath)WixUtilExtension.dll</HintPath>
      <Name>WixUtilExtension</Name>
    </WixExtension>
    <WixExtension Include="WixUIExtension">
      <HintPath>$(WixToolPath)WixUIExtension.dll</HintPath>
      <Name>WixUIExtension</Name>
    </WixExtension>
    <WixExtension Include="WixNetFxExtension">
      <HintPath>$(WixToolPath)WixNetFxExtension.dll</HintPath>
      <Name>WixNetFxExtension</Name>
    </WixExtension>
  </ItemGroup>
  <ItemGroup>
    <Folder Include="Localization" />
    <Folder Include="Resources" />
    <Folder Include="Resources\Images" />
    <Folder Include="Resources\Images\Background" />
    <Folder Include="Resources\Images\General" />
  </ItemGroup>
  <ItemGroup>
    <Content Include="Resources\Banner.bmp" />
    <Content Include="Resources\Dialog.bmp" />
    <Content Include="Resources\Exporter Plugin End User License Agreement.rtf" />
    <Content Include="Resources\Images\Background\28190.bmp" />
    <Content Include="Resources\Images\Background\28402.bmp" />
    <Content Include="Resources\Images\Background\28674.bmp" />
    <Content Include="Resources\Images\Background\30668.bmp" />
    <Content Include="Resources\Images\Background\30725.bmp" />
    <Content Include="Resources\Images\Epic_Games_logo.bmp" />
    <Content Include="Resources\Images\General\banner.bmp" />
    <Content Include="Resources\Images\General\unreal-logo.jpg" />
    <Content Include="Resources\UnrealEngine.ico" />
  </ItemGroup>
  <ItemGroup>
    <EmbeddedResource Include="Localization\en-us.wxl" />
    <EmbeddedResource Include="Localization\de-de.wxl" />
    <EmbeddedResource Include="Localization\es-es.wxl" />
    <EmbeddedResource Include="Localization\fr-fr.wxl" />
    <EmbeddedResource Include="Localization\ja-jp.wxl" />
    <EmbeddedResource Include="Localization\ko-kr.wxl" />
    <EmbeddedResource Include="Localization\pt-pt.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_es-es.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_fr-fr.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_ja-jp.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_ko-kr.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_pt-pt.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_zh-cn.wxl" />
    <EmbeddedResource Include="Localization\zh-cn.wxl" />
  </ItemGroup>
  <Import Project="$(WixTargetsPath)" Condition=" '$(WixTargetsPath)' != '' " />
  <PropertyGroup>
    <PostBuildEvent />
  </PropertyGroup>
  <!--
	To modify your build process, add your task inside one of the targets below and uncomment it.
	Other similar extension points exist, see Wix.targets.
	<Target Name="BeforeBuild">
	</Target>
	<Target Name="AfterBuild">
	</Target>
	-->
</Project>