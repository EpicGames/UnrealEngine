<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi">
	<Fragment>
		<UI>

			<!-- Defining Font -->
			<TextStyle Id="TitleBanner" FaceName="Tahoma" Size="16" Red="255" Green="255" Blue="255"/>
			<TextStyle Id="Strong" FaceName="Tahoma" Size="12" Red="15" Green="15" Blue="15"/>
			<TextStyle Id="Para" FaceName="Tahoma" Size="10" Red="15" Green="15" Blue="15"/>

			<!--Define our Binaries assets -->
			<!-- The Background -->
			<Binary Id="Background1" SourceFile="Resources/Images/Background/30725.bmp"/>
			<Binary Id="Background2" SourceFile="Resources/Images/Background/30668.bmp"/>
			<Binary Id="Background3" SourceFile="Resources/Images/Background/28674.bmp"/>
			<Binary Id="Background4" SourceFile="Resources/Images/Background/28402.bmp"/>
			<Binary Id="Background5" SourceFile="Resources/Images/Background/28190.bmp"/>

			<!-- General Epic Medias-->
			<Binary Id="BannerFill" SourceFile="Resources/Images/General/banner.bmp"/>
			<Binary Id="LogoEpicGames" SourceFile="Resources/Images/Epic_Games_logo.bmp"/>

		</UI>
	</Fragment>
</Wix>
