<?xml version="1.0" encoding="utf-8"?>
<!-- Copyright (c) .NET Foundation and contributors. All rights reserved. Licensed under the Microsoft Reciprocal License. See LICENSE.TXT file in the project root for full license information. -->

<WixLocalization Culture="pt-pt" xmlns="http://schemas.microsoft.com/wix/2006/localization">
    <String Id="msierrUSRFailedUserCreate" Overridable="yes">Falha ao criar usuário. ([2] [3] [4] [5])</String>
    <String Id="msierrUSRFailedUserCreatePswd" Overridable="yes">Falha ao criar usuário devido a senha inválida. ([2] [3] [4] [5])</String>
    <String Id="msierrUSRFailedUserGroupAdd" Overridable="yes">Falha ao adicionar o usuário ao grupo. ([2] [3] [4] [5])</String>
    <String Id="msierrUSRFailedUserCreateExists" Overridable="yes">Falha ao criar o usuário, porque ele já existe. ([2] [3] [4] [5])</String>
    <String Id="msierrSMBFailedCreate" Overridable="yes">Falha ao criar o compartilhamento de rede. ([2] [3] [4] [5])</String>
    <String Id="msierrSMBFailedDrop" Overridable="yes">Falha ao cair compartilhamento de rede. ([2] [3] [4] [5])</String>
    <String Id="msierrPERFMONFailedRegisterDLL" Overridable="yes">Falha ao registrar DLL com PerfMon. ([2] [3] [4] [5])</String>
    <String Id="msierrPERFMONFailedUnregisterDLL" Overridable="yes">Falha ao cancelar o registro de DLL com PerfMon. ([2] [3] [4] [5])</String>
    <String Id="msierrInstallPerfCounterData" Overridable="yes">Falha ao instalar contadores de desempenho. ([2] [3] [4] [5])</String>
    <String Id="msierrUninstallPerfCounterData" Overridable="yes">Falha ao desinstalar contadores de desempenho. ([2] [3] [4] [5])</String>
    <String Id="msierrSecureObjectsFailedCreateSD" Overridable="yes">Falha ao criar o descritor de segurança [3] \ [4], erro do sistema: [2]</String>
    <String Id="msierrSecureObjectsFailedSet" Overridable="yes">Falha ao definir o descritor de segurança sobre o objeto [3], erro do sistema: [2]</String>
    <String Id="msierrSecureObjectsUnknownType" Overridable="yes">Objeto Desconhecido Tipo [3], erro do sistema: [2]</String>
    <String Id="msierrXmlFileFailedRead" Overridable="yes">Houve uma falha ao configurar arquivos XML.</String>
    <String Id="msierrXmlFileFailedOpen" Overridable="yes">Falha ao abrir o arquivo XML [3], erro do sistema: [2]</String>
    <String Id="msierrXmlFileFailedSelect" Overridable="yes">Falha ao localizar nó: [3] no arquivo XML: [4], erro do sistema: [2]</String>
    <String Id="msierrXmlFileFailedSave" Overridable="yes">Falha ao salvar as alterações para o arquivo XML [3], erro do sistema: [2]</String>
    <String Id="msierrXmlConfigFailedRead" Overridable="yes">Houve uma falha ao configurar arquivos XML.</String>
    <String Id="msierrXmlConfigFailedOpen" Overridable="yes">Falha ao abrir o arquivo XML [3], erro do sistema: [2]</String>
    <String Id="msierrXmlConfigFailedSelect" Overridable="yes">Falha ao localizar nó: [3] no arquivo XML: [4], erro do sistema: [2]</String>
    <String Id="msierrXmlConfigFailedSave" Overridable="yes">Falha ao salvar as alterações para o arquivo XML [3], erro do sistema: [2]</String>
</WixLocalization>
