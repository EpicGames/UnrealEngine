<?xml version="1.0" encoding="utf-8"?>
<!-- Copyright (c) .NET Foundation and contributors. All rights reserved. Licensed under the Microsoft Reciprocal License. See LICENSE.TXT file in the project root for full license information. -->

<WixLocalization Culture="zh-cn" xmlns="http://schemas.microsoft.com/wix/2006/localization">
    <String Id="msierrUSRFailedUserCreate" Overridable="yes">Failed to create user.  ([2]   [3]   [4]   [5])</String>
    <String Id="msierrUSRFailedUserCreatePswd" Overridable="yes">Failed to create user due to invalid password.  ([2]   [3]   [4]   [5])</String>
    <String Id="msierrUSRFailedUserGroupAdd" Overridable="yes">Failed to add user to group.  ([2]   [3]   [4]   [5])</String>
    <String Id="msierrUSRFailedUserCreateExists" Overridable="yes">Failed to create user because it already exists.  ([2]   [3]   [4]   [5])</String>

    <String Id="msierrSMBFailedCreate" Overridable="yes">Failed to create network share.  ([2]   [3]   [4]   [5])</String>
    <String Id="msierrSMBFailedDrop" Overridable="yes">Failed to drop network share.  ([2]   [3]   [4]   [5])</String>

    <String Id="msierrPERFMONFailedRegisterDLL" Overridable="yes">Failed to register DLL with PerfMon.  ([2]   [3]   [4]   [5])</String>
    <String Id="msierrPERFMONFailedUnregisterDLL" Overridable="yes">Failed to unregister DLL with PerfMon.  ([2]   [3]   [4]   [5])</String>

    <String Id="msierrInstallPerfCounterData" Overridable="yes">Failed to install performance counters.  ([2]   [3]   [4]   [5])</String>
    <String Id="msierrUninstallPerfCounterData" Overridable="yes">Failed to uninstall performance counters.  ([2]   [3]   [4]   [5])</String>

    <String Id="msierrSecureObjectsFailedCreateSD" Overridable="yes">Failed to create security descriptor for [3]\[4], system error: [2]</String>
    <String Id="msierrSecureObjectsFailedSet" Overridable="yes">Failed to set security descriptor on object [3], system error: [2]</String>
    <String Id="msierrSecureObjectsUnknownType" Overridable="yes">Unknown Object Type [3], system error: [2]</String>

    <String Id="msierrXmlFileFailedRead" Overridable="yes">There was a failure while configuring XML files.</String>
    <String Id="msierrXmlFileFailedOpen" Overridable="yes">Failed to open XML file [3], system error: [2]</String>
    <String Id="msierrXmlFileFailedSelect" Overridable="yes">Failed to find node: [3] in XML file: [4], system error: [2]</String>
    <String Id="msierrXmlFileFailedSave" Overridable="yes">Failed to save changes to XML file [3], system error: [2]</String>

    <String Id="msierrXmlConfigFailedRead" Overridable="yes">There was a failure while configuring XML files.</String>
    <String Id="msierrXmlConfigFailedOpen" Overridable="yes">Failed to open XML file [3], system error: [2]</String>
    <String Id="msierrXmlConfigFailedSelect" Overridable="yes">Failed to find node: [3] in XML file: [4], system error: [2]</String>
    <String Id="msierrXmlConfigFailedSave" Overridable="yes">Failed to save changes to XML file [3], system error: [2]</String>
</WixLocalization>
