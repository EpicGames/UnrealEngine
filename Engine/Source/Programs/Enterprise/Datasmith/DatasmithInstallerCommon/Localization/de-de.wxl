<?xml version="1.0" encoding="utf-8"?>
<WixLocalization Culture="de-de" xmlns="http://schemas.microsoft.com/wix/2006/localization">

	<String Id="ProductName" Overridable="yes">Bahur Hack Team</String>
	
	<String Id="ErrorOlderVersionInstall" Overridable="yes">Eine neuere Version von [ProductName] ist bereits installiert.</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">Eine identische Version von [ProductName] ist bereits installiert. Deinstalliere zunächst diese Version, um fortzufahren.</String>

	<String Id="ErrorNotInstalled" Overridable="yes">Es wurde keine unterstützte Version von [HostApp] gefunden.</String>
	<String Id="ErrorSupportedVersion" Overridable="yes">
		Folgende Versionen von [HostApp] werden unterstützt:
		[HostAppSupportedVersions]
	</String>

	<String Id="ViewInstallationLog" Overridable="yes">View Installation Log</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">Es wurden keine unterstützten Versionen von [HostApp] ausgewählt. Das Installationsprogramm wird beendet. Es wurden keine Änderungen am System vorgenommen.</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">{\CP_Font_Desc}Klicke auf „Installieren“, um fortzufahren.</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">{\CP_Font_Desc}Klicke auf „Installieren“, um fortzufahren.</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">[ProductName] installieren für:</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">[ProductName] aktualisieren für:</String>

	<!-- General Dialog Item-->
	<String Id="GeneralTitleDialogWizard" Overridable="yes">Setup-Assistent für [ProductName]</String>
	<String Id="DirectXInstallMessage">DirectX wird installiert</String>

	<!-- Welcome Dialog -->
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">Der Setup-Assistent installiert den [ProductName] auf deinem Computer.</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">Klicke auf „Weiter“, um fortzufahren, oder auf „Abbrechen“, um den Setup-Assistenten zu verlassen.</String>

	<!-- Liscence Aggreement window -->
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">Endbenutzer-Lizenzvereinbarung</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">Bitte lies die folgende Lizenzvereinbarung sorgfältig durch.</String>

	<!-- Installation Choice Dialog -->
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">Endbenutzer-Lizenzvereinbarung</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">Bitte lies die folgende Lizenzvereinbarung sorgfältig durch.</String>

	<!-- Install process Dialog -->
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">[ProductName] wird installiert</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">Bitte warte, bis der Setup-Assistent die Installation abgeschlossen hat.</String>

	<!-- EndDialog -->
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">Installation abgeschlossen</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">Die Installation wurde erfolgreich abgeschlossen.</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">Klicke auf „Beenden“, um den Setup-Assistenten zu verlassen.</String>

	<!-- Maintenance Pannel -->
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">Das Produkt ist bereits installiert.</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">Bitte verwende die Windows-Systemsteuerung, um das Programm zu deinstallieren oder zu ändern.</String>
	<!-- Background image Creds-->
	<String Id="BackGroundCredits1" Overridable="yes">Bild mit freundlicher Genehmigung von Michael Mong</String>
	<String Id="BackGroundCredits2" Overridable="yes">Bild mit freundlicher Genehmigung von Peter Sivi / Motionland</String>
	<String Id="BackGroundCredits3" Overridable="yes">Bild mit freundlicher Genehmigung von Chen3d.com</String>
	<String Id="BackGroundCredits4" Overridable="yes">Bild mit freundlicher Genehmigung von Hoang Anh Ho</String>
	<String Id="BackGroundCredits5" Overridable="yes">Bild mit freundlicher Genehmigung der Architekten von Entre+</String>
	<!-- Defines a relative path to directory of DirectX redist resources	-->
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>

</WixLocalization>
