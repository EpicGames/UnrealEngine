<?xml version="1.0" encoding="utf-8"?>
<WixLocalization Culture="en-us" xmlns="http://schemas.microsoft.com/wix/2006/localization">
  
  <String Id="ErrorOlderVersionInstall" Overridable="yes">Newer version of [ProductName] already installed.</String>
  
  <String Id="ErrorSameVersionInstall" Overridable="yes">Identical version of [ProductName] already installed. Uninstall this version first before proceeding.</String>
  
  <String Id="ErrorNotInstalled" Overridable="yes">No supported version of [HostApp] has been found.</String>
  <String Id="ErrorSupportedVersion" Overridable="yes">
	  The supported versions of [HostApp] are: 
	  [HostAppSupportedVersions]
  </String>
  
  <String Id="ErrorNothingToInstall" Overridable="yes">No supported versions of [HostApp] have been selected. The installer will exit without modifying your system.</String>
  
  <String Id="InstallDirDlgDescription" Overridable="yes">{\CP_Font_Desc}Click Install to continue.</String>
  <String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">{\CP_Font_Desc}Click Install to continue.</String>
  <String Id="InstallDirDlgMessage" Overridable="yes">Install [ProductName] for:</String>
  <String Id="InstallDirDlgMessageUpgrade" Overridable="yes">Update [ProductName] for:</String>
	
  <!-- General Dialog Item-->
  <String Id="GeneralTitleDialogWizard" Overridable="yes">[ProductName] Setup Wizard</String>
  <String Id="DirectXInstallMessage">Installing DirectX</String>

  <!-- Welcome Dialog -->
  <String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">The Setup Wizard will install [ProductName] on your computer</String>
  <String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">Click Next to continue or cancel to exit the Setup Wizard</String>

  <!-- Liscence Aggreement window -->
  <String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">End User License Agreement</String>
  <String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">Please read the following license agreement carefully</String>

  <!-- Installation Choice Dialog -->
  <String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">End User License Agreement</String>
  <String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">Please read the following license agreement carefully</String>

  <!-- Install process Dialog -->
  <String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">Installing [ProductName]</String>
  <String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">Please wait while the Setup Wizard completes the installation.</String>

  <!-- EndDialog -->
  <String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">Installation Complete</String>
  <String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">The installation has been completed successfully.</String>
  <String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">Click the Finish button to exit the Setup Wizard.</String>

  <!-- Maintenance Pannel -->
  <String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">Product already installed.</String>
  <String Id="ManagementPannelMessageActions" Overridable="yes">Please use Windows control panel to uninstall or change the program.</String>
  <!-- Background image Creds-->
  <String Id="BackGroundCredits1" Overridable="yes">Image courtesy of Michael Mong</String>
  <String Id="BackGroundCredits2" Overridable="yes">Image courtesy of Peter Sivi / Motionland</String>
  <String Id="BackGroundCredits3" Overridable="yes">Image courtesy of Chen3d.com</String>
  <String Id="BackGroundCredits4" Overridable="yes">Image courtesy of Hoang Anh Ho</String>
  <String Id="BackGroundCredits5" Overridable="yes">Image courtesy of Entre+ architects</String>  
  <!-- Defines a relative path to directory of DirectX redist resources	-->
  <String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>
  
</WixLocalization>
