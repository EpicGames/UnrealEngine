<?xml version="1.0" encoding="utf-8"?>
<WixLocalization Culture="es-es" xmlns="http://schemas.microsoft.com/wix/2006/localization">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">Ya hay instalada una versión más reciente de [ProductName].</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">Ya hay instalada una versión idéntica de [ProductName]. Desinstala esta versión antes de continuar.</String>

	<String Id="ErrorNotInstalled" Overridable="yes">No se ha encontrado ninguna versión compatible de [HostApp].</String>
	<String Id="ErrorSupportedVersion" Overridable="yes">
		Las versiones compatibles de [HostApp] son:
		[HostAppSupportedVersions]
	</String>

	<String Id="ViewInstallationLog" Overridable="yes">View Installation Log</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">No se ha seleccionado ninguna versión compatible de [HostApp]. Se va a cerrar el instalador sin modificar tu sistema.</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">{\CP_Font_Desc}Haz clic en Instalar para continuar.</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">{\CP_Font_Desc}Haz clic en Instalar para continuar.</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">Instalar [ProductName] para:</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">Actualizar [ProductName] para:</String>

	<!-- General Dialog Item-->
	<String Id="GeneralTitleDialogWizard" Overridable="yes">Unreal Datasmith Exporter para el asistente de configuración de [HostApp]</String>
	<String Id="DirectXInstallMessage">Instalando DirectX</String>

	<!-- Welcome Dialog -->
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">El asistente de configuración instalará [ProductName] en tu PC.</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">Haz clic en Siguiente para continuar o en Cancelar para salir del asistente de configuración.</String>

	<!-- Liscence Aggreement window -->
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">Acuerdo de Licencia de Usuario Final</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">Lee detenidamente el siguiente acuerdo de licencia.</String>

	<!-- Installation Choice Dialog -->
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">End User License Agreement</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">Please read the following license agreement carefully</String>

	<!-- Install process Dialog -->
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">Acuerdo de Licencia de Usuario Final</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">La instalación se ha completado correctamente.</String>

	<!-- EndDialog -->
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">Instalación completada.</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">La instalación se ha completado correctamente.</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">Haz clic en el botón Finalizar para salir del asistente de configuración.</String>

	<!-- Maintenance Pannel -->
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">El producto ya está instalado.</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">Usa el panel de control de Windows para desinstalar o cambiar el programa.</String>
	<!-- Background image Creds-->
	<String Id="BackGroundCredits1" Overridable="yes">Imagen proporcionada por Michael Mong</String>
	<String Id="BackGroundCredits2" Overridable="yes">Imagen proporcionada por Peter Sivi / Motionland</String>
	<String Id="BackGroundCredits3" Overridable="yes">Imagen proporcionada por Chen3d.com</String>
	<String Id="BackGroundCredits4" Overridable="yes">Imagen proporcionada por Hoang Anh Ho</String>
	<String Id="BackGroundCredits5" Overridable="yes">Imagen proporcionada por Entre+ architects</String>
	<!-- Defines a relative path to directory of DirectX redist resources	-->
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>

</WixLocalization>
