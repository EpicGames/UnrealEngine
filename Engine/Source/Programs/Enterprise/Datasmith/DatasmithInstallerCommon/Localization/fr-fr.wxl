<?xml version="1.0" encoding="utf-8"?>
<WixLocalization Culture="fr-fr" xmlns="http://schemas.microsoft.com/wix/2006/localization">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">Une version plus récente de [ProductName] est déjà installée.</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">Une version identique de [ProductName] est déjà installée. Désinstallez d’abord cette version avant de poursuivre.</String>

	<String Id="ErrorNotInstalled" Overridable="yes">Aucune version prise en charge de [HostApp] n’a été trouvée.</String>
	<String Id="ErrorSupportedVersion" Overridable="yes">
		Les versions prises en charge de [HostApp] sont les suivantes:
		[HostAppSupportedVersions]
	</String>

	<String Id="ViewInstallationLog" Overridable="yes">View Installation Log</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">Aucune version prise en charge de [HostApp] n’a été sélectionnée. Le programme d’installation se terminera sans modifier votre système.</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">{\CP_Font_Desc}Cliquez sur Installer pour continuer.</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">{\CP_Font_Desc}Cliquez sur Installer pour continuer.</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">Installer [ProductName] pour:</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">Mise à jour de [ProductName] pour:</String>

	<!-- General Dialog Item-->
	<String Id="GeneralTitleDialogWizard" Overridable="yes">[ProductName] pour l’assistant de configuration de [HostApp]</String>
	<String Id="DirectXInstallMessage">Installation de DirectX</String>

	<!-- Welcome Dialog -->
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">L’assistant de configuration va installer [ProductName] sur votre ordinateur</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">Cliquez sur Suivant pour continuer ou sur Annuler pour quitter l’assistant de configuration</String>

	<!-- Liscence Aggreement window -->
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">Contrat de licence de l’utilisateur final</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">Veuillez lire attentivement le contrat de licence suivant</String>

	<!-- Installation Choice Dialog -->
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">Contrat de licence de l’utilisateur final</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">Veuillez lire attentivement le contrat de licence suivant</String>

	<!-- Install process Dialog -->
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">Installation de [ProductName]</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">Veuillez patienter pendant que l’assistant de configuration termine l’installation.</String>

	<!-- EndDialog -->
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">Installation terminée</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">L’installation s’est déroulée avec succès.</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">Cliquez sur le bouton Terminer pour quitter l’assistant de configuration.</String>

	<!-- Maintenance Pannel -->
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">Produit déjà installé.</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">Veuillez utiliser le panneau de configuration de Windows pour désinstaller ou modifier le programme.</String>
	<!-- Background image Creds-->
	<String Id="BackGroundCredits1" Overridable="yes">Image reproduite avec l’aimable autorisation de Michael Mong</String>
	<String Id="BackGroundCredits2" Overridable="yes">Image reproduite avec l’aimable autorisation de Peter Sivi/Motionland</String>
	<String Id="BackGroundCredits3" Overridable="yes">Image reproduite avec l’aimable autorisation de Chen3d.com</String>
	<String Id="BackGroundCredits4" Overridable="yes">Image reproduite avec l’aimable autorisation de Hoang Anh Ho</String>
	<String Id="BackGroundCredits5" Overridable="yes">Image reproduite avec l’aimable autorisation de Entre+ architects</String>
	<!-- Defines a relative path to directory of DirectX redist resources	-->
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>

</WixLocalization>
