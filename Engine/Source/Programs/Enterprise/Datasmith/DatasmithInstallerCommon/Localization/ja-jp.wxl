<?xml version="1.0" encoding="utf-8"?>
<WixLocalization Culture="ja-jp" xmlns="http://schemas.microsoft.com/wix/2006/localization">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">[ProductName] の新しいバージョンがすでにインストールされています。</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">[ProductName] の同一バージョンがすでにインストールされています。このバージョンをアンインストールしてから続行してください。</String>

	<String Id="ErrorNotInstalled" Overridable="yes">サポートされている [HostApp] のバージョンが見つかりませんでした。</String>
	<String Id="ErrorSupportedVersion" Overridable="yes">
		サポートされている [HostApp] のバージョン:
		[HostAppSupportedVersions]
	</String>

	<String Id="ViewInstallationLog" Overridable="yes">View Installation Log</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">サポートされている [HostApp] のバージョンが選択されていません。インストーラーはシステムを変更せずに終了します。</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">{\CP_Font_Desc}「Install（インストール）」をクリックして続行します。</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">{\CP_Font_Desc}「Install（インストール）」をクリックして続行します。</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">[ProductName] インストール:</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">[ProductName] アップデート:</String>

	<!-- General Dialog Item-->
	<String Id="GeneralTitleDialogWizard" Overridable="yes">[ProductName] セットアップ ウィザード</String>
	<String Id="DirectXInstallMessage">DirectX のインストール</String>

	<!-- Welcome Dialog -->
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">セットアップウィザードは、ご使用のコンピューターに [ProductName]をインストールします。</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">「Next （次へ）」をクリックして続行するか、「Cancel（キャンセル）」をクリックしてセットアップウィザードを終了してください。</String>

	<!-- Liscence Aggreement window -->
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">エンドユーザーライセンス契約</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">以下のライセンス契約をよくお読みください。</String>

	<!-- Installation Choice Dialog -->
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">エンドユーザーライセンス契約</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">以下のライセンス契約をよくお読みください。</String>

	<!-- Install process Dialog -->
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">[ProductName] をインストールしています。</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">セットアップウィザードがインストールを完了するまでお待ちください。</String>

	<!-- EndDialog -->
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">インストール完了</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">インストールは正常に完了しました。</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">「Finish （完了）」ボタンをクリックしてセットアップウィザードを終了します。</String>

	<!-- Maintenance Pannel -->
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">製品はすでにインストールされています。</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">Windows のコントロールパネルを使用して、プログラムをアンインストールするか変更してください。</String>
	<!-- Background image Creds-->
	<String Id="BackGroundCredits1" Overridable="yes">Image courtesy of Michael Mong</String>
	<String Id="BackGroundCredits2" Overridable="yes">Image courtesy of Peter Sivi / Motionland</String>
	<String Id="BackGroundCredits3" Overridable="yes">Image courtesy of Chen3d.com</String>
	<String Id="BackGroundCredits4" Overridable="yes">Image courtesy of Hoang Anh Ho</String>
	<String Id="BackGroundCredits5" Overridable="yes">Image courtesy of Entre+ architects</String>
	<!-- Defines a relative path to directory of DirectX redist resources	-->
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>

</WixLocalization>
