<?xml version="1.0" encoding="utf-8"?>
<WixLocalization Culture="ko-kr" xmlns="http://schemas.microsoft.com/wix/2006/localization">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">[ProductName] 최신 버전이 이미 설치되어 있습니다.</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">[ProductName] 의 동일 버전이 이미 설치되어 있습니다. 진행하기 전에 먼저 이 버전을 제거하세요.</String>

	<String Id="ErrorNotInstalled" Overridable="yes">[HostApp] 지원 버전을 찾을 수 없습니다.</String>
	<String Id="ErrorSupportedVersion" Overridable="yes">
		[HostApp] 지원 버전은 다음과 같습니다:
		[HostAppSupportedVersions]
	</String>

	<String Id="ViewInstallationLog" Overridable="yes">View Installation Log</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">[HostApp] 지원 버전이 선택되지 않았습니다. 시스템 변경 없이 인스톨러가 종료됩니다.</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">{\CP_Font_Desc}계속하려면 설치를 눌러주세요.</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">{\CP_Font_Desc}계속하려면 설치를 눌러주세요.</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">[ProductName] 설치 대상:</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">[ProductName] 업데이트 대상:</String>

	<!-- General Dialog Item-->
	<String Id="GeneralTitleDialogWizard" Overridable="yes">[ProductName]용 언리얼 데이터스미스 익스포터 구성 마법사</String>
	<String Id="DirectXInstallMessage">DirectX 설치 중</String>

	<!-- Welcome Dialog -->
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">구성 마법사가 [ProductName]용 언리얼 데이터스미스 익스포터를 컴퓨터에 설치합니다.</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">계속을 클릭하여 설치를 진행하거나 취소를 선택하여 구성 마법사를 종료하세요.</String>

	<!-- Liscence Aggreement window -->
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">최종 사용자 라이선스 계약</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">다음 라이선스 계약을 자세히 살펴보세요.</String>

	<!-- Installation Choice Dialog -->
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">최종 사용자 라이선스 계약</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">다음 라이선스 계약을 자세히 살펴보세요.</String>

	<!-- Install process Dialog -->
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">[ProductName]용 언리얼 데이터스미스 익스포터 설치 중</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">구성 마법사가 설치를 완료하는 동안 잠시만 기다려 주세요.</String>

	<!-- EndDialog -->
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">설치 완료</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">설치가 성공적으로 완료되었습니다.</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">완료 버튼을 클릭하여 구성 마법사를 종료하세요.</String>

	<!-- Maintenance Pannel -->
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">제품이 이미 설치되어 있습니다.</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">Windows 제어판을 통해 프로그램을 제거하거나 변경하세요.</String>
	<!-- Background image Creds-->
	<String Id="BackGroundCredits1" Overridable="yes">Michael Mong 이미지 제공</String>
	<String Id="BackGroundCredits2" Overridable="yes">Peter Sivi / Motionland 이미지 제공</String>
	<String Id="BackGroundCredits3" Overridable="yes">Chen3d.com 이미지 제공</String>
	<String Id="BackGroundCredits4" Overridable="yes">Hoang Anh Ho 이미지 제공</String>
	<String Id="BackGroundCredits5" Overridable="yes">Entre+ architects 이미지 제공</String>
	<!-- Defines a relative path to directory of DirectX redist resources	-->
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>

</WixLocalization>
