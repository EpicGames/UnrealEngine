<?xml version="1.0" encoding="utf-8"?>
<WixLocalization Culture="pt-pt" xmlns="http://schemas.microsoft.com/wix/2006/localization">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">A versão mais recente de [ProductName] já está instalada.</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">Uma versão idêntica de [ProductName] já está instalada. Desinstale esta versão antes de continuar.</String>

	<String Id="ErrorNotInstalled" Overridable="yes">Nenhuma versão compatível de [HostApp] foi encontrada.</String>
	<String Id="ErrorSupportedVersion" Overridable="yes">
		As versões compatíveis do [HostApp] são:
		[HostAppSupportedVersions]
	</String>

	<String Id="ViewInstallationLog" Overridable="yes">View Installation Log</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">Nenhuma versão compatível de [HostApp] foi selecionada. O instalador será encerrado sem modificar seu sistema.</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">{\CP_Font_Desc}Clique em Instalar para continuar.</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">{\CP_Font_Desc}Clique em Instalar para continuar.</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">Instalar [ProductName] para:</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">Atualizar [ProductName] para:</String>

	<!-- General Dialog Item-->
	<String Id="GeneralTitleDialogWizard" Overridable="yes">Assistente de Instalação do [ProductName]</String>
	<String Id="DirectXInstallMessage">Instalando DirectX</String>

	<!-- Welcome Dialog -->
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">O Assistente de Instalação instalará o Unreal Datasmith Exporter para [ProductName] no seu computador</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">Clique em Próximo para continuar ou cancele para sair do Assistente de Instalação</String>

	<!-- Liscence Aggreement window -->
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">Contrato de Licença do Usuário Final</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">Leia com atenção o contrato de licença a seguir</String>

	<!-- Installation Choice Dialog -->
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">Contrato de Licença do Usuário Final</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">Leia com atenção o contrato de licença a seguir</String>

	<!-- Install process Dialog -->
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">Instalando [ProductName]</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">Aguarde enquanto o Assistente de Instalação conclui a instalação</String>

	<!-- EndDialog -->
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">Instalação concluída</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">A instalação foi concluída com êxito.</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">Clique no botão Concluir para sair do Assistente de Instalação.</String>

	<!-- Maintenance Pannel -->
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">O programa já está instalado.</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">Use o Painel de Controle do Windows para desinstalar ou alterar o programa.</String>
	<!-- Background image Creds-->
	<String Id="BackGroundCredits1" Overridable="yes">Imagem fornecida por Michael Mong</String>
	<String Id="BackGroundCredits2" Overridable="yes">Imagem fornecida por Peter Sivi / Motionland</String>
	<String Id="BackGroundCredits3" Overridable="yes">Imagem fornecida por Chen3d.com</String>
	<String Id="BackGroundCredits4" Overridable="yes">Imagem fornecida por Hoang Anh Ho</String>
	<String Id="BackGroundCredits5" Overridable="yes">Imagem fornecida por Entre+ architects</String>
	<!-- Defines a relative path to directory of DirectX redist resources	-->
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>

</WixLocalization>
