<?xml version="1.0" encoding="utf-8"?>
<WixLocalization Culture="zh-cn" xmlns="http://schemas.microsoft.com/wix/2006/localization">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">已安装了更高版本的[ProductName]。</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">已安装了相同版本的[ProductName]，请先卸载此版本以继续。</String>

	<String Id="ErrorNotInstalled" Overridable="yes">未找到支持的[HostApp]版本。</String>
	<String Id="ErrorSupportedVersion" Overridable="yes">
		支持的[HostApp]版本为：
		[HostAppSupportedVersions]
	</String>

	<String Id="ViewInstallationLog" Overridable="yes">View Installation Log</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">未选择支持的[HostApp]版本，将退出安装程序，且不会对你的项目做任何更改。</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">{\CP_Font_Desc}点击“安装”以继续。</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">{\CP_Font_Desc}点击“安装”以继续。</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">安装[ProductName]：</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">更新[ProductName]：</String>

	<!-- General Dialog Item-->
	<String Id="GeneralTitleDialogWizard" Overridable="yes">面向[ProductName]的虚幻Datasmith导出器设置向导</String>
	<String Id="DirectXInstallMessage">正在安装DirectX</String>

	<!-- Welcome Dialog -->
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">设置向导将在你的计算机上安装面向[ProductName]的虚幻Datasmith导出器</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">点击“下一步”以继续，或点击“取消”退出设置向导</String>

	<!-- Liscence Aggreement window -->
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">终端用户许可协议</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">请仔细阅读以下许可协议</String>

	<!-- Installation Choice Dialog -->
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">终端用户许可协议</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">请仔细阅读以下许可协议</String>

	<!-- Install process Dialog -->
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">正在安装面向[ProductName]的虚幻Datasmith导出器</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">设置向导正在完成安装，请稍候。</String>

	<!-- EndDialog -->
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">安装完成</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">安装已成功。</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">点击“完成”按钮退出设置向导。</String>

	<!-- Maintenance Pannel -->
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">产品已安装。</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">请使用Windows控制面板卸载或修改程序。</String>
	<!-- Background image Creds-->
	<String Id="BackGroundCredits1" Overridable="yes">图片由Michael Mong提供</String>
	<String Id="BackGroundCredits2" Overridable="yes">图片由Peter Sivi / Motionland提供</String>
	<String Id="BackGroundCredits3" Overridable="yes">图片由Chen3d.com提供</String>
	<String Id="BackGroundCredits4" Overridable="yes">图片由Hoang Anh Ho提供</String>
	<String Id="BackGroundCredits5" Overridable="yes">图片由Entre+ architects提供</String>
	<!-- Defines a relative path to directory of DirectX redist resources	-->
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>

</WixLocalization>
