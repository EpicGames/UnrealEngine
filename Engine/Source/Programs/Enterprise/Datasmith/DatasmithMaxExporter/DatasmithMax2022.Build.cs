// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public abstract class DatasmithMaxBase : ModuleRules
	{
		public DatasmithMaxBase(ReadOnlyTargetRules Target)
			: base(Target)
		{

			bUseRTTI = true;

			// To avoid clashes with Max SDK
			// todo: separate Slate code from 3ds max to a module
			bUseUnity = false;


			PublicDefinitions.Add("NEW_DIRECTLINK_PLUGIN=1");

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DatasmithExporter",
					"DatasmithExporterUI",

					"UdpMessaging", // required for DirectLink networking
					"UEOpenExr",

					"IntelTBB",
					"Slate",
					"SlateCore",
				}
			);

			PrivateIncludePathModuleNames.Add("Launch");

			// Max SDK setup
			{
				string MaxVersionString = GetMaxVersion();
				string MaxSDKLocation = "";

				// Try with build machine setup
				string SDKRootEnvVar = System.Environment.GetEnvironmentVariable("UE_SDKS_ROOT");
				if (SDKRootEnvVar != null && SDKRootEnvVar != "")
				{
					MaxSDKLocation = Path.Combine(SDKRootEnvVar, "HostWin64", "Win64", "3dsMax", MaxVersionString);
				}

				if (!Directory.Exists(MaxSDKLocation))
				{
					// Try with custom setup
					string MaxSDKEnvVar = System.Environment.GetEnvironmentVariable("ADSK_3DSMAX_SDK_" + MaxVersionString);
					if (MaxSDKEnvVar != null && MaxSDKEnvVar != "")
					{
						MaxSDKLocation = MaxSDKEnvVar;
					}
				}

				// Make sure this version of Max is actually installed
				if (Directory.Exists(MaxSDKLocation))
				{
					PublicSystemIncludePaths.Add(Path.Combine(MaxSDKLocation, "include"));

					string LibraryPaths = Path.Combine(MaxSDKLocation, "lib", "x64", "Release");
					PublicAdditionalLibraries.Add(Path.Combine(LibraryPaths, "assetmanagement.lib"));
					PublicAdditionalLibraries.Add(Path.Combine(LibraryPaths, "bmm.lib"));
					PublicAdditionalLibraries.Add(Path.Combine(LibraryPaths, "core.lib"));
					PublicAdditionalLibraries.Add(Path.Combine(LibraryPaths, "geom.lib"));
					PublicAdditionalLibraries.Add(Path.Combine(LibraryPaths, "maxutil.lib"));
					PublicAdditionalLibraries.Add(Path.Combine(LibraryPaths, "Maxscrpt.lib"));
					PublicAdditionalLibraries.Add(Path.Combine(LibraryPaths, "mesh.lib"));
				}
			}

			// Itoo ForestPack/RailClone API
			string ItooInterfaceLocation = Path.Combine(ModuleDirectory, "ThirdParty", "Itoo");
			bool bWithItooInterface = Directory.Exists(ItooInterfaceLocation);
			PublicDefinitions.Add("WITH_ITOO_INTERFACE=" + (bWithItooInterface ? "1" : "0"));
			if (bWithItooInterface)
			{
				PrivateIncludePaths.Add(ItooInterfaceLocation);
			}

			PrivateIncludePaths.Add(ModuleDirectory);

			// Only 3ds max version 2022 and higher is supported now.
			//if (int.Parse(GetMaxVersion()) < 2022)
			//{
			//	// Being cheeky here as suppressing the warning doesn't work in ModuleRules for some reason
			//	CppStandard = Enum.GetValues<CppStandardVersion>().Where(x => x.ToString().Contains("Cpp17")).First();
			//}
		}

		public abstract string GetMaxVersion();
	}

	[SupportedPlatforms("Win64")]
	public class DatasmithMax2022 : DatasmithMaxBase
	{
		public DatasmithMax2022(ReadOnlyTargetRules Target)
			: base(Target)
		{
		}
		public override string GetMaxVersion() { return "2022"; }
	}
}