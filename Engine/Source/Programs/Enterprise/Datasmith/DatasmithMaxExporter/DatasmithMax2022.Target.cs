// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.Collections.Generic;
using System.IO;

[SupportedPlatforms("Win64")]
public abstract class DatasmithMaxBaseTarget : TargetRules
{
	public DatasmithMaxBaseTarget(TargetInfo Target)
		: base(Target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		SolutionDirectory = "Programs/Datasmith";
		bBuildInSolutionByDefault = false;
		bLegalToDistributeBinary = true;

		bShouldCompileAsDLL = true;
		LinkType = TargetLinkType.Monolithic;

		WindowsPlatform.ModuleDefinitionFile = "Programs/Enterprise/Datasmith/DatasmithMaxExporter/DatasmithMaxExporterWithDirectLink.def";

		if(StaticAnalyzer != StaticAnalyzer.None)
		{
			string MaxSDKLocation = "";
			string SDKRootEnvVar = System.Environment.GetEnvironmentVariable("UE_SDKS_ROOT");
			if (SDKRootEnvVar != null && SDKRootEnvVar != "")
			{
				MaxSDKLocation = Path.Combine(SDKRootEnvVar, "HostWin64", "Win64", "3dsMax");
			}

			if (!Directory.Exists(MaxSDKLocation))
			{
			}

			WindowsPlatform.Compiler = WindowsCompiler.Clang;
			Console.WriteLine("Skipping " + MaxSDKLocation);
			AdditionalCompilerArguments = "-Xanalyzer,-isystem-after" + MaxSDKLocation;
		}
		else
		{
			WindowsPlatform.Compiler = WindowsCompiler.VisualStudio;
		}

		bBuildDeveloperTools = false;
		bBuildWithEditorOnlyData = true;
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = true;
		bCompileICU = false;
		bUsesSlate = false;

		bHasExports = true;
		bForceEnableExceptions = true;

		// For DirectLinkUI (see FDatasmithExporterManager::FInitOptions)
		GlobalDefinitions.Add("UE_EXTERNAL_PROFILING_ENABLED=0");
		GlobalDefinitions.Add("NO_INIUTIL_USING");

		// Disable CrashReporter
		GlobalDefinitions.Add("NOINITCRASHREPORTER=1");

		// todo: remove?
		// bSupportEditAndContinue = true;
	}

	protected void AddCopyPostBuildStep(TargetInfo Target)
	{
		// Add a post-build step that copies the output to a file with the .dle extension
		string OutputName = "$(TargetName)";
		if (Target.Configuration != UnrealTargetConfiguration.Development)
		{
			OutputName = string.Format("{0}-{1}-{2}", OutputName, Target.Platform, Target.Configuration);
		}

		string SrcOutputFileName = string.Format(@"$(EngineDir)\Binaries\Win64\{0}\{1}.dll", ExeBinariesSubFolder, OutputName);
		string DstOutputFileName = string.Format(@"$(EngineDir)\Binaries\Win64\{0}\{1}.gup", ExeBinariesSubFolder, OutputName);

		PostBuildSteps.Add(string.Format("echo Copying {0} to {1}...", SrcOutputFileName, DstOutputFileName));
		PostBuildSteps.Add(string.Format("copy /Y \"{0}\" \"{1}\" 1>nul", SrcOutputFileName, DstOutputFileName));

		PostBuildSteps.Add(string.Format("echo Copying {0} to {1}...", SrcOutputFileName, DstOutputFileName));
		PostBuildSteps.Add(string.Format("copy /Y \"{0}\" \"{1}\" 1>nul", SrcOutputFileName, DstOutputFileName));
	}
}

public class DatasmithMax2022Target : DatasmithMaxBaseTarget
{
	public DatasmithMax2022Target(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "DatasmithMax2022";
		ExeBinariesSubFolder = @"3DSMax\2022";



		AddCopyPostBuildStep(Target);
	}
}
