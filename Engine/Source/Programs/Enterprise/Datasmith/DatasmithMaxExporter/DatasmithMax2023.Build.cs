// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;

namespace UnrealBuildTool.Rules
{
	[SupportedPlatforms("Win64")]
	public class DatasmithMax2023 : DatasmithMaxBase
	{
		public DatasmithMax2023(ReadOnlyTargetRules Target)
			: base(Target)
		{
		}
		public override string GetMaxVersion() { return "2023"; }
	}
}