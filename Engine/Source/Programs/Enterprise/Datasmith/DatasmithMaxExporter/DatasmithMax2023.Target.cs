// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class DatasmithMax2023Target : DatasmithMaxBaseTarget
{
	public DatasmithMax2023Target(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "DatasmithMax2023";
		ExeBinariesSubFolder = @"3DSMax\2023";

		AddCopyPostBuildStep(Target);
	}
}
