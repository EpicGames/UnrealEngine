// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class DatasmithMax2024Target : DatasmithMaxBaseTarget
{
	public DatasmithMax2024Target(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "DatasmithMax2024";
		ExeBinariesSubFolder = @"3DSMax\2024";

		AddCopyPostBuildStep(Target);
	}
}
