// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;

namespace UnrealBuildTool.Rules
{
	[SupportedPlatforms("Win64")]
	public class DatasmithMax2025 : DatasmithMaxBase
	{
		public DatasmithMax2025(ReadOnlyTargetRules Target)
			: base(Target)
		{
		}
		public override string GetMaxVersion() { return "2025"; }
	}
}