// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class DatasmithMax2025Target : DatasmithMaxBaseTarget
{
	public DatasmithMax2025Target(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "DatasmithMax2025";
		ExeBinariesSubFolder = @"3DSMax\2025";

		AddCopyPostBuildStep(Target);
	}
}
