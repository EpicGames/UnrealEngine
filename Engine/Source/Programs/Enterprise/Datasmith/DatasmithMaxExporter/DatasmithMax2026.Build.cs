// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;

namespace UnrealBuildTool.Rules
{
	[SupportedPlatforms("Win64")]
	public class DatasmithMax2026 : DatasmithMaxBase
	{
		public DatasmithMax2026(ReadOnlyTargetRules Target)
			: base(Target)
		{
		}
		public override string GetMaxVersion() { return "2026"; }
	}
}