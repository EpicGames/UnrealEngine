// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class DatasmithMax2026Target : DatasmithMaxBaseTarget
{
	public DatasmithMax2026Target(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "DatasmithMax2026";
		ExeBinariesSubFolder = @"3DSMax\2026";

		AddCopyPostBuildStep(Target);
	}
}
