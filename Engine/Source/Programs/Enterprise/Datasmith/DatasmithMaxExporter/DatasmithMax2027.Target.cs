// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class DatasmithMax2027Target : DatasmithMaxBaseTarget
{
	public DatasmithMax2027Target(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "DatasmithMax2027";
		ExeBinariesSubFolder = @"3DSMax\2027";

		AddCopyPostBuildStep(Target);
	}
}
