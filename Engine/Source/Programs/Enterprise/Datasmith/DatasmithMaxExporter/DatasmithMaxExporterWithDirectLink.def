LIBRARY
EXPORTS
   LibDescription       @1       PRIVATE
   LibNumberClasses     @2       PRIVATE
   LibClassDesc         @3       PRIVATE
   LibVersion           @4       PRIVATE
   LibInitialize        @6       PRIVATE
   LibShutdown          @7       PRIVATE
SECTIONS
   .data READ WRITE
