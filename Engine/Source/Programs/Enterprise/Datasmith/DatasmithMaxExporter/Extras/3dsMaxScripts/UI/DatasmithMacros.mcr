Macroscript Datasmith_ExportOptions_AnimatedTransformsActiveSegment
category: ~DATASMITH_CATEGORY~
internalCategory: "Datasmith"
tooltip: ~DATASMITH_EXPORTOPTIONS_ANIMATEDTRANSFORMSACTIVESEGMENT_TOOLTIP~
autoUndoEnabled:false
(
	on execute do 
	(
		Datasmith_SetExportOption_AnimatedTransforms true
		updateToolbarButtons()
	)
)


Macroscript Datasmith_ExportOptions_AnimatedTransformsCurrentFrameOnly
category: ~DATASMITH_CATEGORY~
internalCategory: "Datasmith"
tooltip: ~DATASMITH_EXPORTOPTIONS_ANIMATEDTRANSFORMSCURRENTFRAMEONLY_TOOLTIP~
autoUndoEnabled:false
(
	on execute do 
	(
		Datasmith_SetExportOption_AnimatedTransforms false
		updateToolbarButtons()
	)
)


macroScript Datasmith_AddModifier
category: ~DATASMITH_CATEGORY~
internalcategory: "Datasmith"
buttontext: ~DATASMITH_ADD_ATTRIBUTES_MODIFIER_TEXT~
tooltip: ~DATASMITH_ADD_ATTRIBUTES_MODIFIER_TOOLTIP~
(
	on execute do 
	(
	  modPanel.addModToSelection (UnrealStudio_Datasmith_Attributes ()) ui:on
	)
)

Macroscript Datasmith_ExportOptions_TextureResolution64
category: ~DATASMITH_CATEGORY~
internalCategory: "Datasmith"
autoUndoEnabled:false
(
	on execute do 
	(
		Datasmith_SetExportOption_TextureResolution 0
		updateToolbarButtons()
	)
)


Macroscript Datasmith_ExportOptions_TextureResolution128
category: ~DATASMITH_CATEGORY~
internalCategory: "Datasmith"
autoUndoEnabled:false
(
	on execute do 
	(
		Datasmith_SetExportOption_TextureResolution 1
		updateToolbarButtons()
	)
)


Macroscript Datasmith_ExportOptions_TextureResolution256
category: ~DATASMITH_CATEGORY~
internalCategory: "Datasmith"
autoUndoEnabled:false
(
	on execute do 
	(
		Datasmith_SetExportOption_TextureResolution 2
		updateToolbarButtons()
	)
)


Macroscript Datasmith_ExportOptions_TextureResolution512
category: ~DATASMITH_CATEGORY~
internalCategory: "Datasmith"
autoUndoEnabled:false
(
	on execute do 
	(
		Datasmith_SetExportOption_TextureResolution 3
		updateToolbarButtons()
	)
)


Macroscript Datasmith_ExportOptions_TextureResolution1024
category: ~DATASMITH_CATEGORY~
internalCategory: "Datasmith"
autoUndoEnabled:false
(
	on execute do 
	(
		Datasmith_SetExportOption_TextureResolution 4
		updateToolbarButtons()
	)
)


Macroscript Datasmith_ExportOptions_TextureResolution2048
category: ~DATASMITH_CATEGORY~
internalCategory: "Datasmith"
autoUndoEnabled:false
(
	on execute do 
	(
		Datasmith_SetExportOption_TextureResolution 5
		updateToolbarButtons()
	)
)


Macroscript Datasmith_ExportOptions_TextureResolution4096
category: ~DATASMITH_CATEGORY~
internalCategory: "Datasmith"
autoUndoEnabled:false
(
	on execute do 
	(
		Datasmith_SetExportOption_TextureResolution 6
		updateToolbarButtons()
	)
)


Macroscript Datasmith_ExportOptions_XRefScenes
category: ~DATASMITH_CATEGORY~
internalCategory: "Datasmith"
tooltip: ~DATASMITH_EXPORTOPTIONS_XREFSCENES_TOOLTIP~
autoUndoEnabled:false
(
	on isChecked do
	(
		Datasmith_GetExportOption_XRefScenes()
	)
	on execute do 
	(
		Datasmith_SetExportOption_XRefScenes(not Datasmith_GetExportOption_XRefScenes())
		updateToolbarButtons()
	)
)
