
fn Datasmith_GetUnrealEnginePathFromRegistry =
(
    local UnrealEngineRegPath = "Software\\Wow6432Node\\EpicGames\\Unreal Engine"
    local UnrealEngineRegKey = undefined
    local Result = registry.openKey HKEY_LOCAL_MACHINE UnrealEngineRegPath accessRights:#readOnly key:&UnrealEngineRegKey

    if not Result then
    (
        return undefined
    )

    local EnginePath = undefined
    local EnginePathType = undefined
    local Result = registry.queryValue UnrealEngineRegKey "3dsMaxEngineDir" type:&EnginePathType value:&EnginePath

    registry.closeKey UnrealEngineRegKey

    return EnginePath
)

fn Datasmith_GetUnrealEnginePath =
(
    local Path = undefined
    try (Path = Datasmith_GetUnrealEnginePathFromRegistry()) catch ()
    if (Path != undefined) do
    (
        return Path
    )

    return "C:/ProgramData/Epic/Exporter/3dsMaxEngine/"
)


global Datasmith_ExportDialog
fn Datasmith_ExportDialog selected:false =
(
	local path = getdir #export --get the export folder directory that 3ds Max uses in the context of this project
	local name = GetFilenameFile maxFileName
	local user_chosen_fpath = GetSaveFileName \
		caption:"Export Datasmith File" \
		filename: (path + "\\" + name+".udatasmith") \
		types:"Datasmith (*.udatasmith)|*.udatasmith|All Formats (*.*)|*.*" \ --allow for *.* as a conveninence for the user
		historyCategory:"ExportFile"
	if (user_chosen_fpath != undefined) then
	(
		local path = GetFilenamePath user_chosen_fpath
		local name = GetFilenameFile user_chosen_fpath
		Datasmith_Export name path selected:selected
	)
)

UnrealEnginePath = Datasmith_GetUnrealEnginePath()

UnrealEnginePathExists = doesFileExist UnrealEnginePath

if (not UnrealEnginePathExists) do
(
	format "Datasmith: Error. Engine folder '%' doesn't exist. Please, reinstall the plugin. \n" UnrealEnginePath
)

Datasmith_OnLoad true UnrealEnginePath

