(

p = getThisScriptFilename()
dir_path = getFilenamePath p

rcc_path = dir_path + "datasmith_icons.rcc"

v = maxVersion()
-- Register icons 
-- 3sx Max starting 2018(20000 is for 2018) uses Qt5(and PySide2)
-- 3sx Max starting 2025(26000 is for 2024) has "qtmax.LoadMaxMultiResIcon" function which simplifies code especially since PySide verfsion changes with 3ds Max versions
if v[1] < 20000 then 
(
	pyside = Python.Import "PySide"
	Python.Import "PySide.QtCore"
    pyside.QtCore.QResource.registerResource rcc_path
)
else if v[1] < 26000 then
(
	pyside = Python.Import "PySide2"
	Python.Import "PySide2.QtCore"
	pyside.QtCore.QResource.registerResource rcc_path
)
else
(
	qtmax = Python.Import "qtmax"
	qtmax.LoadMaxMultiResIcon rcc_path
)

)

Datasmith_SetupActions()

global Datasmith_ExportOptions_AnimatedTransformsIndex
fn Datasmith_ExportOptions_AnimatedTransformsIndex =
(
	if (Datasmith_GetExportOption_AnimatedTransforms()) then 2 else 1
)

global Datasmith_ExportOptions_TextureResolutionIndex
fn Datasmith_ExportOptions_TextureResolutionIndex =
(
	Datasmith_GetExportOption_TextureResolution() + 1
)

