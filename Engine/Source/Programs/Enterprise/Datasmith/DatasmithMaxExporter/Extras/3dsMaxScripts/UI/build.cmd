C:\Qt\5.15.1\msvc2019_64\bin\rcc.exe -binary datasmith_icons.qrc --format-version 1 -o datasmith_icons.rcc