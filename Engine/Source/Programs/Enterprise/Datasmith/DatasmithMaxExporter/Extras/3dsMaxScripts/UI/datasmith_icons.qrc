
<!DOCTYPE RCC><RCC version="1.0">
<qresource prefix="Datasmith/Icons">
    <file alias="DatasmithAutoSyncIconOff_24.png">images/DatasmithAutoSyncIconOff24.png</file>
    <file alias="DatasmithAutoSyncIconOff_30.png">images/DatasmithAutoSyncIconOff30.png</file>
    <file alias="DatasmithAutoSyncIconOff_36.png">images/DatasmithAutoSyncIconOff36.png</file>
    <file alias="DatasmithAutoSyncIconOff_48.png">images/DatasmithAutoSyncIconOff48.png</file>
    <file alias="DatasmithAutoSyncIconOn_24.png">images/DatasmithAutoSyncIconOn24.png</file>
    <file alias="DatasmithAutoSyncIconOn_30.png">images/DatasmithAutoSyncIconOn30.png</file>
    <file alias="DatasmithAutoSyncIconOn_36.png">images/DatasmithAutoSyncIconOn36.png</file>
    <file alias="DatasmithAutoSyncIconOn_48.png">images/DatasmithAutoSyncIconOn48.png</file>
    <file alias="DatasmithIcon_24.png">images/DatasmithIcon24.png</file>
    <file alias="DatasmithIcon_30.png">images/DatasmithIcon30.png</file>
    <file alias="DatasmithIcon_36.png">images/DatasmithIcon36.png</file>
    <file alias="DatasmithIcon_48.png">images/DatasmithIcon48.png</file>
    <file alias="DatasmithLogIcon_24.png">images/DatasmithLogIcon24.png</file>
    <file alias="DatasmithLogIcon_30.png">images/DatasmithLogIcon30.png</file>
    <file alias="DatasmithLogIcon_36.png">images/DatasmithLogIcon36.png</file>
    <file alias="DatasmithLogIcon_48.png">images/DatasmithLogIcon48.png</file>
    <file alias="DatasmithManageConnectionsIcon_24.png">images/DatasmithManageConnectionsIcon24.png</file>
    <file alias="DatasmithManageConnectionsIcon_30.png">images/DatasmithManageConnectionsIcon30.png</file>
    <file alias="DatasmithManageConnectionsIcon_36.png">images/DatasmithManageConnectionsIcon36.png</file>
    <file alias="DatasmithManageConnectionsIcon_48.png">images/DatasmithManageConnectionsIcon48.png</file>
    <file alias="DatasmithSettingsIcon_24.png">images/DatasmithSettingsIcon24.png</file>
    <file alias="DatasmithSettingsIcon_30.png">images/DatasmithSettingsIcon30.png</file>
    <file alias="DatasmithSettingsIcon_36.png">images/DatasmithSettingsIcon36.png</file>
    <file alias="DatasmithSettingsIcon_48.png">images/DatasmithSettingsIcon48.png</file>
    <file alias="DatasmithSyncIcon_24.png">images/DatasmithSyncIcon24.png</file>
    <file alias="DatasmithSyncIcon_30.png">images/DatasmithSyncIcon30.png</file>
    <file alias="DatasmithSyncIcon_36.png">images/DatasmithSyncIcon36.png</file>
    <file alias="DatasmithSyncIcon_48.png">images/DatasmithSyncIcon48.png</file>

</qresource>
</RCC>

