Synchronize="Synchronisieren"
SynchronizeTooltip="Mit Direct Link synchronisieren"
ToggleAutoSyncTooltip="Automatische Synchronisation von Direct Link umschalten"
ManageConnections="Verbindungen"
ManageConnectionsTooltip="Verbindungen verwalten"
Export="Exportieren"
ExportTooltip="In Datasmith-Datei exportieren"
Datasmith="Datasmith"
BakedSize2Desc="Gebakte prozedurale Texmaps dürfen 65.536 Pixel nicht überschreiten (z. B. 256x256)."
BakedSize5Desc="Gebakte prozedurale Texmaps dürfen 4 Megapixel nicht überschreiten (z. B. 2.048x2.048)."
BakedSize4Desc="Gebakte prozedurale Texmaps dürfen 1 Megapixel nicht überschreiten (z. B. 1.024x1.024)."
BakedSize3Desc="Gebakte prozedurale Texmaps dürfen 0,25 Megapixel nicht überschreiten (z. B. 512x512)."
ShowMessagesTooltip="Meldungen anzeigen"
CurrentFrameOnly="Nur aktueller Frame"
ActiveTimeSegment="Aktives Zeitsegment"
Messages="Meldungen"
ToggleAutoSync="Auto Sync &#xA;umschalten"
ExportXRefScenesTooltip="Aktiviert den Export/die Synchronisation von XRef-Szenen"
ExportSelected="Export ausgewählt"
BakedSize0Desc="Gebakte prozedurale Texmaps dürfen 4096 Pixel nicht überschreiten (z. B. 64x64)."
AddDatasmithAttributesModifier="Datasmith-Attributmodifikator hinzufügen"
LimitTextureResolution="Texturauflösung beschränken"
ExportXRefScenes="XRef-Szenen exportieren"
AnimatedTransforms="Animierte Transformationen"
FileExport="Dateiexport"
ExportSelectedTooltip="Ausgewählte Objekte in Datasmith-Datei exportieren"
Settings="Einstellungen"
BakedSize1Desc="Gebakte prozedurale Texmaps dürfen 16.384 Pixel nicht überschreiten (z. B. 128x128)."
Tools="Werkzeuge"
BakedSize6Desc="Gebakte prozedurale Texmaps dürfen 16 Megapixel nicht überschreiten (z. B. 4.096x4.096)."
