# These should be rather compact - used as subscripts for icons of the buttons (that's why ToggleAutoSync has line break character)
Synchronize="Synchronize"
ToggleAutoSync="Toggle &#xA;Auto Sync"
Export="Export"
ExportSelected="Export Selected"
Messages="Messages"
ManageConnections="Connections"

# Names of categories of buttons, also should not be very long(but can be longer than buttons subscripts)
FileExport="File Export"
Settings="Settings"
Tools="Tools"

# Tooltips - any length
SynchronizeTooltip="Synchronize with Direct Link"
ToggleAutoSyncTooltip="Toggle Direct Link Auto Sync"
ManageConnectionsTooltip="Manage Connections"
ExportTooltip="Export to Datasmith file"
ShowMessagesTooltip="Show Messages"
ExportSelectedTooltip="Export Selected objects to Datasmith file"
ExportXRefScenesTooltip="Enables export/sync of XRef Scenes"
BakedSize0Desc="Baked procedural texmaps would not exceed 4096 pixels (e.g. 64x64)"
BakedSize1Desc="Baked procedural texmaps would not exceed 16384 pixels (e.g. 128x128)"
BakedSize2Desc="Baked procedural texmaps would not exceed 65536 pixels (e.g. 256x256)"
BakedSize3Desc="Baked procedural texmaps would not exceed 0.25 Mega pixels (e.g. 512x512)"
BakedSize4Desc="Baked procedural texmaps would not exceed 1 Mega pixel (e.g. 1024x1024)"
BakedSize5Desc="Baked procedural texmaps would not exceed 4 Mega pixels (e.g. 2048x2048)"
BakedSize6Desc="Baked procedural texmaps would not exceed 16 Mega pixels (e.g. 4096x4096)"


# Any length supported, but displayed in the toolbar directly(not a tooltip) so shorter is better
AnimatedTransforms="Animated Transforms"
CurrentFrameOnly="Current Frame Only"
ActiveTimeSegment="Active Time Segment"
ExportXRefScenes="Export XRef Scenes"
AddDatasmithAttributesModifier="Add Datasmith Attributes Modifier"
LimitTextureResolution="Limit Texture Resolution"


Datasmith="Datasmith"

