<?xml version="1.0" encoding="utf-8"?>
<prese_wind:ResourceDictionary xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml" xmlns:context="clr-namespace:WPFCustomControls.ContextQueries;assembly=WPFCustomControls" xmlns:ads="clr-namespace:Autodesk.Windows;assembly=AdWindows" xmlns:wpfcust="clr-namespace:WPFCustomControls;assembly=WPFCustomControls" xmlns:prese_wind="clr-namespace:System.Windows;assembly=PresentationFramework">
  <wpfcust:MaxRibbonTab
    IsActive="true"
    Name="Datasmith"
    Title="Datasmith"
    UID="9d925117-9b31-4445-a189-ccca79934d90"
    x:Key="Datasmith">
    <wpfcust:MaxModelingRibbonPanel
      CollapsedPanelImage="Icons/Datasmith/DatasmithSyncIcon32.png"
      FloatingGroup="0"
      FloatingOrder="0"
      UID="0b8165dd-f894-42a6-9bfc-64e5b189fa9d">
      <wpfcust:MaxRibbonPanelSource
        Title="Direct Link"
        UID="5f41f38d-6458-4e7a-bb7b-1799477ffa1c">
        <ads:RibbonButton
          CommandHandler="{wpfcust:ActionItem ActionId=0, TableId=689133272}"
          Description="Synchroniser avec Direct Link"
          Image="Icons/Datasmith/DatasmithSyncIcon32.png"
          IsCheckable="true"
          IsToolTipEnabled="true"
          LargeImage="Icons/Datasmith/DatasmithSyncIcon32.png"
          MinWidth="0"
          Orientation="Vertical"
          ResizeStyle="HideText"
          ShowText="true"
          Size="Large"
          Text="Synchroniser"
          UID="e12deb32-dc84-4f23-874f-000722c61b11" />
        <wpfcust:MaxRibbonToggleButton
          AlternateImage="Icons/Datasmith/DatasmithAutoSyncIconOff32.png"
          CommandHandler="{wpfcust:ActionItem ActionId=1, TableId=689133272}"
          Description="Bascule de sync. auto. Direct Link"
          Id="1.689133272"
          IsCheckable="true"
          IsToolTipEnabled="true"
          LargeImage="Icons/Datasmith/DatasmithAutoSyncIconOn32.png"
          MinWidth="0"
          Orientation="Vertical"
          ResizeStyle="HideText"
          ShowText="true"
          Size="Large"
          Text="Basculer &#xA;Auto Sync"
          UID="1f535f66-ea8f-41c1-b835-8855215bbed2" />
        <ads:RibbonPanelBreak
          AllowInToolBar="false"
          IsToolTipEnabled="true"
          MinWidth="100"
          ShowImage="false"
          ShowText="false"
          UID="38c8f9a4-a524-4fa5-82d5-f60d80dcef06" />
        <ads:RibbonRowPanel
          AllowInToolBar="false"
          IsToolTipEnabled="true"
          MinWidth="100"
          ShowText="false"
          UID="533984eb-3982-460e-b97b-d71ec74800f8">
          <ads:RibbonButton
            CommandHandler="{wpfcust:ActionItem ActionId=2, TableId=689133272}"
            Description="Gérer les connexions"
            Image="Icons/Datasmith/DatasmithManageConnectionsIcon32.png"
            IsCheckable="true"
            IsToolTipEnabled="true"
            LargeImage="Icons/Datasmith/DatasmithManageConnectionsIcon32.png"
            MinWidth="0"
            ResizeStyle="HideText"
            ShowText="true"
            Size="Large"
            Text="Connexions"
            UID="e7499676-fcc7-4d7b-9b23-d71a9beea4e9" />
        </ads:RibbonRowPanel>
      </wpfcust:MaxRibbonPanelSource>
    </wpfcust:MaxModelingRibbonPanel>
    <wpfcust:MaxModelingRibbonPanel
      CollapsedPanelImage="Icons/Datasmith/DatasmithIcon32.png"
      FloatingGroup="0"
      FloatingOrder="0"
      UID="0113ae1a-ac1c-4b19-bd49-e69ea811778d">
      <wpfcust:MaxRibbonPanelSource
        Title="Exportation de fichier"
        UID="6bff3880-3957-435d-b72d-e49ca2555709">
        <ads:RibbonButton
          CommandHandler="{wpfcust:ActionItem ActionId=3, TableId=689133272}"
          Description="Exporter vers un fichier Datasmith"
          Image="Icons/Datasmith/DatasmithIcon32.png"
          IsCheckable="true"
          IsToolTipEnabled="false"
          LargeImage="Icons/Datasmith/DatasmithIcon32.png"
          MinWidth="0"
          Orientation="Vertical"
          ResizeStyle="HideText"
          ShowText="true"
          Size="Large"
          Text="Exporter"
          UID="a1de5acd-fef8-45be-8e01-ec194b9b5191" />
        <ads:RibbonButton
          CommandHandler="{wpfcust:ActionItem ActionId=5, TableId=689133272}"
          Description="Exporter les objets sélectionnés vers un fichier Datasmith"
          Image="Icons/Datasmith/DatasmithIcon32.png"
          IsCheckable="true"
          IsToolTipEnabled="false"
          LargeImage="Icons/Datasmith/DatasmithIcon32.png"
          MinWidth="0"
          Orientation="Vertical"
          ResizeStyle="HideText"
          ShowText="true"
          Size="Large"
          Text="Exporter la sélection"
          UID="405855b6-f3c3-4589-af11-7969a5badaa6" />
        <ads:RibbonPanelBreak
          AllowInToolBar="false"
          IsToolTipEnabled="true"
          MinWidth="100"
          ShowText="false"
          UID="d6de1715-c063-4e7a-b1ec-325f11ee079a" />
        <ads:RibbonRowPanel
          AllowInToolBar="false"
          Image="Icons/MissingImage_16.png"
          IsToolTipEnabled="true"
          LargeImage="Icons/MissingImage_32.png"
          MinWidth="100"
          ShowText="false"
          UID="4f43d4b3-f741-4262-8a64-ed43f3354248">
          <wpfcust:MaxRibbonLabel
            AllowInToolBar="false"
            IsToolTipEnabled="false"
            MinWidth="150"
            ResizeStyle="NoResize"
            ShowImage="false"
            ShowText="true"
            Text="Transformations animées:"
            UID="3907c965-cc4b-4257-8e99-a3c24dea6fa9" />
          <ads:RibbonSeparator
            AllowInToolBar="false"
            IsToolTipEnabled="true"
            MinWidth="0"
            ResizeStyle="NoResize"
            SeparatorStyle="Spacer"
            ShowText="false"
            UID="5308eeb0-047b-4bd0-bb89-56745a049b4a" />
          <wpfcust:MaxRibbonSplitButton
            Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
            IsCheckable="false"
            IsSplit="false"
            IsSynchronizedWithCurrentItem="true"
            IsToolTipEnabled="true"
            LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
            ListImageSize="Standard"
            MaxscriptIndexCallback="Datasmith_ExportOptions_AnimatedTransformsIndex()"
            MinWidth="0"
            ResizeStyle="HideText"
            ShowImage="false"
            ShowText="true"
            UID="59b29aac-ba4b-47bc-beec-31caf1bd6e2b">
            <ads:RibbonCommandItem
              AllowInToolBar="false"
              CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_ExportOptions_AnimatedTransformsCurrentFrameOnly, MacroCategory=Datasmith}"
              Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              IsCheckable="true"
              IsToolTipEnabled="true"
              LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              MinWidth="100"
              Name="Image actuelle uniquement"
              ShowImage="false"
              ShowText="false"
              Text="Image actuelle uniquement"
              UID="ef4472c0-fa88-4379-af73-842e4b57f752" />
            <ads:RibbonCommandItem
              AllowInToolBar="false"
              CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_ExportOptions_AnimatedTransformsActiveSegment, MacroCategory=Datasmith}"
              Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              IsCheckable="true"
              IsToolTipEnabled="true"
              LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              MinWidth="100"
              Name="Segment de temps actif"
              ShowImage="false"
              ShowText="false"
              Text="Segment de temps actif"
              UID="4097c080-33ee-430d-a47f-ab6e5c06c007" />
          </wpfcust:MaxRibbonSplitButton>
        </ads:RibbonRowPanel>
      </wpfcust:MaxRibbonPanelSource>
    </wpfcust:MaxModelingRibbonPanel>
    <wpfcust:MaxModelingRibbonPanel
      CollapsedPanelImage="Icons/Datasmith/DatasmithSettingsIcon32.png"
      FloatingGroup="0"
      FloatingOrder="0"
      UID="42ca9067-e3a4-4d3c-8d83-af9bb10b1124">
      <wpfcust:MaxRibbonPanelSource
        Title="Paramètres"
        UID="cc748308-a562-4516-8c94-4b2dae7a3a8b">
        <ads:RibbonRowPanel
          AllowInToolBar="false"
          Image="Icons/MissingImage_16.png"
          IsToolTipEnabled="true"
          LargeImage="Icons/MissingImage_32.png"
          MinWidth="100"
          ShowText="false"
          UID="939d8499-112d-4b8a-8b95-375ca7f8e029">
          <wpfcust:MaxRibbonLabel
            AllowInToolBar="false"
            IsToolTipEnabled="false"
            MinWidth="0"
            ResizeStyle="NoResize"
            ShowText="true"
            Text="Limiter la résolution des textures"
            UID="4519fb6c-d5d3-45ee-908a-6464f8639ba1" />
          <ads:RibbonSeparator
            AllowInToolBar="false"
            Image="Icons/MissingImage_16.png"
            IsToolTipEnabled="true"
            LargeImage="Icons/MissingImage_32.png"
            MinWidth="0"
            ResizeStyle="NoResize"
            ShowText="false"
            UID="1c3c4b93-a99e-4246-ac1f-7173c87aec67" />
          <wpfcust:MaxRibbonSplitButton
            Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
            IsCheckable="false"
            IsSplit="false"
            IsSynchronizedWithCurrentItem="true"
            IsToolTipEnabled="true"
            LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
            ListImageSize="Standard"
            MaxscriptIndexCallback="Datasmith_ExportOptions_TextureResolutionIndex()"
            MinWidth="0"
            ResizeStyle="HideText"
            ShowImage="false"
            ShowText="true"
            UID="a721d8e7-992f-464a-b2b4-c080a4f39fd9">
            <ads:RibbonCommandItem
              AllowInToolBar="false"
              CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_ExportOptions_TextureResolution64, MacroCategory=Datasmith}"
              Description="Les texmaps procéduraux précalculés n'excéderaient pas 4096 pixels (c.-à-d. 64x64)"
              Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              IsCheckable="true"
              IsToolTipEnabled="true"
              LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              MinWidth="100"
              Name="4K pixels"
              ShowImage="false"
              ShowText="false"
              Text="4K pixels"
              UID="28d398da-505c-4f91-9cf3-14316cd2893b" />
            <ads:RibbonCommandItem
              AllowInToolBar="false"
              CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_ExportOptions_TextureResolution128, MacroCategory=Datasmith}"
              Description="Les texmaps procéduraux précalculés n'excéderaient pas 16384 pixels (c.-à-d. 128x128)"
              Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              IsCheckable="true"
              IsToolTipEnabled="true"
              LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              MinWidth="100"
              Name="16K pixels"
              ShowImage="false"
              ShowText="false"
              Text="16K pixels"
              UID="89a922df-7286-40b9-833e-c51419fe3c1d" />
            <ads:RibbonCommandItem
              AllowInToolBar="false"
              CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_ExportOptions_TextureResolution256, MacroCategory=Datasmith}"
              Description="Les texmaps procéduraux précalculés n'excéderaient pas 65536 pixels (c.-à-d. 256x256)"
              Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              IsCheckable="true"
              IsToolTipEnabled="true"
              LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              MinWidth="100"
              Name="64K pixels"
              ShowImage="false"
              ShowText="false"
              Text="64K pixels"
              UID="177f6116-6ee0-4b6f-8d06-72d73bcc89e4" />
            <ads:RibbonCommandItem
              AllowInToolBar="false"
              CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_ExportOptions_TextureResolution512, MacroCategory=Datasmith}"
              Description="Les texmaps procéduraux précalculés n'excéderaient pas 0,25 mégapixel (c.-à-d. 512x512)"
              Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              IsCheckable="true"
              IsToolTipEnabled="true"
              LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              MinWidth="100"
              Name="256K pixels"
              ShowImage="false"
              ShowText="false"
              Text="256K pixels"
              UID="a5dd7f25-278a-4f8e-a9c2-7d73164df5f0" />
            <ads:RibbonCommandItem
              AllowInToolBar="false"
              CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_ExportOptions_TextureResolution1024, MacroCategory=Datasmith}"
              Description="Les texmaps procéduraux précalculés n'excéderaient pas 1 mégapixel (c.-à-d. 1024x1024)"
              Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              IsCheckable="true"
              IsToolTipEnabled="true"
              LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              MinWidth="100"
              Name="1M pixels"
              ShowImage="false"
              ShowText="false"
              Text="1M pixels"
              UID="09fec31c-f9e9-4e55-b33c-f4fb2218a4a0" />
            <ads:RibbonCommandItem
              AllowInToolBar="false"
              CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_ExportOptions_TextureResolution2048, MacroCategory=Datasmith}"
              Description="Les texmaps procéduraux précalculés n'excéderaient pas 4 mégapixels (c.-à-d. 2048x2048)"
              Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              IsCheckable="true"
              IsToolTipEnabled="true"
              LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              MinWidth="100"
              Name="4M pixels"
              ShowImage="false"
              ShowText="false"
              Text="4M pixels"
              UID="ef8585b8-1012-47a7-9af0-46371cf70df2" />
            <ads:RibbonCommandItem
              AllowInToolBar="false"
              CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_ExportOptions_TextureResolution4096, MacroCategory=Datasmith}"
              Description="Les texmaps procéduraux précalculés n'excéderaient pas 16 mégapixels (c.-à-d. 4096x4096)"
              Image="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              IsCheckable="true"
              IsToolTipEnabled="true"
              LargeImage="pack://application:,,,/AdWindows;component/Resources/Images/MissingImage.ico"
              MinWidth="100"
              Name="16M pixels"
              ShowImage="false"
              ShowText="false"
              Text="16M pixels"
              UID="c5c4c61f-7d64-4267-a387-cd6df2704161" />
          </wpfcust:MaxRibbonSplitButton>
          <ads:RibbonRowBreak
            AllowInToolBar="false"
            Image="Icons/MissingImage_16.png"
            IsToolTipEnabled="true"
            LargeImage="Icons/MissingImage_32.png"
            MinWidth="100"
            ShowText="false"
            UID="189ae82b-5e0f-4d3b-aa93-81e928234b27" />
          <wpfcust:MaxRibbonCheckBox
            CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_ExportOptions_XRefScenes, MacroCategory=Datasmith}"
            Description="Activer l'exportation/sync. de scènes XRef"
            IsCheckable="true"
            IsToolTipEnabled="true"
            MinWidth="0"
            Name="ExportXRefScenes"
            ResizeStyle="NoResize"
            ShowText="true"
            Text="Exporter les scènes XRef"
            UID="f8ed3b9b-6c4b-4b0c-92cb-618f4784ab94" />
        </ads:RibbonRowPanel>
      </wpfcust:MaxRibbonPanelSource>
    </wpfcust:MaxModelingRibbonPanel>
    <wpfcust:MaxModelingRibbonPanel
      CollapsedPanelImage="Icons/Datasmith/DatasmithToolsIcon32.png"
      FloatingGroup="0"
      FloatingOrder="0"
      ResizeStyle="NeverCollapsePanel"
      UID="a2baf0eb-f693-4079-b75d-e5b8c818aedc">
      <wpfcust:MaxRibbonPanelSource
        Title="Outils"
        UID="6159b143-ad36-4498-9b1c-916917018ef4">
        <ads:RibbonButton
          CommandHandler="{wpfcust:ActionItem ActionId=4, TableId=689133272}"
          Description="Afficher les messages"
          Image="Icons/Datasmith/DatasmithLogIcon32.png"
          IsCheckable="true"
          IsToolTipEnabled="false"
          LargeImage="Icons/Datasmith/DatasmithLogIcon32.png"
          MinWidth="0"
          Orientation="Vertical"
          ResizeStyle="HideText"
          ShowText="true"
          Size="Large"
          Text="Messages"
          UID="57863e62-4371-4d09-b12b-50e0a4c3afd8" />
        <ads:RibbonSeparator
          AllowInToolBar="false"
          IsToolTipEnabled="true"
          MinWidth="0"
          ResizeStyle="NoResize"
          ShowImage="false"
          ShowText="false"
          UID="c9e25ff1-0624-464c-bf24-4a033b8b0407" />
        <ads:RibbonRowPanel
          AllowInToolBar="false"
          IsToolTipEnabled="true"
          MinWidth="100"
          ShowImage="false"
          ShowText="false"
          Size="Large"
          UID="3a10e004-8897-4ca0-8824-0662af3c5097">
          <wpfcust:MaxRibbonToggleButton
            CommandHandler="{wpfcust:ActionItem MacroName=Datasmith_AddModifier, MacroCategory=Datasmith}"
            Id="Datasmith_AddModifier.Datasmith"
            IsCheckable="true"
            IsToolTipEnabled="true"
            LargeImage="Icons/MissingImage_32.png"
            MinWidth="0"
            OriginalImage="Icons/MissingImage_16.png"
            ResizeStyle="HideText"
            ShowImage="false"
            ShowText="true"
            Text="Ajouter le modificateur d'attributs Datasmith"
            UID="09bcd514-f707-4848-9bf4-32e73fe0d1c2" />
        </ads:RibbonRowPanel>
      </wpfcust:MaxRibbonPanelSource>
    </wpfcust:MaxModelingRibbonPanel>
  </wpfcust:MaxRibbonTab>
</prese_wind:ResourceDictionary>