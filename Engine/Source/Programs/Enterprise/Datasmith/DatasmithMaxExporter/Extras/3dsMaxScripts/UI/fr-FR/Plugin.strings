Synchronize="Synchroniser"
SynchronizeTooltip="Synchroniser avec Direct Link"
ToggleAutoSync="Basculer &#xA;Auto Sync"
ToggleAutoSyncTooltip="Bascule de sync. auto. Direct Link"
ManageConnections="Connexions"
ManageConnectionsTooltip="Gérer les connexions"
Export="Exporter"
ExportTooltip="Exporter vers un fichier Datasmith"
Datasmith="Datasmith"
# Messages="Messages"
ShowMessagesTooltip="Afficher les messages"
BakedSize2Desc="Les texmaps procéduraux précalculés n'excéderaient pas 65536 pixels (c.-à-d. 256x256)"
BakedSize5Desc="Les texmaps procéduraux précalculés n'excéderaient pas 4 mégapixels (c.-à-d. 2048x2048)"
BakedSize4Desc="Les texmaps procéduraux précalculés n'excéderaient pas 1 mégapixel (c.-à-d. 1024x1024)"
BakedSize3Desc="Les texmaps procéduraux précalculés n'excéderaient pas 0,25 mégapixel (c.-à-d. 512x512)"
CurrentFrameOnly="Image actuelle uniquement"
ActiveTimeSegment="Segment de temps actif"
Messages="Messages"
ExportXRefScenesTooltip="Activer l'exportation/sync. de scènes XRef"
ExportSelected="Exporter la sélection"
BakedSize0Desc="Les texmaps procéduraux précalculés n'excéderaient pas 4096 pixels (c.-à-d. 64x64)"
AddDatasmithAttributesModifier="Ajouter le modificateur d'attributs Datasmith"
LimitTextureResolution="Limiter la résolution des textures"
ExportXRefScenes="Exporter les scènes XRef"
AnimatedTransforms="Transformations animées"
FileExport="Exportation de fichier"
ExportSelectedTooltip="Exporter les objets sélectionnés vers un fichier Datasmith"
Settings="Paramètres"
BakedSize1Desc="Les texmaps procéduraux précalculés n'excéderaient pas 16384 pixels (c.-à-d. 128x128)"
Tools="Outils"
BakedSize6Desc="Les texmaps procéduraux précalculés n'excéderaient pas 16 mégapixels (c.-à-d. 4096x4096)"
