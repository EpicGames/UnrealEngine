Synchronize="同期"
SynchronizeTooltip="Direct Linkと同期する"
ToggleAutoSyncTooltip="Direct Link Auto Syncを切り替える"
ManageConnections="接続"
ManageConnectionsTooltip="接続を管理する"
Export="エクスポート"
ExportTooltip="Datasmithファイルエクスポート"
Datasmith="Datasmith"
ShowMessagesTooltip="メッセージを表示する"
BakedSize2Desc="ベイク済みプロシージャル テクスチャマップは 65536 ピクセル (例:256x256) を超えません"
BakedSize5Desc="ベイク済みプロシージャル テクスチャマップは 4 メガピクセル (例:2048x2048) を超えません"
BakedSize4Desc="ベイク済みプロシージャル テクスチャマップは 1 メガピクセル (例:1024x1024) を超えません"
BakedSize3Desc="ベイク済みプロシージャル テクスチャマップは 0.25 メガピクセル (例:512x512) を超えません"
CurrentFrameOnly="現在のフレームのみ"
ActiveTimeSegment="アクティブなタイムセグメント"
Messages="メッセージ"
ToggleAutoSync="自動同期&#xA;を切り替える"
ExportXRefScenesTooltip="XRef シーンのエクスポート/同期を有効にします"
ExportSelected="選択されたものをエクスポート"
BakedSize0Desc="ベイク済みプロシージャル テクスチャマップは 4096 ピクセル (例:64x64) を超えません"
AddDatasmithAttributesModifier="Datasmith属性モディファイアを追加"
LimitTextureResolution="テクスチャ解像度を制限"
ExportXRefScenes="XRefシーンをエクスポート"
AnimatedTransforms="アニメートされたトランスフォーム"
FileExport="ファイルエクスポート"
ExportSelectedTooltip="選択されたオブジェクトをDatasmithファイルにエクスポートする"
Settings="設定"
BakedSize1Desc="ベイク済みプロシージャル テクスチャマップは 16384 ピクセル (例:128x128) を超えません"
Tools="ツール"
BakedSize6Desc="ベイク済みプロシージャル テクスチャマップは 16 メガピクセル (例:4096x4096) を超えません"
