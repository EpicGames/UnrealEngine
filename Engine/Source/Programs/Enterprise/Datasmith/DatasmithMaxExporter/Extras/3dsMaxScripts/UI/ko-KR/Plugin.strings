Synchronize="동기화"
SynchronizeTooltip="다이렉트 링크와 동기화"
ToggleAutoSync="토글 &#xA;자동 동기화"
ToggleAutoSyncTooltip="다이렉트 링크 자동 동기화 토글"
ManageConnections="연결"
ManageConnectionsTooltip="연결 관리"
Export="익스포트"
ExportTooltip="데이터스미스 파일로 익스포트"
Datasmith="데이터스미스"
ShowMessagesTooltip="메시지 표시"
BakedSize2Desc="베이크된 프로시저럴 텍스처맵은 65536픽셀을 초과하지 않습니다(예: 256x256)"
BakedSize5Desc="베이크된 프로시저럴 텍스처맵은 4 메가픽셀을 초과하지 않습니다(예: 2048x2048"
BakedSize4Desc="베이크된 프로시저럴 텍스처맵은 1 메가픽셀을 초과하지 않습니다(예: 1024x1024"
BakedSize3Desc="베이크된 프로시저럴 텍스처맵은 0.25 메가픽셀을 초과하지 않습니다(예: 512x512)"
CurrentFrameOnly="현재 프레임만"
ActiveTimeSegment="활성 시간 세그먼트"
Messages="메시지"
ExportXRefScenesTooltip="XRef 씬 내보내기/동기화 활성화"
ExportSelected="선택한 항목 익스포트"
BakedSize0Desc="베이크된 프로시저럴 텍스처맵은 4096픽셀을 초과하지 않습니다(예: 64x64)"
AddDatasmithAttributesModifier="데이터스미스 어트리뷰트 모디파이어 추가"
LimitTextureResolution="텍스처 해상도 제한"
ExportXRefScenes="XRef 씬 익스포트"
AnimatedTransforms="애니메이션 트랜스폼"
FileExport="파일 익스포트"
ExportSelectedTooltip="선택한 오브젝트를 데이터스미스 파일로 익스포트"
Settings="세팅"
BakedSize1Desc="베이크된 프로시저럴 텍스처맵은 16384픽셀을 초과하지 않습니다(예: 128x128)"
Tools="툴"
BakedSize6Desc="베이크된 프로시저럴 텍스처맵은 16 메가픽셀을 초과하지 않습니다(예: 4096x4096"
