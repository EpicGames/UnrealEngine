p = getThisScriptFilename()
dir_path = getFilenamePath p

rcc_path = dir_path + "datasmith_icons.rcc"

v = maxVersion()
if v[1] < 20000 then -- 3sx Max starting 2018 uses Qt5(and PySide2)
(
	pyside = Python.Import "PySide"
	Python.Import "PySide.QtCore"
    pyside.QtCore.QResource.registerResource rcc_path
)
else
(
	pyside = Python.Import "PySide2"
	Python.Import "PySide2.QtCore"
	pyside.QtCore.QResource.registerResource rcc_path
)

Datasmith_SetupActions()

global Datasmith_ExportOptions_IncludeIndex
fn Datasmith_ExportOptions_IncludeIndex =
(
	if (Datasmith_GetExportOption_SelectedOnly()) then 2 else 1
)

global Datasmith_ExportOptions_AnimatedTransformsIndex
fn Datasmith_ExportOptions_AnimatedTransformsIndex =
(
	if (Datasmith_GetExportOption_AnimatedTransforms()) then 2 else 1
)

