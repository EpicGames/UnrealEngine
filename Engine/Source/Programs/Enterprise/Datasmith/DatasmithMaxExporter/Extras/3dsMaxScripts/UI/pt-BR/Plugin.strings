Synchronize="Sincronizar"
SynchronizeTooltip="Sincronizar com Direct Link"
ToggleAutoSyncTooltip="Alternar Sincronização Automática com Direct Link"
ManageConnections="Conexões"
ManageConnectionsTooltip="Gerenciar Conexões"
Export="Exportar"
ExportTooltip="Exportar para arquivo do Datasmith"
Datasmith="Datasmith"
ShowMessagesTooltip="Mostrar Mensagens"
BakedSize2Desc="Mapas de texto processuais incorporados não excederiam 65536 pixels (por exemplo: 256x256)"
BakedSize5Desc="Mapas de texto processuais incorporados não excederiam 4 Mega pixels (por exemplo: 2048x2048)"
BakedSize4Desc="Mapas de texto processuais incorporados não excederiam 1 Mega pixel (por exemplo: 1024x1024)"
BakedSize3Desc="Mapas de texto processuais incorporados não excederiam 0,25 Mega pixel (por exemplo: 512x512)"
CurrentFrameOnly="Somente Quadro Atual"
ActiveTimeSegment="Segmento de Tempo Ativo"
Messages="Mensagens"
ToggleAutoSync="Alternar &#xA;Sincronização Automática"
ExportXRefScenesTooltip="Permite a exportação/sincronização de cenas de RefX"
ExportSelected="Exportar Selecionados"
BakedSize0Desc="Mapas de texto processuais incorporados não excederiam 4096 pixels (por exemplo: 64x64)"
AddDatasmithAttributesModifier="Adicionar Modificador de Atributos do Datasmith"
LimitTextureResolution="Limitar Resolução de Textura"
ExportXRefScenes="Exportar Cenas de RefX"
AnimatedTransforms="Transformações Animadas"
FileExport="Exportar Arquivo"
ExportSelectedTooltip="Exportar objetos selecionados para arquivo do Datasmith"
Settings="Configurações"
BakedSize1Desc="Mapas de texto processuais incorporados não excederiam 16384 pixels (por exemplo: 128x128)"
Tools="Ferramentas"
BakedSize6Desc="Mapas de texto processuais incorporados não excederiam 16 Mega pixels (por exemplo: 4096x4096)"
