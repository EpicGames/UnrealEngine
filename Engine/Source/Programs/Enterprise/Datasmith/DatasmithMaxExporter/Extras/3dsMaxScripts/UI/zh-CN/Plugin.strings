Synchronize="同步"
SynchronizeTooltip="与Direct Link同步"
ToggleAutoSync="开启/关闭 &#xA;自动同步"
ToggleAutoSyncTooltip="开启/关闭Direct Link自动同步"
ManageConnections="连接"
ManageConnectionsTooltip="管理连接"
Export="导出"
ExportTooltip="导出到Datasmith文件"
Datasmith="Datasmith"
BakedSize2Desc="烘焙的程序化纹理贴图不会超过65536像素（如256x256）"
BakedSize5Desc="烘焙的程序化纹理贴图不会超过400万像素（如2048x2048）"
BakedSize4Desc="烘焙的程序化纹理贴图不会超过100万像素（如1024x1024）"
BakedSize3Desc="烘焙的程序化纹理贴图不会超过25万像素（如512x512）"
ShowMessagesTooltip="显示消息"
CurrentFrameOnly="仅当前帧"
ActiveTimeSegment="激活时间段"
Messages="消息"
ExportXRefScenesTooltip="启用XRef Scenes导出/同步"
ExportSelected="导出选中项"
BakedSize0Desc="烘焙的程序化纹理贴图不会超过4096像素（如64x64）"
AddDatasmithAttributesModifier="添加Datasmith属性修改器"
LimitTextureResolution="限制纹理分辨率"
ExportXRefScenes="导出XRef Scenes"
AnimatedTransforms="动画变形"
FileExport="文件导出"
ExportSelectedTooltip="将选中对象导出到Datasmith文件"
Settings="设置"
BakedSize1Desc="烘焙的程序化纹理贴图不会超过16384像素（如128x128）"
Tools="工具"
BakedSize6Desc="烘焙的程序化纹理贴图不会超过1600万像素（如4096x4096）"
