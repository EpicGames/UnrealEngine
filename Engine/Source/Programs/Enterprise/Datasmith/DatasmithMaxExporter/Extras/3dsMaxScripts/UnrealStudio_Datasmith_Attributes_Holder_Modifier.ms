/*
Copyright Epic Games, Inc. All Rights Reserved.

This script defines an attribute holder modieir that stores tags and properties
used in the 3ds Max to Unreal workflow with Unreal Studio and Datasmith
*/

	
plugin modifier UnrealStudio_Datasmith_Attributes
name:"Datasmith Attributes"
classID:#(0x60178e47, 0x2feafe95) --classid generated in the listener for this script
extends:EmptyModifier
replaceUI:1
version:1 --hook for update event handler in the future if updates to the script are necessary must be integer value

(
	----------------------------------------------------------------
	--LOCALS
	local intUIWidth = 145
	
	 --filter function for pickbutton, restricting to geometry objects only
	
		fn IsGeometricObject obj = 
		(
			local isValidObject = false
			--test for Geometric Objects that are not target objects
			if (superclassof obj == GeometryClass) and (classof obj != Targetobject) do isValidObject = true
			isValidObject
		)
	
	---------------------------------------------------------------
	-- PARAM BLOCK
	parameters main rollout:params
	(
	  
	  --------------------------------------------------------------------------
	  -- LIGHTMAP UVS
	  --parameters that should be picked up by the Datasmith exporter
	  param_int_LightmapChannel type:#integer animatable:false default:-1 --holds a reference to the custom mesh object node
	  
	  --parameters that are meant to connect the UI and plugin togheter

	  UI_ddlLightmapUVMode_Val type:#integer ui:ddlSetLightmapUVMode default:1 animatable:false
	  UI_spnLightmapUVChannel_Val type:#integer ui:spinSetLightMapUVChannel default:1 animatable:false
	  UI_spnLightmapUVChannel_Enable type:#boolean default:false animatable:false
	  
	--change handlers
	  on UI_ddlLightmapUVMode_Val set val do
	  (
		    case val of
			(
				----Automatic (Default) selected
				1: (
						UI_spnLightmapUVChannel_Enable = false
						param_int_LightmapChannel = -1
					)
					
				----Specify Map Channel selected
				2: (
						UI_spnLightmapUVChannel_Enable = true
						param_int_LightmapChannel = UI_spnLightmapUVChannel_Val
					)
				default:(
						UI_spnLightmapUVChannel_Enable = false
						param_int_LightmapChannel = -1
					)
			)--end case
		
	  )--end on UI_ddlLightmapUVMode_Val
	  
	  
	  on UI_spnLightmapUVChannel_Val set val do
	  (
		 param_int_LightmapChannel = val 
	  )

	--------------------------------------------------------------------------
	-- COLLISION MESH
	  
	  --parameters that should be picked up by the Datasmith exporter
	  param_node_CollisionMeshObject type:#node ui:pkbtnSetCollisionMeshObject -- holds a reference to the custom mesh object node
	  
	  --parameters that are meant to connect the UI and plugin togheter
	  UI_chkbxSetCollisionMeshObject_Val type:#boolean default:false animatable:false ui:chkbxSetCollisionMeshMode
	  UI_pkbtnSetCollisionMeshObject_Enable type:#boolean default:false animatable:false
	  
	  
	  --change handlers
	  on UI_chkbxSetCollisionMeshObject_Val set val do 
	  (
		  UI_pkbtnSetCollisionMeshObject_Enable = val
		  if val == false do param_node_CollisionMeshObject = undefined --clears the reference to the node
	  )		  
	
	--------------------------------------------------------------------------
	-- STATIC MESH EXPORT OPTION
	
	  --parameters that should be picked up by the Datasmith exporter
	  param_int_StaticMeshExportMode type:#integer animatable:false default:-1 --holds a reference to the custom mesh object node
	  
	  --parameters that should be picked up by the Datasmith exporter
	   UI_ddlSetStaticMeshExportMode_Val type:#integer ui:ddlSetStaticMeshExportMode default:1 animatable:false
	  
	  
	   --change handlers
	  on UI_ddlSetStaticMeshExportMode_Val set val do 
	  (
		  --- This indirection is used for a better support with the c++ exporter
		    case val of
			(
				----Automatic (Default (Static Mesh)) selected
				1: (
						param_int_StaticMeshExportMode = -1
					)
				----(Bounding Box) selected
				2: (
						param_int_StaticMeshExportMode = 1
					)
				---If the option isn't supported fail back to default
				default:(
					
						param_int_StaticMeshExportMode = -1
					)
			)--end case
	  )		  
  )--end paramblock
  
  
  
  ----------------------------------------------------------------
  -- UI ELEMENTS
  rollout params "UE Static Mesh Asset"
  (
	  --------------------------------------------------------------------------
	  -- LIGHTMAP UVS UI
	  
	
	  label lblLightmapUVs "Lightmap UVs" align:#left width:intUIWidth
	  
	  dropdownlist ddlSetLightmapUVMode items:#("Auto Unwrap UVs (Default)", "Specify Lightmap Channel") selection:1 \
									align:#left width:intUIWidth \
									tooltip:"Lightmap UVs are usually automatically generated. However, if you have manually unwrapped an object use this parameter to specify the desired Map Channel to be used for Lightmaps UVs in Unreal Engine."
	 
	  spinner spinSetLightMapUVChannel "Map Channel: " type:#integer range:[1,99,2] enabled:UI_spnLightmapUVChannel_Enable \
									align:#left width:intUIWidth \
									tooltip:"Specify which UV Map Channel to use as a source for the Lightmap UVs in Unreal." align:#right 
	  
	  
	  --ui handlers
	  on ddlSetLightmapUVMode selected index do spinSetLightMapUVChannel.enabled = UI_spnLightmapUVChannel_Enable --enables the spinner based on dropdown value
	 
	  
	   label lblPadding01 ""
	  
	  
	  --------------------------------------------------------------------------
	  -- COLLISION MESH
	  
	  label  lblCollisions "Collisions" align:#left width:intUIWidth
	  
	  checkbox chkbxSetCollisionMeshMode "Use Custom Collision Mesh" checked:false \
							tooltip:"Pick a single object to use as a custom collision mesh. Local transforms on the current object and picked object should match. Geometry must be convex." \
	                        align:#left width:intUIWidth
	 
	  pickbutton pkbtnSetCollisionMeshObject "Pick Geometric Object" enabled:UI_chkbxSetCollisionMeshObject_Val \
							filter:IsGeometricObject \
							autodisplay:true \
							tooltip:"Pick a single object to use as a custom collision mesh. Local transforms on the current object and picked object should match." \
							message:"Pick Geometric Object" \
							align:#left width:intUIWidth
	 
	 
	  --ui handlers
	  on chkbxSetCollisionMeshMode changed state do pkbtnSetCollisionMeshObject.enabled = UI_pkbtnSetCollisionMeshObject_Enable --enables the pick button based on checkbox
	  on pkbtnSetCollisionMeshObject picked obj do param_node_CollisionMeshObject = obj --sets the picked object as a parameter
	  
	  label lblPadding02 ""
	  
	  
	  --------------------------------------------------------------------------
	  -- STATIC MESH SUBSTITUTION
	  label lblExportGeometryAs "Export Geometry As" align:#left width:intUIWidth
	  
	  dropdownlist ddlSetStaticMeshExportMode items:#("Default (Static Mesh)","Bounding Box") selection:1 \
							tooltip:"Defines wheter or not Datasmith exports the object with a different representation." \
							align:#left width:intUIWidth
	  
  )--end rollout
  

  ----------------------------------------------------------------
  --PLUGIN HANDLERS
  
  --called when modifier applied on a object
  on attachedToNode obj do
  (
	 --look for all uvchannels on the node and automatically set the lightmapuv channel to the highest found. not bullit proof, but a small convenience
	  --will work only on main classes supporting mshop and polyop, otherwise won't do anything
	  --will not be reliable for cases where a modifier is added on several objects with different num of map channels at a time
	 --if (classof obj == Editable_Poly) do UI_spnLightmapUVChannel_Val = (polyop.getNumMaps obj)-1
	 --if (classof obj == Editable_Mesh) do UI_spnLightmapUVChannel_Val = (meshop.getNumMaps obj)-1
	  
	--initialize to auto unwrap by default
	param_int_LightmapChannel = -1
  )--end on attachedToNode
  
)--end plugin
