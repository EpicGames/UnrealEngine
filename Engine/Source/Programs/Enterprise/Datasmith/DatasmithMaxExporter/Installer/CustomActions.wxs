<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi">
	<Fragment>

		<!-- Abort install if 3ds Max is not installed -->
		<CustomAction Id="No3dsMaxInstaller" Error="!(loc.ErrorNo3dsMaxInstalled)" />

		<!-- Prevent downgrading -->
		<CustomAction Id="PreventDowngrading" Error="!(loc.ErrorOlderVersionInstall)" />

		<!-- Prevent install if the user already has the same version installed that was built at a different time.  This is a scenario only internal testers will encounter.
				End users will never see this as we bump the installer version number before release.  This will also not prevent install if the user has an older version installed.-->
		<CustomAction Id="PreventSameVersionInstall" Error="!(loc.ErrorSameVersionInstall)" />

		<!-- Declaration of custom actions to evaluate the properties MAX2023DIR, MAX2022DIR, MAX2021DIR, MAX2020DIR, MAX2019DIR, MAX2018DIR, MAX2017DIR, and MAX2016DIR -->
		<!-- See  'InstallExecuteSequence' section for their usage -->
		<CustomAction Id="SETMAX2023DIR" Property="MAX2023DIR" Value="" />
		<CustomAction Id="SETMAX2022DIR" Property="MAX2022DIR" Value="" />
		<CustomAction Id="SETMAX2021DIR" Property="MAX2021DIR" Value="" />
		<CustomAction Id="SETMAX2020DIR" Property="MAX2020DIR" Value="" />
		<CustomAction Id="SETMAX2019DIR" Property="MAX2019DIR" Value="" />
		<CustomAction Id="SETMAX2018DIR" Property="MAX2018DIR" Value="" />
		<CustomAction Id="SETMAX2017DIR" Property="MAX2017DIR" Value="" />
		<CustomAction Id="SETMAX2016DIR" Property="MAX2016DIR" Value="" />
		<!-- Equivalent custom actions to evaluate the 3ds max Design properties -->
		<CustomAction Id="SETMAXDESIGN2023DIR" Property="MAXDESIGN2023DIR" Value="" />
		<CustomAction Id="SETMAXDESIGN2022DIR" Property="MAXDESIGN2022DIR" Value="" />
		<CustomAction Id="SETMAXDESIGN2021DIR" Property="MAXDESIGN2021DIR" Value="" />
		<CustomAction Id="SETMAXDESIGN2020DIR" Property="MAXDESIGN2020DIR" Value="" />
		<CustomAction Id="SETMAXDESIGN2019DIR" Property="MAXDESIGN2019DIR" Value="" />
		<CustomAction Id="SETMAXDESIGN2018DIR" Property="MAXDESIGN2018DIR" Value="" />
		<CustomAction Id="SETMAXDESIGN2017DIR" Property="MAXDESIGN2017DIR" Value="" />
		<CustomAction Id="SETMAXDESIGN2016DIR" Property="MAXDESIGN2016DIR" Value="" />


		<!-- Adding DXDEBUG options -->
		<CustomActionRef Id="WixExitEarlyWithSuccess" />
		<CustomAction Id="InstallDirectX"	FileKey="DXSETUPEXE" ExeCommand="/silent" Execute="deferred" Impersonate="no" Return="check"/>

	</Fragment>
</Wix>