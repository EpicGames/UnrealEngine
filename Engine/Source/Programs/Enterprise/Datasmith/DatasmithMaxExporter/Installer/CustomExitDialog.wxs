<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi">
	<Fragment>
		<UI>
			<Dialog Id="CustomExitDialogCancel" Width="370" Height="270" Title="!(loc.ExitDialog_Title)" NoMinimize="yes">
				<Control Id="Finish" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Cancel="yes" Text="!(loc.WixUIFinish)"/>
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUICancel)"/>
				<Control Id="Bitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="234" TabSkip="no" Text="!(loc.ExitDialogBitmap)"/>
				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUIBack)"/>
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0"/>

				<Control Id="Description" Type="Text" X="135" Y="70" Width="220" Height="40" Transparent="yes" NoPrefix="yes" Text="!(loc.ExitDialogDescription)"/>

				<Control Id="DescriptionType" Type="Text" X="135" Y="110" Width="220" Height="40" Transparent="yes" NoPrefix="yes" Text="!(loc.ErrorNothingToInstall)"/>

				<Control Id="VersionNumber" Type="Text" X="10" Y="248" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />
			</Dialog>

			<!-- Mandatory dummy dialog to avoid failure at compilation time -->
			<Dialog Id="CustomExitDummyDialogError" Width="370" Height="270" Title="!(loc.ExitDialog_Title)" NoMinimize="yes">
				<Control Id="Finish" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Cancel="yes" Text="!(loc.WixUIFinish)"/>
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUICancel)"/>
				<Control Id="Bitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="234" TabSkip="no" Text="!(loc.ExitDialogBitmap)"/>
				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUIBack)"/>
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0"/>

				<Control Id="Description" Type="Text" X="135" Y="70" Width="220" Height="40" Transparent="yes" NoPrefix="yes" Text="!(loc.ExitDialogDescription)"/>

				<Control Id="DescriptionType" Type="Text" X="135" Y="110" Width="220" Height="40" Transparent="yes" NoPrefix="yes" Text="CUSTOMWIXUI_EXITDIALOGDESCTEXT"/>

				<Control Id="DescriptionHttp" Type="Text" X="135" Y="135" Width="220" Height="40" Transparent="yes" NoPrefix="yes" Text="CUSTOMWIXUI_EXITDIALOGDESCHYPERLINK"/>

				<Control Id="Title" Type="Text" X="135" Y="20" Width="220" Height="60" Transparent="yes" NoPrefix="yes" Text="[CUSTOMWIXUI_EXITDIALOGTITLETEXT]"/>
				<Control Id="VersionNumber" Type="Text" X="10" Y="248" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />
			</Dialog>

			<Dialog Id="CustomExitDialogError" Width="370" Height="270" Title="!(loc.ExitDialog_Title)" ErrorDialog="yes">

				<!-- DO NOT CHANGE THE ORDER OF DECLARATION OF UI ELEMENTS -->
				<!-- The ErrorText control must be the first -->
				<Control Id="ErrorText" Type="Text" X="125" Y="110" Width="230" Height="40" TabSkip="no" Transparent="yes" Text="Error text" />

				<Control Id="Y" Type="PushButton" X="236" Y="243" Width="56" Height="17" TabSkip="yes" Text="Yes">
					<Publish Event="EndDialog" Value="ErrorYes">1</Publish>
				</Control>
				<Control Id="A" Type="PushButton" X="236" Y="243" Width="56" Height="17" TabSkip="yes" Text="Abort">
					<Publish Event="EndDialog" Value="ErrorAbort">1</Publish>
				</Control>
				<Control Id="C" Type="PushButton" X="236" Y="243" Width="56" Height="17" TabSkip="yes" Text="Cancel">
					<Publish Event="EndDialog" Value="ErrorCancel">1</Publish>
				</Control>
				<Control Id="I" Type="PushButton" X="236" Y="243" Width="56" Height="17" TabSkip="yes" Text="Ignore">
					<Publish Event="EndDialog" Value="ErrorIgnore">1</Publish>
				</Control>
				<Control Id="N" Type="PushButton" X="236" Y="243" Width="56" Height="17" TabSkip="yes" Text="No">
					<Publish Event="EndDialog" Value="ErrorNo">1</Publish>
				</Control>
				<Control Id="O" Type="PushButton" X="236" Y="243" Width="56" Height="17" TabSkip="yes" Text="Close">
					<Publish Event="EndDialog" Value="ErrorOk">1</Publish>
				</Control>
				<Control Id="R" Type="PushButton" X="236" Y="243" Width="56" Height="17" TabSkip="yes" Text="Retry">
					<Publish Event="EndDialog" Value="ErrorRetry">1</Publish>
				</Control>

				<Control Id="ErrorIcon" Type="Icon" X="15" Y="15" Width="24" Height="24" FixedSize="yes" IconSize="32" Hidden="yes" Text="WixUI_Ico_Exclam" />

				<!-- PUT ADDITIONAL UI ELEMENTS BELOW THIS LINE -->

				<Control Id="Bitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="234" Text="!(loc.ExitDialogBitmap)"/>

				<Control Id="Missing3dsMax" Type="Text" X="125" Y="140" Width="230" Height="80" Transparent="yes" Hidden="yes" Text="!(loc.Error3dsMaxSupportedVersion)">
					<Condition Action="show">NOTFOUND3DSMAX</Condition>
				</Control>

				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0" />
				<Control Id="VersionNumber" Type="Text" X="10" Y="248" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />

			</Dialog>

			<InstallUISequence>
				<!-- 
				<Show Dialog="CustomExitDialogCancel" OnExit="success" Overridable="yes">NOT MAX2022CHECKED AND NOT MAX2021CHECKED AND NOT MAX2020CHECKED AND NOT MAX2019CHECKED AND NOT MAX2018CHECKED AND NOT MAX2017CHECKED AND NOT MAX2016CHECKED AND NOT MAXDESIGN2022CHECKED AND NOT MAXDESIGN2021CHECKED AND NOT MAXDESIGN2020CHECKED AND NOT MAXDESIGN2019CHECKED AND NOT MAXDESIGN2018CHECKED AND NOT MAXDESIGN2017CHECKED AND NOT MAXDESIGN2016CHECKED</Show>
				<Show Dialog="CustomExitDummyDialogError" OnExit="error" Overridable="yes">0</Show>
				-->
				
			</InstallUISequence>

			<AdminUISequence>
				<!-- 
				<Show Dialog="CustomExitDialogCancel" OnExit="success" Overridable="yes">NOT MAX2022CHECKED AND NOT MAX2021CHECKED AND NOT MAX2020CHECKED AND NOT MAX2019CHECKED AND NOT MAX2018CHECKED AND NOT MAX2017CHECKED AND NOT MAX2016CHECKED AND NOT MAXDESIGN2022CHECKED AND NOT MAXDESIGN2021CHECKED AND NOT MAXDESIGN2020CHECKED AND NOT MAXDESIGN2019CHECKED AND NOT MAXDESIGN2018CHECKED AND NOT MAXDESIGN2017CHECKED AND NOT MAXDESIGN2016CHECKED</Show>
				<Show Dialog="CustomExitDummyDialogError" OnExit="error" Overridable="yes">0</Show>
				-->
			</AdminUISequence>
		</UI>
	</Fragment>
</Wix>