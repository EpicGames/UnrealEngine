<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi">
	<Fragment>
		<UI>
			<Dialog Id="CustomInstallDirDlg" Width="800" Height="450" Title="!(loc.InstallDirDlg_Title)">

				<!-- Header -->
				<Control Id="BannerVoid" Type="Bitmap" X="0" Y="0" Width="800" Height="50" TabSkip="no" Text="BannerFill" />
				<Control Id="BannerLogo" Type="Bitmap" X="736" Y="5" Width="40" Height="42" TabSkip="no" Text="LogoEpicGames" />
				<Control Id="HeaderTitle" Type="Text" X="20" Y="15" Width="600" Height="35"  Transparent="yes" Text="{\TitleBanner}!(loc.GeneralTitleDialogWizard)" />
				<Control Id="Description" Type="Text" X="25" Y="23" Width="0" Height="0" Transparent="yes" NoPrefix="yes" Text="{\Strong}!(loc.InstallDirDlgDescription)" >
					<Condition Action="hide">Installed OR UPGRADEFOUND</Condition>
					<Condition Action="show">NOT Installed AND NOT UPGRADEFOUND</Condition>
				</Control>
				<!-- / Header -->

				<!-- Left Container -->
				<Control Id="Image1" Type="Bitmap" X="20" Y="75" Width="450" Height="253" TabSkip="no" Text="Background2" />
				<Control Id="Caption1" Type="Text" X="20" Y="330" Width="210" Height="75"   Transparent="yes"  Text="{\Para}!(loc.BackGroundCredits2)"/>
				<!-- /Left Container -->

				<!--Right Container -->
				<Control Id="Message" Type="Text" X="485" Y="75" Width="290" Height="30" NoPrefix="yes" Text="!(loc.InstallDirDlgMessage)">
					<Condition Action="hide">Installed OR UPGRADEFOUND</Condition>
					<Condition Action="show">NOT Installed AND NOT UPGRADEFOUND</Condition>
				</Control>

				<Control Id="CheckBox2023" Type="CheckBox" X="485" Y="110" Width="300" Height="20" Property="MAX2023CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2023" >
					<Condition Action="disable">NOT MAX2023VALID</Condition>
					<Condition Action="enable">MAX2023VALID</Condition>
				</Control>

				<Control Id="CheckBox2022" Type="CheckBox" X="485" Y="130" Width="300" Height="20" Property="MAX2022CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2022" >
					<Condition Action="disable">NOT MAX2022VALID</Condition>
					<Condition Action="enable">MAX2022VALID</Condition>
				</Control>

				<Control Id="CheckBox2021" Type="CheckBox" X="485" Y="150" Width="300" Height="20" Property="MAX2021CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2021" >
					<Condition Action="disable">NOT MAX2021VALID</Condition>
					<Condition Action="enable">MAX2021VALID</Condition>
				</Control>

				<Control Id="CheckBox2020" Type="CheckBox" X="485" Y="170" Width="300" Height="20" Property="MAX2020CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2020" >
					<Condition Action="disable">NOT MAX2020VALID</Condition>
					<Condition Action="enable">MAX2020VALID</Condition>
				</Control>
				
				<Control Id="CheckBox2019" Type="CheckBox" X="485" Y="190" Width="300" Height="20" Property="MAX2019CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2019" >
					<Condition Action="disable">NOT MAX2019VALID</Condition>
					<Condition Action="enable">MAX2019VALID</Condition>
				</Control>

				<Control Id="CheckBox2018" Type="CheckBox" X="485" Y="210" Width="300" Height="20" Property="MAX2018CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2018" >
					<Condition Action="disable">NOT MAX2018VALID</Condition>
					<Condition Action="enable">MAX2018VALID</Condition>
				</Control>

				<Control Id="CheckBox2017" Type="CheckBox" X="485" Y="230" Width="300" Height="20" Property="MAX2017CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2017" >
					<Condition Action="disable">NOT MAX2017VALID</Condition>
					<Condition Action="enable">MAX2017VALID</Condition>
				</Control>

				<Control Id="CheckBox2016" Type="CheckBox" X="485" Y="250" Width="300" Height="20" Property="MAX2016CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2016" >
					<Condition Action="disable">NOT MAX2016VALID</Condition>
					<Condition Action="enable">MAX2016VALID</Condition>
				</Control>
				<!-- / Right Container-->

				<!-- Bottom Container -->
				<!--Note, we borrow the loc info from the VerifyReady dialog which has an install button. We also only use the elevated version of the button normally reserved for ALLUSERS mode -->
				<Control Id="Next" Type="PushButton" ElevationShield="yes" X="635" Y="415" Width="80" Height="17" Default="yes" Hidden="yes" Disabled="yes" Text="!(loc.VerifyReadyDlgInstall)">
					<Condition Action="show">1</Condition>
					<Condition Action="enable">1</Condition>
					<Condition Action="default">1</Condition>
				</Control>

				<Control Id="Cancel" Type="PushButton" X="720" Y="415" Width="56" Height="17" Cancel="yes" Text="!(loc.WixUICancel)">
					<Publish Event="SpawnDialog" Value="CancelDlg">1</Publish>
				</Control>
				<Control Id="BottomLine" Type="Line" X="0" Y="400" Width="800" Height="0" />
				<Control Id="VersionNumber" Type="Text" X="20" Y="416" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />
				<!-- /Bottom Container -->

			</Dialog>
			<ProgressText Action="CAInstallDirectX">Installing DirectX</ProgressText>
		</UI>
	</Fragment>
</Wix>