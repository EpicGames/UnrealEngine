<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi">
	<Fragment>
		<UI>
			<Dialog Id="CustomWelcomeDialog" Width="800" Height="450" Title="!(loc.WelcomeDlg_Title)">
				<!-- Header -->
				<Control Id="BannerVoid" Type="Bitmap" X="0" Y="0" Width="800" Height="50" TabSkip="no" Text="BannerFill" />
				<Control Id="BannerLogo" Type="Bitmap" X="736" Y="5" Width="40" Height="42" TabSkip="no" Text="LogoEpicGames" />
				<Control Id="HeaderTitle" Type="Text" X="20" Y="15" Width="600" Height="35"  Transparent="yes" Text="{\TitleBanner}!(loc.GeneralTitleDialogWizard)"></Control>
				<!-- / Header -->

				<!-- Left Container -->
				<Control Id="Image1" Type="Bitmap" X="20" Y="75" Width="450" Height="253" TabSkip="no" Text="Background1" />
				<Control Id="Caption1" Type="Text" X="20" Y="330" Width="210" Height="75"  Transparent="yes"  Text="{\Para}Image courtesy of Michael Mong"/>
				<!-- /Left Container -->

				<!-- Right Container -->
				<Control Id="Title" Type="Text" X="490" Y="75" Width="270" Height="60" Transparent="yes" NoPrefix="yes" Text="{\Strong}!(loc.WelcomeDlgTitle)" />
				<Control Id="Description" Type="Text" X="490" Y="120" Width="270" Height="60" Transparent="yes" NoPrefix="yes" Text="{\Para}!(loc.WelcomeDlgDescription)" >
					<Condition Action="show">NOT Installed AND NOT UPGRADEFOUND</Condition>
					<Condition Action="hide">Installed OR UPGRADEFOUND</Condition>
				</Control>
				<!-- /Right Container -->

				<!--  Bottom Control Button -->
				<Control Id="BottomLine" Type="Line" X="0" Y="400" Width="800" Height="0" />
				<Control Id="VersionNumber" Type="Text" X="20" Y="416" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />

				<Control Id="Back" Type="PushButton" X="580" Y="416" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUIBack)" />
				<Control Id="Next" Type="PushButton" X="640" Y="416" Width="56" Height="17" Default="yes" Text="!(loc.WixUINext)" >
					<Publish Property="WixUI_InstallMode" Value="Update">Installed</Publish>
				</Control>
				<Control Id="Cancel" Type="PushButton" X="720" Y="415" Width="56" Height="17" Cancel="yes" Text="!(loc.WixUICancel)">
					<Publish Event="SpawnDialog" Value="CancelDlg">1</Publish>
				</Control>
				<!--  / Bottom Control Button -->
			</Dialog>
			<InstallUISequence>
				<Show Dialog="CustomWelcomeDialog" Before="ProgressDlg" Overridable="yes">NOT Installed</Show>
			</InstallUISequence>
		</UI>
	</Fragment>
</Wix>
