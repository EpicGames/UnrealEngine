<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="4.0" DefaultTargets="Build" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <PropertyGroup>
    <EngineSourcePath>..\..\..\..\..\</EngineSourcePath>
    <PrereqPath>$(EngineSourcePath)Programs\PrereqInstaller\</PrereqPath>
    <WixToolPath>$(EngineSourcePath)ThirdParty\WiX\3.8\</WixToolPath>
    <WixTargetsPath>$(WixToolPath)Wix.targets</WixTargetsPath>
    <WixTasksPath>$(WixToolPath)wixtasks.dll</WixTasksPath>
    <SuppressSpecificWarnings>
        1076;1077
    </SuppressSpecificWarnings>
  </PropertyGroup>
  <PropertyGroup>
    <Configuration Condition=" '$(Configuration)' == '' ">Debug</Configuration>
    <Platform Condition=" '$(Platform)' == '' ">x86</Platform>
    <ProductVersion>3.10</ProductVersion>
    <ProjectGuid>72766649-c13a-4889-82b6-ca7198101785</ProjectGuid>
    <SchemaVersion>2.0</SchemaVersion>
    <OutputName>UnrealDatasmithMaxExporter</OutputName>
    <OutputType>Package</OutputType>
    <WixTasksPath>wixtasks.dll</WixTasksPath>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|x86' ">
    <OutputPath>$(SolutionDir)\bin\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Configuration)\</IntermediateOutputPath>
    <DefineConstants>Debug</DefineConstants>
    <SuppressIces>ICE30;ICE82;ICE03</SuppressIces>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|x86' ">
    <OutputPath>$(SolutionDir)\bin\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Configuration)\</IntermediateOutputPath>
    <SuppressIces>ICE30;ICE82;ICE03;ICE60</SuppressIces>
    <LeaveTemporaryFiles>False</LeaveTemporaryFiles>
    <SuppressPdbOutput>True</SuppressPdbOutput>
    <VerboseOutput>True</VerboseOutput>
    <CompilerAdditionalOptions>-trace</CompilerAdditionalOptions>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|x64' ">
    <DefineConstants>Debug</DefineConstants>
    <OutputPath>$(SolutionDir)\bin\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Platform)\$(Configuration)\</IntermediateOutputPath>
    <SuppressIces>ICE30;ICE82;ICE03</SuppressIces>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|x64' ">
    <OutputPath>$(SolutionDir)\bin\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Platform)\$(Configuration)\</IntermediateOutputPath>
    <SuppressIces>ICE30;ICE82;ICE03</SuppressIces>
  </PropertyGroup>
  <ItemGroup>
    <Compile Include="CustomActions.wxs" />
    <Compile Include="CustomExitDialog.wxs" />
    <Compile Include="CustomInstallDirDlg.wxs" />
    <Compile Include="CustomLicenseAgreementDlg.wxs" />
    <Compile Include="CustomWelcomeDialog.wxs" />
    <Compile Include="CustomWixUI_InstallDir.wxs" />
    <Compile Include="GenericUiBrandDialog.wxs" />
    <Compile Include="MaxExporter.wxs" />
  </ItemGroup>
  <ItemGroup>
    <WixExtension Include="WixUtilExtension">
      <HintPath>$(WixToolPath)WixUtilExtension.dll</HintPath>
      <Name>WixUtilExtension</Name>
    </WixExtension>
    <WixExtension Include="WixUIExtension">
      <HintPath>$(WixToolPath)WixUIExtension.dll</HintPath>
      <Name>WixUIExtension</Name>
    </WixExtension>
    <WixExtension Include="WixNetFxExtension">
      <HintPath>$(WixToolPath)WixNetFxExtension.dll</HintPath>
      <Name>WixNetFxExtension</Name>
    </WixExtension>
  </ItemGroup>
  <ItemGroup>
    <Folder Include="Localization" />
    <Folder Include="Resources" />
    <Folder Include="Resources\Images" />
    <Folder Include="Resources\Images\Background" />
    <Folder Include="Resources\Images\General" />
  </ItemGroup>
  <ItemGroup>
    <Content Include="Resources\Banner.bmp" />
    <Content Include="Resources\Dialog.bmp" />
    <Content Include="Resources\Exporter Plugin End User License Agreement.rtf" />
    <Content Include="Resources\Images\Background\28190.bmp" />
    <Content Include="Resources\Images\Background\28402.bmp" />
    <Content Include="Resources\Images\Background\28674.bmp" />
    <Content Include="Resources\Images\Background\30668.bmp" />
    <Content Include="Resources\Images\Background\30725.bmp" />
    <Content Include="Resources\Images\Epic_Games_logo.bmp" />
    <Content Include="Resources\Images\General\banner.bmp" />
    <Content Include="Resources\Images\General\unreal-logo.jpg" />
    <Content Include="Resources\UnrealEngine.ico" />
  </ItemGroup>
  <ItemGroup>
    <EmbeddedResource Include="Localization\en-us.wxl" />
  </ItemGroup>
  <Import Project="$(WixTargetsPath)" Condition=" '$(WixTargetsPath)' != '' " />
  <!--
  <Import Project="$(MSBuildExtensionsPath32)\Microsoft\WiX\v3.x\Wix.targets" Condition=" '$(WixTargetsPath)' == '' AND Exists('$(MSBuildExtensionsPath32)\Microsoft\WiX\v3.x\Wix.targets') " />
  <Target Name="EnsureWixToolsetInstalled" Condition=" '$(WixTargetsImported)' != 'true' ">
    <Error Text="The WiX Toolset v3.11 (or newer) build tools must be installed to build this project. To download the WiX Toolset, see http://wixtoolset.org/releases/" />
  </Target>
  -->
  <!--
	To modify your build process, add your task inside one of the targets below and uncomment it.
	Other similar extension points exist, see Wix.targets.
	<Target Name="BeforeBuild">
	</Target>
	<Target Name="AfterBuild">
	</Target>
	-->
</Project>