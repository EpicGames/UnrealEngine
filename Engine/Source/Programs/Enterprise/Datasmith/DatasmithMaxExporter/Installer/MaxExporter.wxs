<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi" xmlns:util="http://schemas.microsoft.com/wix/UtilExtension">
	<!-- Include version -->
	<?include ..\..\DatasmithInstallerCommon\ExporterVersion.wxi?>

	<?define ProductVersion="$(var.MajorVersion).$(var.MinorVersion).$(var.PatchVersion).$(var.BuildVersion)"?>

	<!-- NOTE: Language codes can be found here(LCID Decimal column): http://www.science.co.il/Language/Locale-codes.asp -->
	<?define Languages = "1033"?>

	<!-- Identify the specific product release. Using an asterisk rather than a GUID recreates a new product code each time the installer is rebuilt. -->
	<?define ProductCode="*"?>

	<!-- The Upgrade Code. Never changes. This is the true "Unreal Engine Launcher" aplication ID, and keeping it the same allows us to recognise existing installations. -->
	<?define UpgradeCode="{AC65B10D-82C7-4492-8525-B0129219871F}"?>

	<!-- Defines a relative path to the main Engine directory -->
	<?define EngineDir="..\..\..\..\..\..\..\Engine"?>

	<!-- Defines a relative path to our resources directory -->
	<?define ResourcesDir="./Resources"?>

	<!-- Defines a relative path to Prereq installer's vcredist resources directory -->
	<?define VCRedistDir="$(var.EngineDir)\Source\Programs\PrereqInstaller\Resources\VCRedist"?>

	<!-- Defines a relative path to directory of DirectX redist resources -->
	<?define DirectXRedistDir="$(var.EngineDir)\Source\Programs\PrereqInstaller\CustomAction\Resources\DirectXRedist"?>

	<!-- Defines a relative path to directory where Max plugins are stored -->
	<?define BinaryDir="$(var.EngineDir)\Binaries\Win64\3DSMax"?>

	<!-- Defines a relative path to directory where Max scripts are stored -->
	<?define ScriptDir="..\Extras\3dsMaxScripts"?>

	<!-- Defines a relative path to the ThirdParty libraries -->
	<?define ThirdPartyDir="$(var.EngineDir)\Source\ThirdParty"?>

	<Product Id="$(var.ProductCode)" Name="Unreal Datasmith Exporter for 3ds Max" Language="1033" Version="$(var.ProductVersion)" Manufacturer="Epic Games, Inc." UpgradeCode="$(var.UpgradeCode)">

		<Package Platform="x64" InstallerVersion="405" Compressed="yes" InstallScope="perMachine" />

		<!-- Find other installed versions when the product (with this product code) is first installed, but not during maintenance mode or uninstallation. -->
		<Upgrade Id="$(var.UpgradeCode)">
			<!-- NEWPRODUCTFOUND will be set if there is a newer product already installed -->
			<UpgradeVersion
				Minimum="$(var.ProductVersion)"
				IncludeMinimum="no"
				OnlyDetect="yes"
				Language="$(var.Languages)"
				Property="NEWPRODUCTFOUND" />

			<!-- UPGRADEFOUND will be set if there is an older or same product already installed -->
			<UpgradeVersion
				Minimum="0.0.0"
				Maximum="$(var.ProductVersion)"
				IncludeMinimum="yes"
				IncludeMaximum="no"
				Language="$(var.Languages)"
				Property="UPGRADEFOUND" />

			<!-- ANOTHERBUILDINSTALLED will be set if the same product version is already installed -->
			<UpgradeVersion
				Minimum="$(var.ProductVersion)"
				Maximum="$(var.ProductVersion)"
				IncludeMinimum="yes"
				IncludeMaximum="yes"
				OnlyDetect="yes"
				Language="$(var.Languages)"
				Property="ANOTHERBUILDINSTALLED" />
		</Upgrade>

		<!-- Simple Dialog to inform user that installation has aborted because no supported version of 3ds Max has been found -->
		<UI>
			<UIRef Id="CustomWixUI_InstallDir"/>
			<UIRef Id="WixUI_ErrorProgressText"/>

			<!--These dialog references are required for util:CloseApplication to work properly -->
			<DialogRef Id="FilesInUse" />
			<DialogRef Id="MsiRMFilesInUse" />
		</UI>

		<!-- Create the icon for our shortcuts -->
		<!-- NOTE: ARPPRODUCTICON is a special property recognized by Windows Installer, which sets up this icon for us -->
		<Icon Id="UnrealEngine.ico" SourceFile="$(var.ResourcesDir)/UnrealEngine.ico" />
		<Property Id="ARPPRODUCTICON" Value="UnrealEngine.ico" />

		<!-- Customize artwork -->
		<WixVariable Id="WixUIBannerBmp" Value="$(var.ResourcesDir)/default_small_banner.bmp" />
		<WixVariable Id="WixUIDialogBmp" Value="$(var.ResourcesDir)/Dialog.bmp"/>

		<!-- Declaration of properties MAX20xxDIR, used to store installation directory for supported versions of 3ds Max -->
		<!-- @note: Windows Installer is setting those properties to C:\ if registry key is not found -->
		<Property Id="MAX2023DIR">
			<RegistrySearch Id="MAX2023Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\25.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2022DIR">
			<RegistrySearch Id="MAX2022Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\24.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2021DIR">
			<RegistrySearch Id="MAX2021Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\23.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2020DIR">
			<RegistrySearch Id="MAX2020Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\22.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2019DIR">
			<RegistrySearch Id="MAX2019Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\21.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2018DIR">
			<RegistrySearch Id="MAX2018Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\20.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2017DIR">
			<RegistrySearch Id="MAX2017Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\19.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2016DIR">
			<RegistrySearch Id="MAX2016Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\18.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>

		<!-- Declaration of properties MAX20xxVALID used to check if the corresponding install directory of 3ds Max is valid -->
		<Property Id="MAX2023VALID">
			<DirectorySearch Id="CheckMax2023Dir" Path="[MAX2023DIR]" Depth="0">
				<FileSearch Id="CheckMax2023File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2022VALID">
			<DirectorySearch Id="CheckMax2022Dir" Path="[MAX2022DIR]" Depth="0">
				<FileSearch Id="CheckMax2022File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2021VALID">
			<DirectorySearch Id="CheckMax2021Dir" Path="[MAX2021DIR]" Depth="0">
				<FileSearch Id="CheckMax2021File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2020VALID">
			<DirectorySearch Id="CheckMax2020Dir" Path="[MAX2020DIR]" Depth="0">
				<FileSearch Id="CheckMax2020File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
    <Property Id="MAX2019VALID">
			<DirectorySearch Id="CheckMax2019Dir" Path="[MAX2019DIR]" Depth="0">
				<FileSearch Id="CheckMax2019File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2018VALID">
			<DirectorySearch Id="CheckMax2018Dir" Path="[MAX2018DIR]" Depth="0">
				<FileSearch Id="CheckMax2018File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2017VALID">
			<DirectorySearch Id="CheckMax2017Dir" Path="[MAX2017DIR]" Depth="0">
				<FileSearch Id="CheckMax2017File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2016VALID">
			<DirectorySearch Id="CheckMax2016Dir" Path="[MAX2016DIR]" Depth="0">
				<FileSearch Id="CheckMax2016File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>

		<Property Id="WIXUI_INSTALLDIR" Value="INSTALLFOLDER2018" />
		<Property Id="WIXUI_FINALINSTALLDIR" Value="INSTALLFOLDER2018" />

		<!-- Common action to declare type of content in msi -->
		<Media Id="1" Cabinet="Datasmith3dsmaxExporter.cab" EmbedCab="yes" />

		<!-- Group of features to install. Each one if conditional to the existence of 3dsmax.exe in install folder -->
		<!-- @note: Each sub-feature is disabled ('Level = 0') at creation. The feature is enabled, if 3dsmax.exe has been found in the corresponding installation folder. -->
		<Feature Id="ProductFeature" Title="Installer" Level="1">
			<Feature Id="Feature2023" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2023" />
				<Condition Level="1"><![CDATA[MAX2023CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2022" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2022" />
				<Condition Level="1"><![CDATA[MAX2022CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2021" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2021" />
				<Condition Level="1"><![CDATA[MAX2021CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2020" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2020" />
				<Condition Level="1"><![CDATA[MAX2020CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2019" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2019" />
				<Condition Level="1"><![CDATA[MAX2019CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2018" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2018" />
				<Condition Level="1"><![CDATA[MAX2018CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2017" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2017" />
				<Condition Level="1"><![CDATA[MAX2017CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2016" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2016" />
				<Condition Level="1"><![CDATA[MAX2016CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureScript2023" Title="Installer" Level="0">
				<ComponentRef Id="SCRIPT2023" />
				<Condition Level="1"><![CDATA[MAX2023CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureScript2022" Title="Installer" Level="0">
				<ComponentRef Id="SCRIPT2022" />
				<Condition Level="1"><![CDATA[MAX2022CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureScript2021" Title="Installer" Level="0">
				<ComponentRef Id="SCRIPT2021" />
				<Condition Level="1"><![CDATA[MAX2021CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureScript2020" Title="Installer" Level="0">
				<ComponentRef Id="SCRIPT2020" />
				<Condition Level="1"><![CDATA[MAX2020CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureScript2019" Title="Installer" Level="0">
				<ComponentRef Id="SCRIPT2019" />
				<Condition Level="1"><![CDATA[MAX2019CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureScript2018" Title="Installer" Level="0">
				<ComponentRef Id="SCRIPT2018" />
				<Condition Level="1"><![CDATA[MAX2018CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureScript2017" Title="Installer" Level="0">
				<ComponentRef Id="SCRIPT2017" />
				<Condition Level="1"><![CDATA[MAX2017CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureScript2016" Title="Installer" Level="0">
				<ComponentRef Id="SCRIPT2016" />
				<Condition Level="1"><![CDATA[MAX2016CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesign2023" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNEXPORTER2023" />
				<Condition Level="1"><![CDATA[MAXDESIGN2023CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesign2022" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNEXPORTER2022" />
				<Condition Level="1"><![CDATA[MAXDESIGN2022CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesign2021" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNEXPORTER2021" />
				<Condition Level="1"><![CDATA[MAXDESIGN2021CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesign2020" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNEXPORTER2020" />
				<Condition Level="1"><![CDATA[MAXDESIGN2020CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesign2019" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNEXPORTER2019" />
				<Condition Level="1"><![CDATA[MAXDESIGN2019CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesign2018" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNEXPORTER2018" />
				<Condition Level="1"><![CDATA[MAXDESIGN2018CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesign2017" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNEXPORTER2017" />
				<Condition Level="1"><![CDATA[MAXDESIGN2017CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesign2016" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNEXPORTER2016" />
				<Condition Level="1"><![CDATA[MAXDESIGN2016CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesignScript2023" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNSCRIPT2023" />
				<Condition Level="1"><![CDATA[MAXDESIGN2023CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesignScript2022" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNSCRIPT2022" />
				<Condition Level="1"><![CDATA[MAXDESIGN2022CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesignScript2021" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNSCRIPT2021" />
				<Condition Level="1"><![CDATA[MAXDESIGN2021CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesignScript2020" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNSCRIPT2020" />
				<Condition Level="1"><![CDATA[MAXDESIGN2020CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesignScript2019" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNSCRIPT2019" />
				<Condition Level="1"><![CDATA[MAXDESIGN2019CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesignScript2018" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNSCRIPT2018" />
				<Condition Level="1"><![CDATA[MAXDESIGN2018CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesignScript2017" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNSCRIPT2017" />
				<Condition Level="1"><![CDATA[MAXDESIGN2017CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="FeatureDesignScript2016" Title="Installer" Level="0">
				<ComponentRef Id="DESIGNSCRIPT2016" />
				<Condition Level="1"><![CDATA[MAXDESIGN2016CHECKED = 1]]></Condition>
			</Feature>
		</Feature>

		<!-- Adding merge modules to install VC runtimes for VC10, VC11 VC12 and VC14 -->
		<DirectoryRef Id="TempFolder">
			<Merge Id="Microsoft_VC100_CRT_x64.msm" SourceFile="$(var.VCRedistDir)\Microsoft_VC100_CRT_x64.msm" DiskId="1" Language="0"/>
			<Merge Id="Microsoft_VC100_OpenMP_x64.msm" SourceFile="$(var.VCRedistDir)\Microsoft_VC100_OpenMP_x64.msm" DiskId="1" Language="0"/>
			<Merge Id="Microsoft_VC110_CRT_x64.msm" SourceFile="$(var.VCRedistDir)\Microsoft_VC110_CRT_x64.msm" DiskId="1" Language="0"/>
			<Merge Id="Microsoft_VC110_OpenMP_x64.msm" SourceFile="$(var.VCRedistDir)\Microsoft_VC110_OpenMP_x64.msm" DiskId="1" Language="0"/>
			<Merge Id="Microsoft_VC120_CRT_x64.msm" SourceFile="$(var.VCRedistDir)\Microsoft_VC120_CRT_x64.msm" DiskId="1" Language="0"/>
			<Merge Id="Microsoft_VC142_CRT_x64.msm" SourceFile="$(var.VCRedistDir)\Microsoft_VC142_CRT_x64.msm" DiskId="1" Language="0"/>
		</DirectoryRef>

		<!-- Adding installation of VC runtimes for VC10, VC11 VC12 and VC14 to the installation process -->
		<Feature Id="VCRedist" Title="Visual C++ Runtime" AllowAdvertise="yes" Display="expand" Level="1">
			<MergeRef Id="Microsoft_VC100_CRT_x64.msm" />
			<MergeRef Id="Microsoft_VC100_OpenMP_x64.msm" />
			<MergeRef Id="Microsoft_VC110_CRT_x64.msm" />
			<MergeRef Id="Microsoft_VC110_OpenMP_x64.msm" />
			<MergeRef Id="Microsoft_VC120_CRT_x64.msm" />
			<MergeRef Id="Microsoft_VC142_CRT_x64.msm" />
		</Feature>

		<!-- Adding installation modules for DirectX -->
		<DirectoryRef Id="TempFolder">
			<Directory Id="DirectXRedistDirectory" Name="DirectXPrereq">
				<Component Id="DirectXRedist" Guid="{D9FBAD53-79F0-40C8-A75F-980AD8A4997A}">
					<File Id="DXSETUPEXE" Source="$(var.DirectXRedistDir)\dxsetup.exe" KeyPath="yes" Checksum="yes">
						<util:PermissionEx User="Everyone" GenericAll="yes" ChangePermission="yes" />
					</File>
					<File Id="dxupdate.cab" Source="$(var.DirectXRedistDir)\dxupdate.cab"/>
					<File Id="dxdllreg_x86.cab" Source="$(var.DirectXRedistDir)\dxdllreg_x86.cab"/>
					<File Id="dsetup32.dll" Source="$(var.DirectXRedistDir)\dsetup32.dll"/>
					<File Id="dsetup.dll" Source="$(var.DirectXRedistDir)\dsetup.dll"/>
					<File Id="APR2007_xinput_x64.cab" Source="$(var.DirectXRedistDir)\APR2007_xinput_x64.cab"/>
					<File Id="Feb2010_X3DAudio_x64.cab" Source="$(var.DirectXRedistDir)\Feb2010_X3DAudio_x64.cab"/>
					<File Id="Jun2010_D3DCompiler_43_x64.cab" Source="$(var.DirectXRedistDir)\Jun2010_D3DCompiler_43_x64.cab"/>
					<File Id="Jun2010_d3dcsx_43_x64.cab" Source="$(var.DirectXRedistDir)\Jun2010_d3dcsx_43_x64.cab"/>
					<File Id="Jun2010_d3dx10_43_x64.cab" Source="$(var.DirectXRedistDir)\Jun2010_d3dx10_43_x64.cab"/>
					<File Id="Jun2010_d3dx11_43_x64.cab" Source="$(var.DirectXRedistDir)\Jun2010_d3dx11_43_x64.cab"/>
					<File Id="Jun2010_d3dx9_43_x64.cab" Source="$(var.DirectXRedistDir)\Jun2010_d3dx9_43_x64.cab"/>
					<File Id="Jun2010_XAudio_x64.cab" Source="$(var.DirectXRedistDir)\Jun2010_XAudio_x64.cab"/>
				</Component>
			</Directory>
		</DirectoryRef>

		<!-- Adding installation of DirectX to the installation process -->
		<Feature Id="DirectXRedist" Title="Installing DirectX" AllowAdvertise="yes" Display="expand" Level="1">
			<ComponentRef Id="DirectXRedist"/>
		</Feature>

		<SetProperty Id="MAX2023CHECKED" After="AppSearch" Value="1">MAX2023VALID</SetProperty>
		<SetProperty Id="MAX2022CHECKED" After="AppSearch" Value="1">MAX2022VALID</SetProperty>
		<SetProperty Id="MAX2021CHECKED" After="AppSearch" Value="1">MAX2021VALID</SetProperty>
		<SetProperty Id="MAX2020CHECKED" After="AppSearch" Value="1">MAX2020VALID</SetProperty>
		<SetProperty Id="MAX2019CHECKED" After="AppSearch" Value="1">MAX2019VALID</SetProperty>
		<SetProperty Id="MAX2018CHECKED" After="AppSearch" Value="1">MAX2018VALID</SetProperty>
		<SetProperty Id="MAX2017CHECKED" After="AppSearch" Value="1">MAX2017VALID</SetProperty>
		<SetProperty Id="MAX2016CHECKED" After="AppSearch" Value="1">MAX2016VALID</SetProperty>
		<SetProperty Id="MAXDESIGN2023CHECKED" After="AppSearch" Value="1">MAXDESIGN2023VALID</SetProperty>
		<SetProperty Id="MAXDESIGN2022CHECKED" After="AppSearch" Value="1">MAXDESIGN2022VALID</SetProperty>
		<SetProperty Id="MAXDESIGN2021CHECKED" After="AppSearch" Value="1">MAXDESIGN2021VALID</SetProperty>
		<SetProperty Id="MAXDESIGN2020CHECKED" After="AppSearch" Value="1">MAXDESIGN2020VALID</SetProperty>
		<SetProperty Id="MAXDESIGN2019CHECKED" After="AppSearch" Value="1">MAXDESIGN2019VALID</SetProperty>
		<SetProperty Id="MAXDESIGN2018CHECKED" After="AppSearch" Value="1">MAXDESIGN2018VALID</SetProperty>
		<SetProperty Id="MAXDESIGN2017CHECKED" After="AppSearch" Value="1">MAXDESIGN2017VALID</SetProperty>
		<SetProperty Id="MAXDESIGN2016CHECKED" After="AppSearch" Value="1">MAXDESIGN2016VALID</SetProperty>
		<SetProperty Id="NOTFOUND3DSMAX" After="AppSearch" Value="1">NOT MAX2023VALID AND NOT MAX2022VALID AND NOT MAX2021VALID AND NOT MAX2020VALID AND NOT MAX2019VALID AND NOT MAX2018VALID AND NOT MAX2017VALID AND NOT MAX2016VALID AND NOT MAXDESIGN2023VALID AND NOT MAXDESIGN2022VALID AND NOT MAXDESIGN2021VALID AND NOT MAXDESIGN2020VALID AND NOT MAXDESIGN2019VALID AND NOT MAXDESIGN2018VALID AND NOT MAXDESIGN2017VALID AND NOT MAXDESIGN2016VALID</SetProperty>

		<InstallExecuteSequence>

			<!-- Stop the user from installing an older product version -->
			<Custom Action="PreventDowngrading" After="FindRelatedProducts">NEWPRODUCTFOUND</Custom>

			<!-- Stop the user from installing the same product version -->
			<Custom Action="PreventSameVersionInstall" After="PreventDowngrading">ANOTHERBUILDINSTALLED</Custom>

			<!-- Force evaluation of properties MAX20xxDIR before anything else -->
			<!-- @todo: Check if this is still required. -->
			<Custom Action="SETMAX2023DIR" Before="AppSearch" />
			<Custom Action="SETMAX2022DIR" Before="AppSearch" />
			<Custom Action="SETMAX2021DIR" Before="AppSearch" />
			<Custom Action="SETMAX2020DIR" Before="AppSearch" />
			<Custom Action="SETMAX2019DIR" Before="AppSearch" />
			<Custom Action="SETMAX2018DIR" Before="AppSearch" />
			<Custom Action="SETMAX2017DIR" Before="AppSearch" />
			<Custom Action="SETMAX2016DIR" Before="AppSearch" />
			<Custom Action="SETMAXDESIGN2023DIR" Before="AppSearch" />
			<Custom Action="SETMAXDESIGN2022DIR" Before="AppSearch" />
			<Custom Action="SETMAXDESIGN2021DIR" Before="AppSearch" />
			<Custom Action="SETMAXDESIGN2020DIR" Before="AppSearch" />
			<Custom Action="SETMAXDESIGN2019DIR" Before="AppSearch" />
			<Custom Action="SETMAXDESIGN2018DIR" Before="AppSearch" />
			<Custom Action="SETMAXDESIGN2017DIR" Before="AppSearch" />
			<Custom Action="SETMAXDESIGN2016DIR" Before="AppSearch" />

			<!-- Remove the old product before installing in case this is the same version msi and we isntall to the same location -->
			<RemoveExistingProducts Before="InstallInitialize" />

			<Custom Action="InstallDirectX" Before="InstallFinalize">
				<![CDATA[NOT REMOVE]]>
			</Custom>

		</InstallExecuteSequence>

		<!-- Setup install UI sequence -->
		<InstallUISequence>

			<!-- Abort install if no version of the exporter is selected -->
			<Custom Action="WixExitEarlyWithSuccess" Before="ProgressDlg">NOT MAX2023CHECKED AND NOT MAX2022CHECKED AND NOT MAX2021CHECKED AND NOT MAX2020CHECKED AND NOT MAX2019CHECKED AND NOT MAX2018CHECKED AND NOT MAX2017CHECKED AND NOT MAX2016CHECKED AND NOT MAXDESIGN2023CHECKED AND NOT MAXDESIGN2022CHECKED AND NOT MAXDESIGN2021CHECKED AND NOT MAXDESIGN2020CHECKED AND NOT MAXDESIGN2019CHECKED AND NOT MAXDESIGN2018CHECKED AND NOT MAXDESIGN2017CHECKED AND NOT MAXDESIGN2016CHECKED</Custom>

			<!-- Abort install if no supported version of 3ds Max is installed -->
			<Custom Action="No3dsMaxInstaller" Before="ValidateProductID">NOTFOUND3DSMAX</Custom>

			<!-- After searching for previously installed products, prevent downgrading -->
			<Custom Action="PreventDowngrading" After="FindRelatedProducts">NEWPRODUCTFOUND</Custom>

			<!-- After searching for previously installed products, prevent overwriting -->
			<Custom Action="PreventSameVersionInstall" After="PreventDowngrading">ANOTHERBUILDINSTALLED</Custom>

		</InstallUISequence>
	</Product>

	<Fragment>
		<!-- Declaration of installation directory according to the version of 3ds Max -->
		<Directory Id="TARGETDIR" Name="SourceDir">
			<Directory Id="MAX2023DIR">
				<Directory Id="INSTALLFOLDER2023" Name="Plugins" />
				<Directory Id="SCRIPTFOLDER2023" Name="scripts">
					<Directory Id="STARTUPFOLDER2023" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAX2022DIR">
				<Directory Id="INSTALLFOLDER2022" Name="Plugins" />
				<Directory Id="SCRIPTFOLDER2022" Name="scripts">
					<Directory Id="STARTUPFOLDER2022" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAX2021DIR">
				<Directory Id="INSTALLFOLDER2021" Name="Plugins" />
				<Directory Id="SCRIPTFOLDER2021" Name="scripts">
					<Directory Id="STARTUPFOLDER2021" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAX2020DIR">
				<Directory Id="INSTALLFOLDER2020" Name="Plugins" />
				<Directory Id="SCRIPTFOLDER2020" Name="scripts">
					<Directory Id="STARTUPFOLDER2020" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAX2019DIR">
				<Directory Id="INSTALLFOLDER2019" Name="Plugins" />
				<Directory Id="SCRIPTFOLDER2019" Name="scripts">
					<Directory Id="STARTUPFOLDER2019" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAX2018DIR">
				<Directory Id="INSTALLFOLDER2018" Name="Plugins" />
				<Directory Id="SCRIPTFOLDER2018" Name="scripts">
					<Directory Id="STARTUPFOLDER2018" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAX2017DIR">
				<Directory Id="INSTALLFOLDER2017" Name="Plugins" />
				<Directory Id="SCRIPTFOLDER2017" Name="scripts">
					<Directory Id="STARTUPFOLDER2017" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAX2016DIR">
				<Directory Id="INSTALLFOLDER2016" Name="Plugins" />
				<Directory Id="SCRIPTFOLDER2016" Name="scripts">
					<Directory Id="STARTUPFOLDER2016" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAXDESIGN2023DIR">
				<Directory Id="INSTALLFOLDERDESIGN2023" Name="Plugins" />
				<Directory Id="SCRIPTFOLDERDESIGN2023" Name="scripts">
					<Directory Id="STARTUPFOLDERDESIGN2023" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAXDESIGN2022DIR">
				<Directory Id="INSTALLFOLDERDESIGN2022" Name="Plugins" />
				<Directory Id="SCRIPTFOLDERDESIGN2022" Name="scripts">
					<Directory Id="STARTUPFOLDERDESIGN2022" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAXDESIGN2021DIR">
				<Directory Id="INSTALLFOLDERDESIGN2021" Name="Plugins" />
				<Directory Id="SCRIPTFOLDERDESIGN2021" Name="scripts">
					<Directory Id="STARTUPFOLDERDESIGN2021" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAXDESIGN2020DIR">
				<Directory Id="INSTALLFOLDERDESIGN2020" Name="Plugins" />
				<Directory Id="SCRIPTFOLDERDESIGN2020" Name="scripts">
					<Directory Id="STARTUPFOLDERDESIGN2020" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAXDESIGN2019DIR">
				<Directory Id="INSTALLFOLDERDESIGN2019" Name="Plugins" />
				<Directory Id="SCRIPTFOLDERDESIGN2019" Name="scripts">
					<Directory Id="STARTUPFOLDERDESIGN2019" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAXDESIGN2018DIR">
				<Directory Id="INSTALLFOLDERDESIGN2018" Name="Plugins" />
				<Directory Id="SCRIPTFOLDERDESIGN2018" Name="scripts">
					<Directory Id="STARTUPFOLDERDESIGN2018" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAXDESIGN2017DIR">
				<Directory Id="INSTALLFOLDERDESIGN2017" Name="Plugins" />
				<Directory Id="SCRIPTFOLDERDESIGN2017" Name="scripts">
					<Directory Id="STARTUPFOLDERDESIGN2017" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="MAXDESIGN2016DIR">
				<Directory Id="INSTALLFOLDERDESIGN2016" Name="Plugins" />
				<Directory Id="SCRIPTFOLDERDESIGN2016" Name="scripts">
					<Directory Id="STARTUPFOLDERDESIGN2016" Name="Startup" />
				</Directory>
			</Directory>
			<Directory Id="TempFolder" />
		</Directory>
	</Fragment>

	<Fragment>
		<!-- Declaration of installation file for each supported version of 3ds Max -->
		<Component Id="EXPORTER2023" Directory="INSTALLFOLDER2023" Guid="{DEE3B1E9-7E13-47DE-96AA-BCAB543C7D2C}">
			<File Id="DatasmithMax2023.dle" Source="$(var.BinaryDir)\2023\DatasmithMax2023.dle" />
		</Component>
		<Component Id="EXPORTER2022" Directory="INSTALLFOLDER2022" Guid="{386501DB-9F60-4D81-8209-BE36AA0F15B9}">
			<File Id="DatasmithMax2022.dle" Source="$(var.BinaryDir)\2022\DatasmithMax2022.dle" />
			<File Id="EXPORTER2022_tbb.dll" Source="$(var.BinaryDir)\2022\tbb.dll"/>
		</Component>
		<Component Id="EXPORTER2021" Directory="INSTALLFOLDER2021" Guid="{FE8EF472-D4A4-4274-A3C9-F9EA6911E9A4}">
			<File Id="DatasmithMax2021.dle" Source="$(var.BinaryDir)\2021\DatasmithMax2021.dle" />
			<File Id="EXPORTER2021_tbb.dll" Source="$(var.BinaryDir)\2021\tbb.dll"/>
		</Component>
		<Component Id="EXPORTER2020" Directory="INSTALLFOLDER2020" Guid="{6FB2E004-9F9F-4F7C-9C69-E0CD824C0CD6}">
			<File Id="DatasmithMax2020.dle" Source="$(var.BinaryDir)\2020\DatasmithMax2020.dle" />
			<File Id="EXPORTER2020_tbb.dll" Source="$(var.BinaryDir)\2020\tbb.dll"/>
		</Component>
		<Component Id="EXPORTER2019" Directory="INSTALLFOLDER2019" Guid="{7A4EE7B4-C821-4BB0-8447-9D771FF3439E}">
			<File Id="DatasmithMax2019.dle" Source="$(var.BinaryDir)\2019\DatasmithMax2019.dle" />
			<File Id="EXPORTER2019_tbb.dll" Source="$(var.BinaryDir)\2019\tbb.dll"/>
		</Component>
		<Component Id="EXPORTER2018" Directory="INSTALLFOLDER2018" Guid="{8876116A-F406-4376-859F-92EC0D9CBB7D}">
			<File Id="DatasmithMax2018.dle" Source="$(var.BinaryDir)\2018\DatasmithMax2018.dle" />
			<File Id="EXPORTER2018_tbb.dll" Source="$(var.BinaryDir)\2018\tbb.dll"/>
		</Component>
		<Component Id="EXPORTER2017" Directory="INSTALLFOLDER2017" Guid="{A6FE7862-161E-4B3E-903A-34BE496D2D77}">
			<File Id="DatasmithMax2017.dle" Source="$(var.BinaryDir)\2017\DatasmithMax2017.dle" />
			<File Id="EXPORTER2017_tbb.dll" Source="$(var.BinaryDir)\2017\tbb.dll"/>
		</Component>
		<Component Id="EXPORTER2016" Directory="INSTALLFOLDER2016" Guid="{E6564FD7-1ECD-421A-A5BA-A7E3C3F8ED1B}">
			<File Id="DatasmithMax2016.dle" Source="$(var.BinaryDir)\2016\DatasmithMax2016.dle" />
			<File Id="EXPORTER2016_tbb.dll" Source="$(var.BinaryDir)\2016\tbb.dll"/>
		</Component>
		<Component Id="SCRIPT2023" Directory="STARTUPFOLDER2023" Guid="{3ED558E3-ADE6-4D00-96B8-73DF5738AE4D}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_Modifier2023.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="SCRIPT2022" Directory="STARTUPFOLDER2022" Guid="{E1EE0720-9F95-476E-8AFB-B33A211BC3FD}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_Modifier2022.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="SCRIPT2021" Directory="STARTUPFOLDER2021" Guid="{1CB93F82-0E7B-4A64-8093-05EF4883CFE2}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_Modifier2021.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="SCRIPT2020" Directory="STARTUPFOLDER2020" Guid="{41087F9C-C095-47C5-A858-18CCE9E20199}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_Modifier2020.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="SCRIPT2019" Directory="STARTUPFOLDER2019" Guid="{26EB60A4-D4CA-45F0-B621-DE1EF836CC41}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_Modifier2019.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="SCRIPT2018" Directory="STARTUPFOLDER2018" Guid="{9E6250F0-4A4F-4F45-88C3-52128F078E8F}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_Modifier2018.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="SCRIPT2017" Directory="STARTUPFOLDER2017" Guid="{E1C15635-8428-4157-B14D-1E6FF4DC392A}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_Modifier2017.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="SCRIPT2016" Directory="STARTUPFOLDER2016" Guid="{032E772E-15E5-4CA3-BECD-0C4784E3E381}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_Modifier2016.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<!-- Declaration of installation file for each supported version of 3ds Max Design -->
		<Component Id="DESIGNEXPORTER2023" Directory="INSTALLFOLDERDESIGN2023" Guid="{C7F385C3-E934-472B-A038-ED1CE6D383E7}">
			<File Id="DesignDatasmithMax2023.dle" Source="$(var.BinaryDir)\2023\DatasmithMax2023.dle" />
		</Component>
		<Component Id="DESIGNEXPORTER2022" Directory="INSTALLFOLDERDESIGN2022" Guid="{DC0AC876-29D4-4CE3-96D1-CDDF49B78D83}">
			<File Id="DesignDatasmithMax2022.dle" Source="$(var.BinaryDir)\2022\DatasmithMax2022.dle" />
		</Component>
		<Component Id="DESIGNEXPORTER2021" Directory="INSTALLFOLDERDESIGN2021" Guid="{4CDBD4F1-5F7D-47C8-876A-D2BC6E2529DB}">
			<File Id="DesignDatasmithMax2021.dle" Source="$(var.BinaryDir)\2021\DatasmithMax2021.dle" />
		</Component>
		<Component Id="DESIGNEXPORTER2020" Directory="INSTALLFOLDERDESIGN2020" Guid="{2A50A53C-29DB-469A-8667-7B034111235C}">
			<File Id="DesignDatasmithMax2020.dle" Source="$(var.BinaryDir)\2020\DatasmithMax2020.dle" />
		</Component>
		<Component Id="DESIGNEXPORTER2019" Directory="INSTALLFOLDERDESIGN2019" Guid="{E9CB3256-138F-4F00-A236-10153FCB2263}">
			<File Id="DesignDatasmithMax2019.dle" Source="$(var.BinaryDir)\2019\DatasmithMax2019.dle" />
		</Component>
		<Component Id="DESIGNEXPORTER2018" Directory="INSTALLFOLDERDESIGN2018" Guid="{9F786680-7FA4-46E0-A8C9-77A22C3BAFC5}">
			<File Id="DesignDatasmithMax2018.dle" Source="$(var.BinaryDir)\2018\DatasmithMax2018.dle" />
		</Component>
		<Component Id="DESIGNEXPORTER2017" Directory="INSTALLFOLDERDESIGN2017" Guid="{785CFBD3-5E2A-4F07-A1CB-5F9FD20DCB3D}">
			<File Id="DesignDatasmithMax2017.dle" Source="$(var.BinaryDir)\2017\DatasmithMax2017.dle" />
		</Component>
		<Component Id="DESIGNEXPORTER2016" Directory="INSTALLFOLDERDESIGN2016" Guid="{D95BDAA1-C98B-461E-8D08-5ACE1B3A6A13}">
			<File Id="DesignDatasmithMax2016.dle" Source="$(var.BinaryDir)\2016\DatasmithMax2016.dle" />
		</Component>
		<Component Id="DESIGNSCRIPT2023" Directory="STARTUPFOLDERDESIGN2023" Guid="{0BAEFF09-4BE9-4610-8859-F4AF4609C9C8}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_ModifierDesign2023.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="DESIGNSCRIPT2022" Directory="STARTUPFOLDERDESIGN2022" Guid="{A059E944-FF4A-490B-A320-26604C1CD269}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_ModifierDesign2022.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="DESIGNSCRIPT2021" Directory="STARTUPFOLDERDESIGN2021" Guid="{783FF0D6-2CA5-42AF-B153-1019DB844969}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_ModifierDesign2021.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="DESIGNSCRIPT2020" Directory="STARTUPFOLDERDESIGN2020" Guid="{90F5BA5F-921A-4742-8AA6-73B2FDA97D2B}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_ModifierDesign2020.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="DESIGNSCRIPT2019" Directory="STARTUPFOLDERDESIGN2019" Guid="{9655567D-B681-435D-AC57-23F2793530FE}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_ModifierDesign2019.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="DESIGNSCRIPT2018" Directory="STARTUPFOLDERDESIGN2018" Guid="{81586565-A8F0-4897-90EE-A551F9133438}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_ModifierDesign2018.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="DESIGNSCRIPT2017" Directory="STARTUPFOLDERDESIGN2017" Guid="{381D84D8-0487-4C00-9150-CA19752A29B3}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_ModifierDesign2017.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
		<Component Id="DESIGNSCRIPT2016" Directory="STARTUPFOLDERDESIGN2016" Guid="{505FA3B8-BF7D-4E6D-BD89-419C6498B0C1}">
			<File Id="UnrealStudio_Datasmith_Attributes_Holder_ModifierDesign2016.ms" Source="$(var.ScriptDir)\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
		</Component>
	</Fragment>
</Wix>
