The installer of the Datasmith 3ds Max exporter covers the following scenarios:
- No version of 3ds Max installed: A dialog informing the client of the situation will pop up and the installer will exit.
- No exporter installed yet: After a welcome dialog, the user is presented with a dialog stating what the installer is about to do for each version of 3ds Max installed on the computer. The user can deselect the version of 3ds Max for which they do not want the exporter installed. If no flavor of the exporter is installed, an exit dialog is presented and the install aborts. Otherwise, the selected flavors of the exporter are installed.
- Same version of the exporter already installed: A Windows error dialog pops up inviting the user to uninstall the existing version if they want to re-install the same version.
- Older version of the exporter installed, the workflow is similar to the one where no exporter is installed. The wording changed from install to update.

If you want to generate the installer, you must install the Wix Toolset Extension for Visual Studio 2015.
You can download it from https://marketplace.visualstudio.com/items?itemName=RobMensching.WixToolsetVisualStudio2015Extension.

