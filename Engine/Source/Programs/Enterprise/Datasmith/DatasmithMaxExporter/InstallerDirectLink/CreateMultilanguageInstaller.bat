@echo off

set engine_path=%~dp0..\..\..\..\..\..

"%engine_path%\Binaries\ThirdParty\Python3\Win64\python.exe" %~dp0CreateMultilanguageInstaller.py "%engine_path%" %1
