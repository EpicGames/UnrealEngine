import sys
import os
import shutil
import subprocess
from pathlib import Path


engine_path, build_dir = sys.argv[1:]
engine_path = Path(engine_path).resolve()
assert (engine_path/'Binaries').is_dir()
build_dir = Path(build_dir).resolve()
# scripts_dir = sys.argv[2]
scripts_dir = (engine_path/'Restricted/NotForLicensees/Programs/UnrealEngineInstaller/UnrealEngineInstaller/Localization/Scripts').resolve()

msi_name = 'UnrealDatasmithMaxExporter.msi'

en_msi_dir = build_dir/'en-us'
en_msi_path = en_msi_dir/msi_name

assert en_msi_path.is_file(), f"Expected to see 'en-us' msi in {en_msi_path}"

final_msi_dir = build_dir/'Final'
final_msi_path = final_msi_dir/msi_name

final_msi_dir.mkdir(parents=True, exist_ok=True)

shutil.copy(en_msi_dir/msi_name, final_msi_path)

assert final_msi_path.is_file(), f"Expected to see final msi copied from root culture msi '{final_msi_path}'"

def add_language(culture_name, culture_code):
    cmd = [
        str(scripts_dir/'CreateAndEmbedLangTransform.bat'),
        str(en_msi_dir), msi_name, 
        culture_name, str(culture_code),
        str(final_msi_dir)]
    subprocess.check_call(cmd)

for l in 'de-de:1031, ko-kr:1042, es-es:1034, fr-fr:1036, ja-jp:1041, pt-br:1046, zh-cn:2052'.split(', '):
	culture_name, culture_code = l.split(':')
	add_language(culture_name, culture_code)
