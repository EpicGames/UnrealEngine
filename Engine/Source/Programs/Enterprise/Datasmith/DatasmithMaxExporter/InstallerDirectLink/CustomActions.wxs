<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi">
  <Fragment>
    
    <!-- Abort install if 3ds Max is not installed -->
    <CustomAction Id="No3dsMaxInstaller" Error="!(loc.ErrorNo3dsMaxInstalled)" />

    <!-- Prevent downgrading -->
    <CustomAction Id="PreventDowngrading" Error="!(loc.ErrorOlderVersionInstall)" />

    <!-- Prevent install if the user already has the same version installed that was built at a different time.  This is a scenario only internal testers will encounter.
    		End users will never see this as we bump the installer version number before release.  This will also not prevent install if the user has an older version installed.-->
    <CustomAction Id="PreventSameVersionInstall" Error="!(loc.ErrorSameVersionInstall)" />

    <!-- Declaration of custom actions to evaluate the properties MAX2027DIR, MAX2026DIR, MAX2025DIR, MAX2024DIR, MAX2023DIR, MAX2022DIR -->
    <!-- See  'InstallExecuteSequence' section for their usage -->
    <CustomAction Id="SETMAX2027DIR" Property="MAX2027DIR" Value="" />
    <CustomAction Id="SETMAX2026DIR" Property="MAX2026DIR" Value="" />
    <CustomAction Id="SETMAX2025DIR" Property="MAX2025DIR" Value="" />
    <CustomAction Id="SETMAX2024DIR" Property="MAX2024DIR" Value="" />
    <CustomAction Id="SETMAX2023DIR" Property="MAX2023DIR" Value="" />
    <CustomAction Id="SETMAX2022DIR" Property="MAX2022DIR" Value="" />

    <CustomActionRef Id="WixExitEarlyWithSuccess" />

  </Fragment>
</Wix>
