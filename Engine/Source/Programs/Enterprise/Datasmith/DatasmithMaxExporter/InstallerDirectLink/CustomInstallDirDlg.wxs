<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi">
	<Fragment>
		<UI>
			<Dialog Id="CustomInstallDirDlg" Width="800" Height="450" Title="!(loc.InstallDirDlg_Title)">

				<!-- Header -->
				<Control Id="BannerVoid" Type="Bitmap" X="0" Y="0" Width="800" Height="50" TabSkip="no" Text="BannerFill" />
				<Control Id="BannerLogo" Type="Bitmap" X="736" Y="5" Width="40" Height="42" TabSkip="no" Text="LogoEpicGames" />
				<Control Id="HeaderTitle" Type="Text" X="20" Y="15" Width="600" Height="35"  Transparent="yes" Text="{\TitleBanner}!(loc.GeneralTitleDialogWizard)" />
				<Control Id="DescriptionInstall" Type="Text" X="25" Y="23" Width="0" Height="0" Transparent="yes" NoPrefix="yes" Text="{\Strong}!(loc.InstallDirDlgDescription)" >
					<Condition Action="hide">Installed</Condition>
					<Condition Action="show">NOT Installed AND NOT UPGRADEFOUND</Condition>
				</Control>
				<Control Id="DescriptionUpgrade" Type="Text" X="25" Y="23" Width="0" Height="0" Transparent="yes" NoPrefix="yes" Text="{\Strong}!(loc.InstallDirDlgDescriptionUpgrade)" >
					<Condition Action="hide">NOT Installed AND NOT UPGRADEFOUND</Condition>
					<Condition Action="show">Installed</Condition>
				</Control>
				<!-- / Header -->

				<!-- Left Container -->
				<Control Id="Image1" Type="Bitmap" X="20" Y="75" Width="450" Height="253" TabSkip="no" Text="Background2" />
				<Control Id="Caption1" Type="Text" X="20" Y="330" Width="210" Height="75"   Transparent="yes"  Text="{\Para}!(loc.BackGroundCredits2)"/>
				<!-- /Left Container -->

				<!--Right Container -->
				<Control Id="MessageInstall" Type="Text" X="485" Y="75" Width="290" Height="30" NoPrefix="yes" Text="!(loc.InstallDirDlgMessage)">
					<Condition Action="hide">Installed</Condition>
					<Condition Action="show">NOT Installed AND NOT UPGRADEFOUND</Condition>
				</Control>

				<Control Id="MessageUpgrade" Type="Text" X="485" Y="75" Width="290" Height="30" NoPrefix="yes" Text="!(loc.InstallDirDlgMessageUpgrade)">
					<Condition Action="hide">NOT Installed AND NOT UPGRADEFOUND</Condition>
					<Condition Action="show">Installed</Condition>
				</Control>

				<Control Id="CheckBox2027" Type="CheckBox" X="485" Y="110" Width="300" Height="20" Property="MAX2027CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2027" >
					<Condition Action="disable">NOT MAX2027VALID</Condition>
					<Condition Action="enable">MAX2027VALID</Condition>
				</Control>

				<Control Id="CheckBox2026" Type="CheckBox" X="485" Y="130" Width="300" Height="20" Property="MAX2026CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2026" >
					<Condition Action="disable">NOT MAX2026VALID</Condition>
					<Condition Action="enable">MAX2026VALID</Condition>
				</Control>

				<Control Id="CheckBox2025" Type="CheckBox" X="485" Y="150" Width="300" Height="20" Property="MAX2025CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2025" >
					<Condition Action="disable">NOT MAX2025VALID</Condition>
					<Condition Action="enable">MAX2025VALID</Condition>
				</Control>

				<Control Id="CheckBox2024" Type="CheckBox" X="485" Y="170" Width="300" Height="20" Property="MAX2024CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2024" >
					<Condition Action="disable">NOT MAX2024VALID</Condition>
					<Condition Action="enable">MAX2024VALID</Condition>
				</Control>

				<Control Id="CheckBox2023" Type="CheckBox" X="485" Y="190" Width="300" Height="20" Property="MAX2023CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2023" >
					<Condition Action="disable">NOT MAX2023VALID</Condition>
					<Condition Action="enable">MAX2023VALID</Condition>
				</Control>

				<Control Id="CheckBox2022" Type="CheckBox" X="485" Y="210" Width="300" Height="20" Property="MAX2022CHECKED" CheckBoxValue="1" Integer="yes" Text="Autodesk 3ds Max 2022" >
					<Condition Action="disable">NOT MAX2022VALID</Condition>
					<Condition Action="enable">MAX2022VALID</Condition>
				</Control>

				<!-- / Right Container-->

				<!-- Bottom Container -->
				<!--Note, we borrow the loc info from the VerifyReady dialog which has an install button. We also only use the elevated version of the button normally reserved for ALLUSERS mode -->
				<Control Id="Next" Type="PushButton" ElevationShield="yes" X="635" Y="415" Width="80" Height="17" Default="yes" Hidden="yes" Disabled="yes" Text="!(loc.VerifyReadyDlgInstall)">
					<Condition Action="show">1</Condition>
					<Condition Action="enable">1</Condition>
					<Condition Action="default">1</Condition>
				</Control>

				<Control Id="Cancel" Type="PushButton" X="720" Y="415" Width="56" Height="17" Cancel="yes" Text="!(loc.WixUICancel)">
					<Publish Event="SpawnDialog" Value="CancelDlg">1</Publish>
				</Control>
				<Control Id="BottomLine" Type="Line" X="0" Y="400" Width="800" Height="0" />
				<Control Id="VersionNumber" Type="Text" X="20" Y="416" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />
				<!-- /Bottom Container -->

			</Dialog>
			<ProgressText Action="CAInstallDirectX">Installing DirectX</ProgressText>
		</UI>
	</Fragment>
</Wix>