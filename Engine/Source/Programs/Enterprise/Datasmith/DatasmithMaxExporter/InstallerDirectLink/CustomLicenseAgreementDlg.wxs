<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi">
  <Fragment>
    <UI>
      <Dialog Id="CustomLicenseAgreementDlg" Width="800" Height="450" Title="!(loc.LicenseAgreementDlg_Title)">	  
        <!-- Header -->
        <Control Id="Description" Type="Text" X="20" Y="27" Width="280" Height="15" Transparent="yes" NoPrefix="yes" Text="{\WixUI_Font_Normal_White}!(loc.LicenseAgreementDlgDescription)" />
        <Control Id="Title" Type="Text" X="20" Y="6" Width="400" Height="20" Transparent="yes" NoPrefix="yes" Text="{\TitleBanner}!(loc.LicenseMessageEpicBoxTitleDialog)" />
        <Control Id="BannerVoid" Type="Bitmap" X="0" Y="0" Width="800" Height="50" TabSkip="no" Text="BannerFill" />
		<Control Id="BannerLogo" Type="Bitmap" X="736" Y="5" Width="40" Height="42" TabSkip="no" Text="LogoEpicGames" />
        <!-- / Header -->

        <!-- Middle Content -->
        <Control Id="AgreementText" Type="ScrollableText" X="20" Y="60" Width="760" Height="300" Sunken="no" TabSkip="no">
          <Text  SourceFile="Resources/!(loc.WixUILicenseRtf)" />
        </Control>
        <Control Id="LicenseAcceptedCheckBox" Type="CheckBox" X="20" Y="370" Width="540" Height="18" CheckBoxValue="1" Property="LicenseAccepted" Text="!(loc.LicenseAgreementDlgLicenseAcceptedCheckBox)" />
        <!-- / Middle Content-->

        <!--  Bottom Control Button -->
        <Control Id="BottomLine" Type="Line" X="0" Y="400" Width="800" Height="0" />
        <Control Id="VersionNumber" Type="Text" X="20" Y="416" Width="220" Height="20" Transparent="yes" NoPrefix="yes" Text="[ProductVersion]" />
        <Control Id="Back" Type="PushButton" X="580" Y="416" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUIBack)" />
        <Control Id="Next" Type="PushButton" X="640" Y="416" Width="56" Height="17" Default="yes" Text="!(loc.WixUINext)">
          <Condition Action="disable"><![CDATA[LicenseAccepted <> "1"]]></Condition>
          <Condition Action="enable">LicenseAccepted = "1"</Condition>
        </Control>
        <Control Id="Cancel" Type="PushButton" X="720" Y="416"  Width="56" Height="17" Cancel="yes" Text="!(loc.WixUICancel)">
          <Publish Event="SpawnDialog" Value="CancelDlg">2</Publish>
        </Control>
		<!--  / Bottom Control Button -->
      </Dialog>
    </UI>
  </Fragment>
</Wix>
