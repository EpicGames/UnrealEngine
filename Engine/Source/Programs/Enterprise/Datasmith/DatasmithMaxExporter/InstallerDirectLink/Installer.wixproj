<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="4.0" DefaultTargets="Build" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <PropertyGroup>
    <EngineSourcePath>..\..\..\..\..\</EngineSourcePath>
    <PrereqPath>$(EngineSourcePath)Programs\PrereqInstaller\</PrereqPath>
    <WixToolPath>$(EngineSourcePath)ThirdParty\WiX\3.8\</WixToolPath>
    <WixTargetsPath>$(WixToolPath)Wix.targets</WixTargetsPath>
    <WixTasksPath>$(WixToolPath)wixtasks.dll</WixTasksPath>
    <SuppressSpecificWarnings>
        1076;1077
    </SuppressSpecificWarnings>
  </PropertyGroup>
  <PropertyGroup>
    <Configuration Condition=" '$(Configuration)' == '' ">Debug</Configuration>
    <Platform Condition=" '$(Platform)' == '' ">x86</Platform>
    <ProductVersion>3.10</ProductVersion>
    <ProjectGuid>b2f238a4-8395-4202-81df-6f42f83920dc</ProjectGuid>
    <SchemaVersion>2.0</SchemaVersion>
    <OutputName>UnrealDatasmithMaxExporter</OutputName>
    <OutputType>Package</OutputType>
    <WixTasksPath>wixtasks.dll</WixTasksPath>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|x86' ">
    <OutputPath>$(SolutionDir)\bin\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Configuration)\</IntermediateOutputPath>
    <DefineConstants>CoreStylePath=..\..\..\..\..\..\Content\Slate;D3DShadersPath=..\..\..\..\..\..\Shaders\StandaloneRenderer\D3D</DefineConstants>
    <SuppressIces>ICE30;ICE82;ICE03</SuppressIces>
    <Cultures>
    </Cultures>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|x86' ">
    <OutputPath>$(SolutionDir)\bin\$(Configuration)\</OutputPath>
    <IntermediateOutputPath>obj\$(Configuration)\</IntermediateOutputPath>
    <SuppressIces>ICE30;ICE82;ICE03;ICE60</SuppressIces>
    <LeaveTemporaryFiles>False</LeaveTemporaryFiles>
    <SuppressPdbOutput>True</SuppressPdbOutput>
    <VerboseOutput>True</VerboseOutput>
    <CompilerAdditionalOptions>-trace</CompilerAdditionalOptions>
    <DefineConstants>CoreStylePath=..\..\..\..\..\..\Content\Slate;D3DShadersPath=..\..\..\..\..\..\Shaders\StandaloneRenderer\D3D</DefineConstants>
  </PropertyGroup>
  <ItemGroup>
    <Compile Include="CustomActions.wxs" />
    <Compile Include="CustomExitDialog.wxs" />
    <Compile Include="CustomInstallDirDlg.wxs" />
    <Compile Include="CustomLicenseAgreementDlg.wxs" />
    <Compile Include="CustomWelcomeDialog.wxs" />
    <Compile Include="CustomWixUI_InstallDir.wxs" />
    <Compile Include="GenericUiBrandDialog.wxs" />
    <Compile Include="MaxExporter.wxs" />
    <HarvestDirectory Include="..\..\..\..\..\..\Content\Slate">
      <DirectoryRefId>CoreStyleFolder</DirectoryRefId>
      <ComponentGroupName>CoreStyle</ComponentGroupName>
      <PreprocessorVariable>var.CoreStylePath</PreprocessorVariable>
      <Link>Slate</Link>
    </HarvestDirectory>
    <HarvestDirectory Include="..\..\..\..\..\..\Shaders\StandaloneRenderer\D3D">
      <DirectoryRefId>D3DShadersFolder</DirectoryRefId>
      <ComponentGroupName>D3DShaders</ComponentGroupName>
      <PreprocessorVariable>var.D3DShadersPath</PreprocessorVariable>
      <Link>D3D</Link>
    </HarvestDirectory>
  </ItemGroup>
  <ItemGroup>
    <WixExtension Include="WixUtilExtension">
      <HintPath>$(WixToolPath)WixUtilExtension.dll</HintPath>
      <Name>WixUtilExtension</Name>
    </WixExtension>
    <WixExtension Include="WixUIExtension">
      <HintPath>$(WixToolPath)WixUIExtension.dll</HintPath>
      <Name>WixUIExtension</Name>
    </WixExtension>
    <WixExtension Include="WixNetFxExtension">
      <HintPath>$(WixToolPath)WixNetFxExtension.dll</HintPath>
      <Name>WixNetFxExtension</Name>
    </WixExtension>
  </ItemGroup>
  <ItemGroup>
    <Folder Include="Localization" />
    <Folder Include="Resources" />
  </ItemGroup>
  <ItemGroup>
    <Content Include="Resources\Dialog.bmp" />
    <Content Include="Resources\Epic_Games_logo.png" />
    <Content Include="Resources\Exporter Plugin End User License Agreement.rtf" />
    <Content Include="Resources\UnrealEngine.ico" />
  </ItemGroup>
  <ItemGroup>
    <EmbeddedResource Include="Localization\de-DE.wxl" />
    <EmbeddedResource Include="Localization\en-US.wxl" />
    <EmbeddedResource Include="Localization\es-ES.wxl" />
    <EmbeddedResource Include="Localization\fr-FR.wxl" />
    <EmbeddedResource Include="Localization\ja-JP.wxl" />
    <EmbeddedResource Include="Localization\ko-KR.wxl" />
    <EmbeddedResource Include="Localization\pt-br.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_es-es.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_fr-fr.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_ja-jp.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_ko-kr.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_pt-br.wxl" />
    <EmbeddedResource Include="Localization\UtilExtension_zh_cn.wxl" />
    <EmbeddedResource Include="Localization\zh-CN.wxl" />
  </ItemGroup>
  <Import Project="$(WixTargetsPath)" Condition=" '$(WixTargetsPath)' != '' " />
  <PropertyGroup>
    <PostBuildEvent>call "$(ProjectDir)CreateMultilanguageInstaller.bat" $(OutDir)</PostBuildEvent>
  </PropertyGroup>
  <!--
  <Import Project="$(MSBuildExtensionsPath32)\Microsoft\WiX\v3.x\Wix.targets" Condition=" '$(WixTargetsPath)' == '' AND Exists('$(MSBuildExtensionsPath32)\Microsoft\WiX\v3.x\Wix.targets') " />
  <Target Name="EnsureWixToolsetInstalled" Condition=" '$(WixTargetsImported)' != 'true' ">
    <Error Text="The WiX Toolset v3.11 (or newer) build tools must be installed to build this project. To download the WiX Toolset, see http://wixtoolset.org/releases/" />
  </Target>
  -->
  <!--
	To modify your build process, add your task inside one of the targets below and uncomment it.
	Other similar extension points exist, see Wix.targets.
	<Target Name="BeforeBuild">
	</Target>
	<Target Name="AfterBuild">
	</Target>
	-->
</Project>