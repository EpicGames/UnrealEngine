<?xml version="1.0" encoding="utf-8"?>
<!-- Copyright (c) .NET Foundation and contributors. All rights reserved. Licensed under the Microsoft Reciprocal License. See LICENSE.TXT file in the project root for full license information. -->

<WixLocalization Culture="es-es" xmlns="http://schemas.microsoft.com/wix/2006/localization">
  <String Id="msierrUSRFailedUserCreate" Overridable="yes">La creación del usuario ha fracasado. ([2] [3] [4] [5])</String>
  <String Id="msierrUSRFailedUserCreatePswd" Overridable="yes">La creación del usuario ha fracasado porque la contraseña es incorrecta. ([2] [3] [4] [5])</String>
  <String Id="msierrUSRFailedUserGroupAdd" Overridable="yes">El aditamento del usuario al grupo ha fracasado. ([2] [3] [4] [5])</String>
  <String Id="msierrUSRFailedUserCreateExists" Overridable="yes">La creación del usuario ha fracasado porque ya existe. ([2] [3] [4] [5])</String>

  <String Id="msierrSMBFailedCreate" Overridable="yes">La creación de la red compartida ha fracasado. ([2] [3] [4] [5])</String>
  <String Id="msierrSMBFailedDrop" Overridable="yes">La eliminación de la red compartida ha fracasado. ([2] [3] [4] [5])</String>

  <String Id="msierrPERFMONFailedRegisterDLL" Overridable="yes">La inscripción al registro de la DLL con PerfMon ha fracasado. ([2] [3] [4] [5])</String>
  <String Id="msierrPERFMONFailedUnregisterDLL" Overridable="yes">La cancelación de la inscripción al registro de la DLL con PerfMon ha fracasado. ([2] [3] [4] [5])</String>

  <String Id="msierrInstallPerfCounterData" Overridable="yes">La instalación de los contadores de rendimiento ha fracasado. ([2] [3] [4] [5])</String>
  <String Id="msierrUninstallPerfCounterData" Overridable="yes">La desinstalación de los contadores de rendimiento ha fracasado. ([2] [3] [4] [5])</String>

  <String Id="msierrSecureObjectsFailedCreateSD" Overridable="yes">La creación de los ACLs ha fracasado por [3]\[4], error del sistema : [2]</String>
  <String Id="msierrSecureObjectsFailedSet" Overridable="yes">El posicionamiento de los ACLs por el objecto [3] ha fracasado, error del sistema: [2]</String>
  <String Id="msierrSecureObjectsUnknownType" Overridable="yes">Tipo de objecto no conocido [3], error del sistema: [2]</String>

  <String Id="msierrXmlFileFailedRead" Overridable="yes">Un problema ha aparecido durante la configuración de los ficheros XML.</String>
  <String Id="msierrXmlFileFailedOpen" Overridable="yes">Fracaso de la apertura de los ficheros XML [3], error del sistema: [2]</String>
  <String Id="msierrXmlFileFailedSelect" Overridable="yes">Fracaso de la búsqueda del nodo: [3] en el fichero XML: [4], error del sistema: [2]</String>
  <String Id="msierrXmlFileFailedSave" Overridable="yes">Fracaso durante la salvaguardia de las modificaciones en el fichero XML [3], error del sistema: [2]</String>

  <String Id="msierrXmlConfigFailedRead" Overridable="yes">Un problema ha aparecido durante la configuración de los ficheros XML.</String>
  <String Id="msierrXmlConfigFailedOpen" Overridable="yes">Fracaso de la apertura de los ficheros XML [3], error del sistema: [2]</String>
  <String Id="msierrXmlConfigFailedSelect" Overridable="yes">Fracaso de la búsqueda del nodo: [3] en el fichero XML: [4], error del sistema: [2]</String>
  <String Id="msierrXmlConfigFailedSave" Overridable="yes">Fracaso durante la salvaguardia de las modificaciones en el fichero XML [3], error del sistema: [2]</String>
</WixLocalization>