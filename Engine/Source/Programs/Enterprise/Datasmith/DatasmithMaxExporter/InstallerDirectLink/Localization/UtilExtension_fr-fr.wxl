<?xml version="1.0" encoding="utf-8"?>
<!-- Copyright (c) .NET Foundation and contributors. All rights reserved. Licensed under the Microsoft Reciprocal License. See LICENSE.TXT file in the project root for full license information. -->

<WixLocalization Culture="fr-fr" xmlns="http://schemas.microsoft.com/wix/2006/localization">
  <String Id="msierrUSRFailedUserCreate" Overridable="yes">La création de l'utilisateur a échoué.  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrUSRFailedUserCreatePswd" Overridable="yes">La création de l'utilisateur a échoué car le mot de passe est invalide.  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrUSRFailedUserGroupAdd" Overridable="yes">L'ajout de l'utilisateur au groupe a échoué.  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrUSRFailedUserCreateExists" Overridable="yes">La création de l'utilisateur a échoué car il existe dejà.  ([2]   [3]   [4]   [5])</String>

  <String Id="msierrSMBFailedCreate" Overridable="yes">La création du partage reseau a échoué.  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrSMBFailedDrop" Overridable="yes">La suppression du partage reseau a échoué.  ([2]   [3]   [4]   [5])</String>

  <String Id="msierrPERFMONFailedRegisterDLL" Overridable="yes">L'inscription au registre de la DLL avec PerfMon a échoué.  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrPERFMONFailedUnregisterDLL" Overridable="yes">La desinscription au registre de la DLL avec PerfMon a échoué.  ([2]   [3]   [4]   [5])</String>

  <String Id="msierrInstallPerfCounterData" Overridable="yes">L'installation des compteurs de performance a échoué.  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrUninstallPerfCounterData" Overridable="yes">La desinstallation des compteurs de performance a échoué.  ([2]   [3]   [4]   [5])</String>

  <String Id="msierrSecureObjectsFailedCreateSD" Overridable="yes">La création des ACLs a échoué pour [3]\[4], erreur systeme: [2]</String>
  <String Id="msierrSecureObjectsFailedSet" Overridable="yes">Le positionnement des ACLs pour l'objet [3] a échoué, erreur systeme: [2]</String>
  <String Id="msierrSecureObjectsUnknownType" Overridable="yes">Type d'objet inconnu [3], erreur systeme: [2]</String>

  <String Id="msierrXmlFileFailedRead" Overridable="yes">Un problème est survenu lors de la configuration des fichiers XML.</String>
  <String Id="msierrXmlFileFailedOpen" Overridable="yes">Echec de l'ouverture des fichiers XML [3], erreur systeme: [2]</String>
  <String Id="msierrXmlFileFailedSelect" Overridable="yes">Echec de la recherche du noeud: [3] dans le fichier XML: [4], erreur systeme: [2]</String>
  <String Id="msierrXmlFileFailedSave" Overridable="yes">Echec lors de la sauvegarde des modifications dans le fichier XML [3], erreur systeme: [2]</String>

  <String Id="msierrXmlConfigFailedRead" Overridable="yes">Un problème est survenu lors de la configuration des fichiers XML.</String>
  <String Id="msierrXmlConfigFailedOpen" Overridable="yes">Echec de l'ouverture des fichiers XML [3], erreur systeme: [2]</String>
  <String Id="msierrXmlConfigFailedSelect" Overridable="yes">Echec de la recherche du noeud: [3] dans le fichier XML: [4], erreur systeme: [2]</String>
  <String Id="msierrXmlConfigFailedSave" Overridable="yes">Echec lors de la sauvegarde des modifications dans le fichier XML [3], erreur systeme: [2]</String>
</WixLocalization>