<?xml version="1.0" encoding="utf-8"?>
<!-- Copyright (c) .NET Foundation and contributors. All rights reserved. Licensed under the Microsoft Reciprocal License. See LICENSE.TXT file in the project root for full license information. -->

<WixLocalization Culture="ja-jp" xmlns="http://schemas.microsoft.com/wix/2006/localization">
  <String Id="msierrUSRFailedUserCreate" Overridable="yes">ユーザー作成に失敗しました。  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrUSRFailedUserCreatePswd" Overridable="yes">パスワードが無効のためユーザー作成に失敗しました。  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrUSRFailedUserGroupAdd" Overridable="yes">ユーザーをグループに追加でいませんでした。  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrUSRFailedUserCreateExists" Overridable="yes">ユーザーが既に存在するため作成できませんでした。  ([2]   [3]   [4]   [5])</String>

  <String Id="msierrSMBFailedCreate" Overridable="yes">ネットワーク共有の作成に失敗しました。  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrSMBFailedDrop" Overridable="yes">ネットワーク共有の削除に失敗しました。  ([2]   [3]   [4]   [5])</String>

  <String Id="msierrPERFMONFailedRegisterDLL" Overridable="yes">DLL を PerfMon に登録でいませんでした。  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrPERFMONFailedUnregisterDLL" Overridable="yes">DLL を PerfMon より登録解除できませんでした。  ([2]   [3]   [4]   [5])</String>

  <String Id="msierrInstallPerfCounterData" Overridable="yes">パフォーマンス カウンタをインストールできませんでした。  ([2]   [3]   [4]   [5])</String>
  <String Id="msierrUninstallPerfCounterData" Overridable="yes">パフォーマンス カウンタをアンインストールできませんでした。  ([2]   [3]   [4]   [5])</String>

  <String Id="msierrSecureObjectsFailedCreateSD" Overridable="yes">[3]\[4] 用セキュリティ ディスクリプターを作成できませんでした、システム エラー： [2]</String>
  <String Id="msierrSecureObjectsFailedSet" Overridable="yes">オブジェクト [3] 上のセキュリティ ディスクリプターを設定できませんでした、システム エラー： [2]</String>
  <String Id="msierrSecureObjectsUnknownType" Overridable="yes">不明なオブジェクト種別 [3]、システム エラー： [2]</String>

  <String Id="msierrXmlFileFailedRead" Overridable="yes">XML ファイル構成中に失敗しました。</String>
  <String Id="msierrXmlFileFailedOpen" Overridable="yes">XML ファイル [3] を開けませんでした、システム エラー： [2]</String>
  <String Id="msierrXmlFileFailedSelect" Overridable="yes">XML ファイル [4] 内にノード [3] が見つかりませんでした、システム エラー： [2]</String>
  <String Id="msierrXmlFileFailedSave" Overridable="yes">XML ファイル [3] へ変更を保存できませんでした、システム エラー： [2]</String>
  
  <String Id="msierrXmlConfigFailedRead" Overridable="yes">XML ファイル構成中に失敗しました。</String>
  <String Id="msierrXmlConfigFailedOpen" Overridable="yes">XML ファイル [3] を開けませんでした、システム エラー： [2]</String>
  <String Id="msierrXmlConfigFailedSelect" Overridable="yes">XML ファイル [4] 内にノード [3] が見つかりませんでした、システム エラー： [2]</String>
  <String Id="msierrXmlConfigFailedSave" Overridable="yes">XML ファイル [3] へ変更を保存できませんでした、システム エラー： [2]</String>
</WixLocalization>
