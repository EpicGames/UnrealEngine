<?xml version='1.0' encoding='utf-8'?>
<WixLocalization xmlns="http://schemas.microsoft.com/wix/2006/localization" Culture="de-de">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">Es ist bereits eine neuere Version von [ProductName] installiert.</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">Es ist bereits eine identische neuere Version von [ProductName] installiert. Zum Fortfahren</String>

	<String Id="ErrorNo3dsMaxInstalled" Overridable="yes">Es wurde keine unterstützte Version von Autodesk 3ds Max gefunden.</String>
	<String Id="Error3dsMaxSupportedVersion" Overridable="yes">Die unterstützten Versionen von Autodesk 3ds Max sind: 
				Autodesk 3ds Max 2027
				Autodesk 3ds Max 2026
				Autodesk 3ds Max 2025
				Autodesk 3ds Max 2024
				Autodesk 3ds Max 2023
				Autodesk 3ds Max 2022
</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">Es wurde keine unterstützte Version von Autodesk 3ds Max gefunden. Das Installationsprogramm wird ohne Änderung des Systems beendet.</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">Klicken Sie zum Fortfahren auf „Installieren“.</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">Klicken Sie zum Fortfahren auf „Installieren“.</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">[ProductName] installieren für:</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">[ProductName] aktualisieren für:</String>

	
	<String Id="GeneralTitleDialogWizard" Overridable="yes">Einrichtungsassistent Unreal Datasmith Exporter für Autodesk 3ds Max</String>
	<String Id="DirectXInstallMessage">Installing DirectX</String>

	
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">Der Einrichtungsassistent installiert Unreal Datasmith Exporter für Autodesk 3ds Max auf deinem Computer.</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">Klicke auf „Weiter“, um fortzufahren, oder auf „Abbrechen“, um den Einrichtungsassistenten zu beenden.</String>

	
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">Endbenutzer-Lizenzvereinbarung</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">Lies dir die folgenden Lizenzvereinbarung sorgfältig durch.</String>

	
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">Endbenutzer-Lizenzvereinbarung</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">Lies dir die folgenden Lizenzvereinbarung sorgfältig durch.</String>

	
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">Unreal Datasmith Exporter für Autodesk 3ds Max wird installiert.</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">Warte, bis der Einrichtungsassistent die Installation abgeschlossen hat.</String>

	
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">Installation abgeschlossen</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">Die Installation wurde erfolgreich abgeschlossen.</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">Klicke auf „Fertigstellen“, um den Einrichtungsassistenten zu beenden.</String>
	<String Id="ErrorExitTitleBanner" Overridable="yes">Die Installation wurde unterbrochen.</String>

	
	<String Id="BackGroundCredits1" Overridable="yes">Bild mit freundlicher Genehmigung von Michael Mong</String>
	<String Id="BackGroundCredits2" Overridable="yes">Bild mit freundlicher Genehmigung von Peter Sivi / Motionland</String>
	<String Id="BackGroundCredits3" Overridable="yes">Bild mit freundlicher Genehmigung von Chen3d.com</String>
	<String Id="BackGroundCredits4" Overridable="yes">Bild mit freundlicher Genehmigung von Hoang Anh Ho</String>
	<String Id="BackGroundCredits5" Overridable="yes">Bild mit freundlicher Genehmigung von Entre+ architects</String>

	
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">Das Produkt ist bereits installiert.</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">Verwende die Windows-Systemsteuerung, um das Programm zu deinstallieren oder zu ändern.</String>
	
	
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>
</WixLocalization>