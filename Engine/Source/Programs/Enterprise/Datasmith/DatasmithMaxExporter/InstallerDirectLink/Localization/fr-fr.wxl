<?xml version='1.0' encoding='utf-8'?>
<WixLocalization xmlns="http://schemas.microsoft.com/wix/2006/localization" Culture="fr-fr">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">Dernière version de [ProductName] déjà installée.</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">Version identique de [ProductName] déjà installée.Pour continuer</String>

	<String Id="ErrorNo3dsMaxInstalled" Overridable="yes">Aucune version prise en charge d'Autodesk 3ds Max n'a été trouvée.</String>
	<String Id="Error3dsMaxSupportedVersion" Overridable="yes">Les versions d'Autodesk 3ds Max prises en charge sont :
				Autodesk 3ds Max 2027
				Autodesk 3ds Max 2026
				Autodesk 3ds Max 2025
				Autodesk 3ds Max 2024
				Autodesk 3ds Max 2023
				Autodesk 3ds Max 2022
</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">Aucune version prise en charge d'Autodesk 3ds Max n'a été trouvée. Le programme d'installation va s'arrêter sans modifier votre système.</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">Cliquez sur Installer pour continuer.</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">Cliquez sur Installer pour continuer.</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">Installer [ProductName] pour :</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">Mettre à jour [ProductName] pour :</String>

	
	<String Id="GeneralTitleDialogWizard" Overridable="yes">Assistant d'installation d'Unreal Datasmith Exporter pour Autodesk 3ds Max</String>
	<String Id="DirectXInstallMessage">Installing DirectX</String>

	
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">L'assistant va installer Unreal Datasmith Exporter pour Autodesk 3ds Max sur votre ordinateur</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">Cliquez sur Suivant pour continuer ou sur Annuler pour quitter l'assistant d'installation</String>

	
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">Contrat de licence utilisateur final</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">Veuillez lire attentivement le contrat de licence suivant</String>

	
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">Contrat de licence utilisateur final</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">Veuillez lire attentivement le contrat de licence suivant</String>

	
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">Installation d'Unreal Datasmith Exporter pour Autodesk 3ds Max</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">Veuillez attendre que l'assistant d'installation termine l'installation.</String>

	
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">Installation terminée</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">L'installation s'est terminée avec succès.</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">Cliquez sur le bouton Terminer pour quitter l'assistant d'installation.</String>
	<String Id="ErrorExitTitleBanner" Overridable="yes">L'installation a été interrompue</String>

	
	<String Id="BackGroundCredits1" Overridable="yes">Image utilisée avec l'aimable autorisation de Michael Mong</String>
	<String Id="BackGroundCredits2" Overridable="yes">Image utilisée avec l'aimable autorisation de Peter Sivi / Motionland</String>
	<String Id="BackGroundCredits3" Overridable="yes">Image utilisée avec l'aimable autorisation de Chen3d.com</String>
	<String Id="BackGroundCredits4" Overridable="yes">Image utilisée avec l'aimable autorisation de Hoang Anh Ho</String>
	<String Id="BackGroundCredits5" Overridable="yes">Image utilisée avec l'aimable autorisation d'Entre+ architects</String>

	
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">Produit déjà installé.</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">Veuillez utiliser le panneau de configuration Windows pour désinstaller ou modifier le programme.</String>
	
	
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>
</WixLocalization>