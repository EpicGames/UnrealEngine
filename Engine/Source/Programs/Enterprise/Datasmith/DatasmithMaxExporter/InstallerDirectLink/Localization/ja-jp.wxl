<?xml version='1.0' encoding='utf-8'?>
<WixLocalization xmlns="http://schemas.microsoft.com/wix/2006/localization" Culture="ja-jp">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">[ProductName] の新しいバージョンはインストール済みです</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">[ProductName] の同じバージョンがすでにインストールされています。続行するには</String>

	<String Id="ErrorNo3dsMaxInstalled" Overridable="yes">Autodesk 3ds Max のサポートされているバージョンがみつかりません</String>
	<String Id="Error3dsMaxSupportedVersion" Overridable="yes">Autodesk 3ds Max のサポートされているバージョンは次のとおりです。
				Autodesk 3ds Max 2027
				Autodesk 3ds Max 2026
				Autodesk 3ds Max 2025
				Autodesk 3ds Max 2024
				Autodesk 3ds Max 2023
				Autodesk 3ds Max 2022
</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">Autodesk 3ds Max のサポートされているバージョンがみつかりませんでした。インストーラはお使いのシステムを変更することなく終了します</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">操作を続行するには、[インストール] をクリックします</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">操作を続行するには、[インストール] をクリックします</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">次の[ProductName]をインストール:</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">次の[ProductName]を更新:</String>

	
	<String Id="GeneralTitleDialogWizard" Overridable="yes">Unreal Datasmith Exporter for Autodesk 3ds Maxセットアップウィザード</String>
	<String Id="DirectXInstallMessage">Installing DirectX</String>

	
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">セットアップ ウィザードが、お使いのコンピュータに Unreal Datasmith Exporter for Autodesk 3ds Max をインストールします</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">[次へ] をクリックして操作を続行するか、[キャンセル] をクリックしてセットアップ ウィザードを終了します</String>

	
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">エンドユーザーライセンス契約</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">次のライセンス契約をよくお読みください</String>

	
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">エンドユーザーライセンス契約</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">次のライセンス契約をよくお読みください</String>

	
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">Unreal Datasmith Exporter for Autodesk 3ds Maxのインストール</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">セットアップ ウィザードによるインストールが完了するまでお待ちください</String>

	
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">インストール完了</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">このインストールは正常に完了しました</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">[完了] ボタンをクリックして、セットアップ ウィザードを終了します</String>
	<String Id="ErrorExitTitleBanner" Overridable="yes">インストールが中断されました</String>

	
	<String Id="BackGroundCredits1" Overridable="yes">画像提供：Michael Mong</String>
	<String Id="BackGroundCredits2" Overridable="yes">画像提供: Peter Sivi / Motionland</String>
	<String Id="BackGroundCredits3" Overridable="yes">画像提供: Chen3d.com</String>
	<String Id="BackGroundCredits4" Overridable="yes">画像提供: Hoang Anh Ho</String>
	<String Id="BackGroundCredits5" Overridable="yes">画像提供: Entre+ architects</String>

	
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">製品はインストール済みです</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">Windows のコントロール パネルを使用してプログラムをアンインストールするか、変更してください</String>
	
	
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>
</WixLocalization>