<?xml version='1.0' encoding='utf-8'?>
<WixLocalization xmlns="http://schemas.microsoft.com/wix/2006/localization" Culture="ko-kr">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">[ProductName]의 최신 버전이 이미 설치되어 있습니다.</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">[ProductName]의 동일한 버전이 이미 설치되어 있습니다. 계속하려면</String>

	<String Id="ErrorNo3dsMaxInstalled" Overridable="yes">지원되는 Autodesk 3ds Max 버전을 찾을 수 없습니다.</String>
	<String Id="Error3dsMaxSupportedVersion" Overridable="yes">지원되는 Autodesk 3ds Max 버전은 다음과 같습니다:
				Autodesk 3ds Max 2027
				Autodesk 3ds Max 2026
				Autodesk 3ds Max 2025
				Autodesk 3ds Max 2024
				Autodesk 3ds Max 2023
				Autodesk 3ds Max 2022
</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">지원되는 Autodesk 3ds Max 버전을 찾을 수 없습니다. 인스톨러가 시스템을 수정하지 않고 종료됩니다.</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">계속하려면 설치를 클릭합니다.</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">계속하려면 설치를 클릭합니다.</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">[ProductName] 설치 대상:</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">[ProductName] 업데이트 대상:</String>

	
	<String Id="GeneralTitleDialogWizard" Overridable="yes">Autodesk 3ds Max용 언리얼 데이터스미스 익스포터 설치 마법사</String>
	<String Id="DirectXInstallMessage">Installing DirectX</String>

	
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">설치 마법사가 Autodesk 3ds Max용 언리얼 데이터스미스 익스포터 를 컴퓨터에 설치합니다.</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">다음을 클릭하여 계속하거나, 취소하여 설정 마법사를 종료합니다.</String>

	
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">최종 이용자 라이선스 계약</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">다음 라이선스 계약을 주의 깊게 읽어주세요.</String>

	
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">최종 이용자 라이선스 계약</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">다음 라이선스 계약을 주의 깊게 읽어주세요.</String>

	
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">Autodesk 3ds Max용 언리얼 데이터스미스 익스포터를 설치하는 중입니다.</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">설치 마법사가 설치를 완료할 때까지 잠시 기다려 주세요.</String>

	
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">설치 완료!</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">설치가 성공적으로 완료되었습니다.</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">마침 버튼을 클릭하여 설치 마법사를 종료합니다.</String>
	<String Id="ErrorExitTitleBanner" Overridable="yes">설치가 중단되었습니다.</String>

	
	<String Id="BackGroundCredits1" Overridable="yes">Michael Mong 이미지 제공</String>
	<String Id="BackGroundCredits2" Overridable="yes">Peter Sivi / Motionland 이미지 제공</String>
	<String Id="BackGroundCredits3" Overridable="yes">Chen3d.com 이미지 제공</String>
	<String Id="BackGroundCredits4" Overridable="yes">Hoang Anh Ho 이미지 제공</String>
	<String Id="BackGroundCredits5" Overridable="yes">Entre+ architects 이미지 제공</String>

	
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">제품이 이미 설치되어 있습니다.</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">프로그램을 제거하거나 변경하려면 Windows 제어판을 사용하세요.</String>
	
	
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>
</WixLocalization>