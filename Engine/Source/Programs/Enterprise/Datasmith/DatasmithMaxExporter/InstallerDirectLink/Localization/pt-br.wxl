<?xml version='1.0' encoding='utf-8'?>
<WixLocalization xmlns="http://schemas.microsoft.com/wix/2006/localization" Culture="pt-br">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">Uma versão mais recente do [ProductName] já está instalada.</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">Uma versão idêntica do [ProductName] já está instalada. Para continuar</String>

	<String Id="ErrorNo3dsMaxInstalled" Overridable="yes">Não foi encontrada uma versão compatível do Autodesk 3ds Max.</String>
	<String Id="Error3dsMaxSupportedVersion" Overridable="yes">As versões compatíveis do Autodesk 3ds Max são: 
				Autodesk 3ds Max 2027
				Autodesk 3ds Max 2026
				Autodesk 3ds Max 2026
				Autodesk 3ds Max 2025
				Autodesk 3ds Max 2024
				Autodesk 3ds Max 2023
				Autodesk 3ds Max 2022
</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">Não foi encontrada uma versão compatível do Autodesk 3ds Max. O instalador será encerrado sem modificar seu sistema.</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">Clique em Instalar para continuar.</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">Clique em Instalar para continuar.</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">Instalar [ProductName] para:</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">Atualizar [ProductName] para:</String>

	
	<String Id="GeneralTitleDialogWizard" Overridable="yes">Assistente de Instalação do Unreal Datasmith Exporter para Autodesk 3ds Max</String>
	<String Id="DirectXInstallMessage">Installing DirectX</String>

	
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">O Assistente de Instalação instalará o Unreal Datasmith Exporter para Autodesk 3ds Max no seu computador</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">Clique em Avançar para continuar ou em Cancelar para sair do Assistente de Instalação</String>

	
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">Contrato de Licença do Usuário Final</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">Leia o contrato de licença a seguir com atenção</String>

	
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">Contrato de Licença do Usuário Final</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">Leia o contrato de licença a seguir com atenção</String>

	
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">Instalando o Unreal Datasmith Exporter para Autodesk 3ds Max</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">Aguarde enquanto o Assistente de Instalação conclui a instalação.</String>

	
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">Instalação Concluída</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">A instalação foi concluída com êxito.</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">Clique no botão Concluir para sair do Assistente de Instalação.</String>
	<String Id="ErrorExitTitleBanner" Overridable="yes">A instalação foi interrompida</String>

	
	<String Id="BackGroundCredits1" Overridable="yes">Imagem cortesia de Michael Mong</String>
	<String Id="BackGroundCredits2" Overridable="yes">Imagem cortesia de Peter Sivi / Motionland</String>
	<String Id="BackGroundCredits3" Overridable="yes">Imagem cortesia de Chen3d.com</String>
	<String Id="BackGroundCredits4" Overridable="yes">Imagem cortesia de Hoang Anh Ho</String>
	<String Id="BackGroundCredits5" Overridable="yes">Imagem cortesia da Entre+ architects</String>

	
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">Produto já instalado.</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">Use o painel de controle do Windows para desinstalar ou alterar o programa.</String>
	
	
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>
</WixLocalization>