<?xml version='1.0' encoding='utf-8'?>
<WixLocalization xmlns="http://schemas.microsoft.com/wix/2006/localization" Culture="zh-cn">

	<String Id="ErrorOlderVersionInstall" Overridable="yes">已安装了更新版本的[ProductName]。</String>

	<String Id="ErrorSameVersionInstall" Overridable="yes">已安装了相同版本的[ProductName]。要继续</String>

	<String Id="ErrorNo3dsMaxInstalled" Overridable="yes">发现不受支持的Autodesk 3ds Max版本。</String>
	<String Id="Error3dsMaxSupportedVersion" Overridable="yes">受支持的Autodesk 3ds Max版本有：
				Autodesk 3ds Max 2026
				Autodesk 3ds Max 2025
				Autodesk 3ds Max 2024
				Autodesk 3ds Max 2023
				Autodesk 3ds Max 2022
</String>

	<String Id="ErrorNothingToInstall" Overridable="yes">未发现受支持的Autodesk 3ds Max版本。安装程序将在不修改系统的情况下退出。</String>

	<String Id="InstallDirDlgDescription" Overridable="yes">点击“安装”以继续。</String>
	<String Id="InstallDirDlgDescriptionUpgrade" Overridable="yes">点击“安装”以继续。</String>
	<String Id="InstallDirDlgMessage" Overridable="yes">为其安装[ProductName]：</String>
	<String Id="InstallDirDlgMessageUpgrade" Overridable="yes">为其更新[ProductName]：</String>

	
	<String Id="GeneralTitleDialogWizard" Overridable="yes">面向Autodesk 3ds Max的虚幻Datasmith导出器设置向导</String>
	<String Id="DirectXInstallMessage">Installing DirectX</String>

	
	<String Id="WelcomeEpicMessageBoxContentLine1" Overridable="yes">设置向导将在你的计算机上安装用于Autodesk 3ds Max的虚幻Datasmith导出器</String>
	<String Id="WelcomeEpicMessageBoxContentLine2" Overridable="yes">点击“下一步”继续，或点击“取消”退出设置向导</String>

	
	<String Id="LicenseMessageEpicBoxTitleDialog" Overridable="yes">最终用户许可协议</String>
	<String Id="LicenseMessageEpicBoxTitleDialogSub" Overridable="yes">请仔细阅读以下许可协议</String>

	
	<String Id="InstallationChoiceMessageEpicBoxTitleDialog" Overridable="yes">最终用户许可协议</String>
	<String Id="InstallationChoiceMessageEpicBoxTitleDialogSub" Overridable="yes">请仔细阅读以下许可协议</String>

	
	<String Id="InstallMessageEpicBoxTitleDialog" Overridable="yes">正在安装面向Autodesk 3ds Max的虚幻Datasmith导出器</String>
	<String Id="InstallMessageEpicMessageBoxContentLine1" Overridable="yes">请等待设置向导完成安装。</String>

	
	<String Id="ExitMessageEpicBoxTitleDialog" Overridable="yes">安装完成</String>
	<String Id="ExitMessageEpicMessageBoxContentLine1" Overridable="yes">安装已顺利完成。</String>
	<String Id="ExitMessageEpicBoxContentLine2" Overridable="yes">点击“完成”按钮退出设置向导。</String>
	<String Id="ErrorExitTitleBanner" Overridable="yes">安装终端</String>

	
	<String Id="BackGroundCredits1" Overridable="yes">图片由Michael Mong提供</String>
	<String Id="BackGroundCredits2" Overridable="yes">图片由Peter Sivi / Motionland提供</String>
	<String Id="BackGroundCredits3" Overridable="yes">图片由Chen3d.com提供</String>
	<String Id="BackGroundCredits4" Overridable="yes">图片由Hoang Anh Ho提供</String>
	<String Id="BackGroundCredits5" Overridable="yes">图片由Entre+ architects提供</String>

	
	<String Id="ManagementPannelMessageAlreadyInstall" Overridable="yes">产品已安装。</String>
	<String Id="ManagementPannelMessageActions" Overridable="yes">请使用Windows控制面板卸载或更改程序。</String>
	
	
	<String Id="WixUILicenseRtf" Overridable="yes">Exporter Plugin End User License Agreement.rtf</String>
</WixLocalization>