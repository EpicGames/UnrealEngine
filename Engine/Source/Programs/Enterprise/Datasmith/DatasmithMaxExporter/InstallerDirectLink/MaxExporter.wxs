<?xml version="1.0" encoding="UTF-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi" xmlns:util="http://schemas.microsoft.com/wix/UtilExtension">
	<!-- Include version -->
	<?include ..\..\DatasmithInstallerCommon\ExporterVersion.wxi?>

	<?define ProductVersion="$(var.MajorVersion).$(var.MinorVersion).$(var.PatchVersion).$(var.BuildVersion)"?>

	<!-- NOTE: Language codes can be found here(LCID Decimal column): http://www.science.co.il/Language/Locale-codes.asp -->
	<!-- en-US:1033, de-de:1031, ko-kr:1042, es-es:1034, fr-fr:1036, ja-jp:1041, pt-br:1046, zh-cn:2052 -->
	<?define Languages = "1033,1031,1042,1034,1036,1041,1046,2052"?>

	<!-- Identify the specific product release. Using an asterisk rather than a GUID recreates a new product code each time the installer is rebuilt. -->
	<?define ProductCode="*"?>

	<!-- The Upgrade Code. Never changes. This is the true "Unreal Engine Launcher" aplication ID, and keeping it the same allows us to recognise existing installations. -->
	<?define UpgradeCode="{AC65B10D-82C7-4492-8525-B0129219871F}"?>

	<!-- Defines a relative path to the main Engine directory -->
	<?define EngineDir="..\..\..\..\..\..\..\Engine"?>

	<!-- Defines a relative path to our resources directory -->
	<?define ResourcesDir="./Resources"?>

	<!-- Defines a relative path to directory where Max plugins are stored -->
	<?define BinaryDir="$(var.EngineDir)\Binaries\Win64\3DSMax"?>

	<!-- Defines a relative path to directory where Max scripts are stored -->
	<?define ScriptDir="..\Extras\3dsMaxScripts"?>

	<!-- Defines a relative path to the ThirdParty libraries -->
	<?define ThirdPartyDir="$(var.EngineDir)\Source\ThirdParty"?>

	<Product Id="$(var.ProductCode)" Name="Unreal Datasmith Exporter for 3ds Max" Language="1033" Version="$(var.ProductVersion)" Manufacturer="Epic Games, Inc." UpgradeCode="$(var.UpgradeCode)">

		<Package
			Platform="x64"
			InstallerVersion="405"
			Languages="$(var.Languages)"
			Compressed="yes"
			InstallScope="perMachine" />

		<!-- Find other installed versions when the product (with this product code) is first installed, but not during maintenance mode or uninstallation. -->
		<Upgrade Id="$(var.UpgradeCode)">
			<!-- NEWPRODUCTFOUND will be set if there is a newer product already installed -->
			<UpgradeVersion
				Minimum="$(var.ProductVersion)"
				IncludeMinimum="no"
				OnlyDetect="yes"
				Language="$(var.Languages)"
				Property="NEWPRODUCTFOUND" />

			<!-- UPGRADEFOUND will be set if there is an older or same product already installed -->
			<UpgradeVersion
				Minimum="0.0.0"
				Maximum="$(var.ProductVersion)"
				IncludeMinimum="yes"
				IncludeMaximum="no"
				Language="$(var.Languages)"
				Property="UPGRADEFOUND" />

			<!-- ANOTHERBUILDINSTALLED will be set if the same product version is already installed -->
			<UpgradeVersion
				Minimum="$(var.ProductVersion)"
				Maximum="$(var.ProductVersion)"
				IncludeMinimum="yes"
				IncludeMaximum="yes"
				OnlyDetect="yes"
				Language="$(var.Languages)"
				Property="ANOTHERBUILDINSTALLED" />
		</Upgrade>

		<!-- Simple Dialog to inform user that installation has aborted because no supported version of 3ds Max has been found -->
		<UI>
			<UIRef Id="CustomWixUI_InstallDir"/>
			<UIRef Id="WixUI_ErrorProgressText"/>

			<!--These dialog references are required for util:CloseApplication to work properly -->
			<DialogRef Id="FilesInUse" />
			<DialogRef Id="MsiRMFilesInUse" />
		</UI>

		<!-- Create the icon for our shortcuts -->
		<!-- NOTE: ARPPRODUCTICON is a special property recognized by Windows Installer, which sets up this icon for us -->
		<Icon Id="UnrealEngine.ico" SourceFile="$(var.ResourcesDir)/UnrealEngine.ico" />
		<Property Id="ARPPRODUCTICON" Value="UnrealEngine.ico" />

		<!-- Customize artwork -->
		<WixVariable Id="WixUIBannerBmp" Value="$(var.ResourcesDir)/default_small_banner.bmp" />
		<WixVariable Id="WixUIDialogBmp" Value="$(var.ResourcesDir)/Dialog.bmp"/>

		<!-- Declaration of properties MAX20xxDIR, used to store installation directory for supported versions of 3ds Max -->
		<!-- @note: Windows Installer is setting those properties to C:\ if registry key is not found -->
		<Property Id="MAX2027DIR">
			<RegistrySearch Id="MAX2027Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\29.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2026DIR">
			<RegistrySearch Id="MAX2026Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\28.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2025DIR">
			<RegistrySearch Id="MAX2025Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\27.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2024DIR">
			<RegistrySearch Id="MAX2024Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\26.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2023DIR">
			<RegistrySearch Id="MAX2023Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\25.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>
		<Property Id="MAX2022DIR">
			<RegistrySearch Id="MAX2022Search" Root="HKLM" Key="SOFTWARE\Autodesk\3dsMax\24.0" Name="Installdir" Type="raw" Win64="yes" />
		</Property>

		<Property Id="DATASMITHUIDIR" Value="C:\ProgramData\Epic\Exporter\"/>
		
		<!-- Declaration of properties MAX20xxVALID used to check if the corresponding install directory of 3ds Max is valid -->
		<Property Id="MAX2027VALID">
			<DirectorySearch Id="CheckMax2027Dir" Path="[MAX2027DIR]" Depth="0">
				<FileSearch Id="CheckMax2027File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2026VALID">
			<DirectorySearch Id="CheckMax2026Dir" Path="[MAX2026DIR]" Depth="0">
				<FileSearch Id="CheckMax2026File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2025VALID">
			<DirectorySearch Id="CheckMax2025Dir" Path="[MAX2025DIR]" Depth="0">
				<FileSearch Id="CheckMax2025File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2024VALID">
			<DirectorySearch Id="CheckMax2024Dir" Path="[MAX2024DIR]" Depth="0">
				<FileSearch Id="CheckMax2024File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2023VALID">
			<DirectorySearch Id="CheckMax2023Dir" Path="[MAX2023DIR]" Depth="0">
				<FileSearch Id="CheckMax2023File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>
		<Property Id="MAX2022VALID">
			<DirectorySearch Id="CheckMax2022Dir" Path="[MAX2022DIR]" Depth="0">
				<FileSearch Id="CheckMax2022File" Name="3dsmax.exe" />
			</DirectorySearch>
		</Property>

		<Property Id="WIXUI_INSTALLDIR" Value="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax" />
		<Property Id="WIXUI_FINALINSTALLDIR" Value="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax" />

		<!-- Common action to declare type of content in msi -->
		<Media Id="1" Cabinet="Datasmith3dsmaxExporter.cab" EmbedCab="yes" />

		<DirectoryRef Id="TARGETDIR">
			<Component Id="RegistryEntries" Guid="{E65D9DC1-71C2-4D07-92B8-6330DB563E4D}">
				<RegistryKey Root="HKLM" Key="Software\EpicGames\Unreal Engine" ForceCreateOnInstall="yes">
					<Permission User="[WIX_ACCOUNT_USERS]" GenericAll="yes" />
					<RegistryValue Type="string" Name="3dsMaxEngineDir" Value="[ENGINEFOLDER]" />
				</RegistryKey>
			</Component>
		</DirectoryRef>

		<!-- Group of features to install. Each one if conditional to the existence of 3dsmax.exe in install folder -->
		<!-- @note: Each sub-feature is disabled ('Level = 0') at creation. The feature is enabled, if 3dsmax.exe has been found in the corresponding installation folder. -->
		<Feature Id="ProductFeature" Title="Installer" Level="1">
			<Feature Id="FeatureCommon" Title="Installer" Level="1">
				<ComponentRef Id="EXPORTERCOMMON" />
				<ComponentRef Id="EXPORTERCOMMONUI" />
				<ComponentRef Id="EXPORTERCOMMONUI_EN_US" />
				<ComponentGroupRef Id="CoreStyle" />
				<ComponentGroupRef Id="D3DShaders" />
				<ComponentRef Id="EMPTYENGINEBINARIESDIR" />
				<ComponentRef Id="RegistryEntries" />
			</Feature>

			<Feature Id="Feature2022" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2022" />
				<ComponentRef Id="EXPORTER2022RIBBON_de_DE" />
				<ComponentRef Id="EXPORTER2022RIBBON_en_US" />
				<ComponentRef Id="EXPORTER2022RIBBON_fr_FR" />
				<ComponentRef Id="EXPORTER2022RIBBON_ja_JP" />
				<ComponentRef Id="EXPORTER2022RIBBON_ko_KR" />
				<ComponentRef Id="EXPORTER2022RIBBON_pt_BR" />
				<ComponentRef Id="EXPORTER2022RIBBON_zh_CN" />
				<ComponentRef Id="EXPORTER2022DARKICONS" />
				<ComponentRef Id="EXPORTER2022LIGHTICONS" />
				<Condition Level="1"><![CDATA[MAX2022CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2023" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2023" />
				<ComponentRef Id="EXPORTER2023RIBBON_de_DE" />
				<ComponentRef Id="EXPORTER2023RIBBON_en_US" />
				<ComponentRef Id="EXPORTER2023RIBBON_fr_FR" />
				<ComponentRef Id="EXPORTER2023RIBBON_ja_JP" />
				<ComponentRef Id="EXPORTER2023RIBBON_ko_KR" />
				<ComponentRef Id="EXPORTER2023RIBBON_pt_BR" />
				<ComponentRef Id="EXPORTER2023RIBBON_zh_CN" />
				<ComponentRef Id="EXPORTER2023DARKICONS" />
				<ComponentRef Id="EXPORTER2023LIGHTICONS" />
				<Condition Level="1"><![CDATA[MAX2023CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2024" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2024" />
				<ComponentRef Id="EXPORTER2024RIBBON_de_DE" />
				<ComponentRef Id="EXPORTER2024RIBBON_en_US" />
				<ComponentRef Id="EXPORTER2024RIBBON_fr_FR" />
				<ComponentRef Id="EXPORTER2024RIBBON_ja_JP" />
				<ComponentRef Id="EXPORTER2024RIBBON_ko_KR" />
				<ComponentRef Id="EXPORTER2024RIBBON_pt_BR" />
				<ComponentRef Id="EXPORTER2024RIBBON_zh_CN" />
				<ComponentRef Id="EXPORTER2024DARKICONS" />
				<ComponentRef Id="EXPORTER2024LIGHTICONS" />
				<Condition Level="1"><![CDATA[MAX2024CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2025" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2025" />
				<ComponentRef Id="EXPORTER2025RIBBON_de_DE" />
				<ComponentRef Id="EXPORTER2025RIBBON_en_US" />
				<ComponentRef Id="EXPORTER2025RIBBON_fr_FR" />
				<ComponentRef Id="EXPORTER2025RIBBON_ja_JP" />
				<ComponentRef Id="EXPORTER2025RIBBON_ko_KR" />
				<ComponentRef Id="EXPORTER2025RIBBON_pt_BR" />
				<ComponentRef Id="EXPORTER2025RIBBON_zh_CN" />
				<ComponentRef Id="EXPORTER2025DARKICONS" />
				<ComponentRef Id="EXPORTER2025LIGHTICONS" />
				<Condition Level="1"><![CDATA[MAX2025CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2027" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2027" />
				<ComponentRef Id="EXPORTER2027RIBBON_de_DE" />
				<ComponentRef Id="EXPORTER2027RIBBON_en_US" />
				<ComponentRef Id="EXPORTER2027RIBBON_fr_FR" />
				<ComponentRef Id="EXPORTER2027RIBBON_ja_JP" />
				<ComponentRef Id="EXPORTER2027RIBBON_ko_KR" />
				<ComponentRef Id="EXPORTER2027RIBBON_pt_BR" />
				<ComponentRef Id="EXPORTER2027RIBBON_zh_CN" />
				<ComponentRef Id="EXPORTER2027DARKICONS" />
				<ComponentRef Id="EXPORTER2027LIGHTICONS" />
				<Condition Level="1"><![CDATA[MAX2027CHECKED = 1]]></Condition>
			</Feature>
			<Feature Id="Feature2026" Title="Installer" Level="0">
				<ComponentRef Id="EXPORTER2026" />
				<ComponentRef Id="EXPORTER2026RIBBON_de_DE" />
				<ComponentRef Id="EXPORTER2026RIBBON_en_US" />
				<ComponentRef Id="EXPORTER2026RIBBON_fr_FR" />
				<ComponentRef Id="EXPORTER2026RIBBON_ja_JP" />
				<ComponentRef Id="EXPORTER2026RIBBON_ko_KR" />
				<ComponentRef Id="EXPORTER2026RIBBON_pt_BR" />
				<ComponentRef Id="EXPORTER2026RIBBON_zh_CN" />
				<ComponentRef Id="EXPORTER2026DARKICONS" />
				<ComponentRef Id="EXPORTER2026LIGHTICONS" />
				<Condition Level="1"><![CDATA[MAX2026CHECKED = 1]]></Condition>
			</Feature>			
		</Feature>

		<SetProperty Id="MAX2027CHECKED" After="AppSearch" Value="1">MAX2027VALID</SetProperty>
		<SetProperty Id="MAX2026CHECKED" After="AppSearch" Value="1">MAX2026VALID</SetProperty>
		<SetProperty Id="MAX2025CHECKED" After="AppSearch" Value="1">MAX2025VALID</SetProperty>
		<SetProperty Id="MAX2024CHECKED" After="AppSearch" Value="1">MAX2024VALID</SetProperty>
		<SetProperty Id="MAX2023CHECKED" After="AppSearch" Value="1">MAX2023VALID</SetProperty>
		<SetProperty Id="MAX2022CHECKED" After="AppSearch" Value="1">MAX2022VALID</SetProperty>
		
		<SetProperty Id="NOTFOUND3DSMAX" After="AppSearch" Value="1">NOT MAX2027VALID AND NOT MAX2026VALID AND NOT MAX2025VALID AND NOT MAX2024VALID AND NOT MAX2023VALID AND NOT MAX2022VALID</SetProperty>

		<InstallExecuteSequence>

			<!-- Stop the user from installing an older product version -->
			<Custom Action="PreventDowngrading" After="FindRelatedProducts">NEWPRODUCTFOUND</Custom>

			<!-- Stop the user from installing the same product version -->
			<Custom Action="PreventSameVersionInstall" After="PreventDowngrading">ANOTHERBUILDINSTALLED</Custom>

			<!-- Remove the old product before installing in case this is the same MSI build with isntallation to the same location. -->
			<RemoveExistingProducts Before="InstallInitialize" />

		</InstallExecuteSequence>

		<!-- Setup install UI sequence -->
		<InstallUISequence>

			<!-- Abort install if no version of the exporter is selected -->
			<Custom Action="WixExitEarlyWithSuccess" Before="ProgressDlg">NOT MAX2027CHECKED AND NOT MAX2026CHECKED AND NOT MAX2025CHECKED AND NOT MAX2024CHECKED AND NOT MAX2023CHECKED AND NOT MAX2022CHECKED</Custom>

			<!-- Abort install if no supported version of 3ds Max is installed -->
			<Custom Action="No3dsMaxInstaller" Before="ValidateProductID">NOTFOUND3DSMAX</Custom>

			<!-- After searching for previously installed products, prevent downgrading -->
			<Custom Action="PreventDowngrading" After="FindRelatedProducts">NEWPRODUCTFOUND</Custom>

			<!-- After searching for previously installed products, prevent overwriting -->
			<Custom Action="PreventSameVersionInstall" After="PreventDowngrading">ANOTHERBUILDINSTALLED</Custom>

		</InstallUISequence>
	</Product>

	<Fragment>
		<Directory Id="TARGETDIR" Name="SourceDir">

			<Directory Id="DATASMITHUIDIR" ComponentGuidGenerationSeed="{7DD763D6-5B8B-4FD5-A7D0-ABABE318FFC8}">
				<Directory Id="ENGINEFOLDER" Name="3dsMaxEngine">
					<Directory Id="BINARIESFOLDER" Name="Binaries" />
					<Directory Id="CONTENTFOLDER" Name="Content">
						<Directory Id="CoreStyleFolder" />
					</Directory>
					<Directory Id="SHADERSFOLDER" Name="Shaders">
						<Directory Id="STANDALONERENDERERFOLDER" Name="StandaloneRenderer">
							<Directory Id="D3DShadersFolder" />
						</Directory>
					</Directory>
				</Directory>
			</Directory>


			<!-- Install 3ds Max plugin into C:\ProgramData\Autodesk\ApplicationPlugins -->
			<Directory Id="CommonAppDataFolder">
				<Directory Id="AutodeskUserFolder" Name="Autodesk">
					<Directory Id="AutodeskApplicationPluginsUserFolder" Name="ApplicationPlugins">
						<Directory Id="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax" Name="UnrealDatasmithMax">
							<Directory Id="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_UI" Name="UI">
								<Directory Id="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_UI_en_Us" Name="en-Us">
								</Directory>
							</Directory>
							<Directory Id="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2022" Name="2022">
							</Directory>
							<Directory Id="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2023" Name="2023">
							</Directory>
							<Directory Id="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2024" Name="2024">
							</Directory>
							<Directory Id="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2025" Name="2025">
							</Directory>
							<Directory Id="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2026" Name="2026">
							</Directory>
							<Directory Id="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2027" Name="2027">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
			</Directory>
			<Directory Id="MAX2022DIR">
				<Directory Id="INSTALLYYY2022DIR_UI_ln" Name="UI_ln">
					<Directory Id="INSTALLYYY2022DIR_UI_ln_Resources" Name="Resources">
						<Directory Id="INSTALLYYY2022DIR_UI_ln_Resources_Dark" Name="Dark">
							<Directory Id="INSTALLYYY2022DIR_UI_ln_Resources_Dark_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2022DIR_UI_ln_Resources_Dark_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
						<Directory Id="INSTALLYYY2022DIR_UI_ln_Resources_Light" Name="Light">
							<Directory Id="INSTALLYYY2022DIR_UI_ln_Resources_Light_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2022DIR_UI_ln_Resources_Light_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2022DIR_de_DE" Name="de-DE">
					<Directory Id="INSTALLXXX2022DIR_de_DE_UI" Name="UI">
						<Directory Id="INSTALLXXX2022DIR_de_DE_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2022DIR_de_DE_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2022DIR_en_US" Name="en-US">
					<Directory Id="INSTALLXXX2022DIR_en_US_UI" Name="UI">
						<Directory Id="INSTALLXXX2022DIR_en_US_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2022DIR_en_US_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2022DIR_fr_FR" Name="fr-FR">
					<Directory Id="INSTALLXXX2022DIR_fr_FR_UI" Name="UI">
						<Directory Id="INSTALLXXX2022DIR_fr_FR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2022DIR_fr_FR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2022DIR_ja_JP" Name="ja-JP">
					<Directory Id="INSTALLXXX2022DIR_ja_JP_UI" Name="UI">
						<Directory Id="INSTALLXXX2022DIR_ja_JP_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2022DIR_ja_JP_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2022DIR_ko_KR" Name="ko-KR">
					<Directory Id="INSTALLXXX2022DIR_ko_KR_UI" Name="UI">
						<Directory Id="INSTALLXXX2022DIR_ko_KR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2022DIR_ko_KR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2022DIR_pt_BR" Name="pt-BR">
					<Directory Id="INSTALLXXX2022DIR_pt_BR_UI" Name="UI">
						<Directory Id="INSTALLXXX2022DIR_pt_BR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2022DIR_pt_BR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2022DIR_zh_CN" Name="zh-CN">
					<Directory Id="INSTALLXXX2022DIR_zh_CN_UI" Name="UI">
						<Directory Id="INSTALLXXX2022DIR_zh_CN_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2022DIR_zh_CN_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
			</Directory>
			<Directory Id="MAX2023DIR">
				<Directory Id="INSTALLYYY2023DIR_UI_ln" Name="UI_ln">
					<Directory Id="INSTALLYYY2023DIR_UI_ln_Resources" Name="Resources">
						<Directory Id="INSTALLYYY2023DIR_UI_ln_Resources_Dark" Name="Dark">
							<Directory Id="INSTALLYYY2023DIR_UI_ln_Resources_Dark_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2023DIR_UI_ln_Resources_Dark_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
						<Directory Id="INSTALLYYY2023DIR_UI_ln_Resources_Light" Name="Light">
							<Directory Id="INSTALLYYY2023DIR_UI_ln_Resources_Light_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2023DIR_UI_ln_Resources_Light_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2023DIR_de_DE" Name="de-DE">
					<Directory Id="INSTALLXXX2023DIR_de_DE_UI" Name="UI">
						<Directory Id="INSTALLXXX2023DIR_de_DE_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2023DIR_de_DE_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2023DIR_en_US" Name="en-US">
					<Directory Id="INSTALLXXX2023DIR_en_US_UI" Name="UI">
						<Directory Id="INSTALLXXX2023DIR_en_US_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2023DIR_en_US_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2023DIR_fr_FR" Name="fr-FR">
					<Directory Id="INSTALLXXX2023DIR_fr_FR_UI" Name="UI">
						<Directory Id="INSTALLXXX2023DIR_fr_FR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2023DIR_fr_FR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2023DIR_ja_JP" Name="ja-JP">
					<Directory Id="INSTALLXXX2023DIR_ja_JP_UI" Name="UI">
						<Directory Id="INSTALLXXX2023DIR_ja_JP_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2023DIR_ja_JP_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2023DIR_ko_KR" Name="ko-KR">
					<Directory Id="INSTALLXXX2023DIR_ko_KR_UI" Name="UI">
						<Directory Id="INSTALLXXX2023DIR_ko_KR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2023DIR_ko_KR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2023DIR_pt_BR" Name="pt-BR">
					<Directory Id="INSTALLXXX2023DIR_pt_BR_UI" Name="UI">
						<Directory Id="INSTALLXXX2023DIR_pt_BR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2023DIR_pt_BR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2023DIR_zh_CN" Name="zh-CN">
					<Directory Id="INSTALLXXX2023DIR_zh_CN_UI" Name="UI">
						<Directory Id="INSTALLXXX2023DIR_zh_CN_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2023DIR_zh_CN_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
			</Directory>
			<Directory Id="MAX2024DIR">
				<Directory Id="INSTALLYYY2024DIR_UI_ln" Name="UI_ln">
					<Directory Id="INSTALLYYY2024DIR_UI_ln_Resources" Name="Resources">
						<Directory Id="INSTALLYYY2024DIR_UI_ln_Resources_Dark" Name="Dark">
							<Directory Id="INSTALLYYY2024DIR_UI_ln_Resources_Dark_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2024DIR_UI_ln_Resources_Dark_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
						<Directory Id="INSTALLYYY2024DIR_UI_ln_Resources_Light" Name="Light">
							<Directory Id="INSTALLYYY2024DIR_UI_ln_Resources_Light_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2024DIR_UI_ln_Resources_Light_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2024DIR_de_DE" Name="de-DE">
					<Directory Id="INSTALLXXX2024DIR_de_DE_UI" Name="UI">
						<Directory Id="INSTALLXXX2024DIR_de_DE_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2024DIR_de_DE_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2024DIR_en_US" Name="en-US">
					<Directory Id="INSTALLXXX2024DIR_en_US_UI" Name="UI">
						<Directory Id="INSTALLXXX2024DIR_en_US_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2024DIR_en_US_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2024DIR_fr_FR" Name="fr-FR">
					<Directory Id="INSTALLXXX2024DIR_fr_FR_UI" Name="UI">
						<Directory Id="INSTALLXXX2024DIR_fr_FR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2024DIR_fr_FR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2024DIR_ja_JP" Name="ja-JP">
					<Directory Id="INSTALLXXX2024DIR_ja_JP_UI" Name="UI">
						<Directory Id="INSTALLXXX2024DIR_ja_JP_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2024DIR_ja_JP_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2024DIR_ko_KR" Name="ko-KR">
					<Directory Id="INSTALLXXX2024DIR_ko_KR_UI" Name="UI">
						<Directory Id="INSTALLXXX2024DIR_ko_KR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2024DIR_ko_KR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2024DIR_pt_BR" Name="pt-BR">
					<Directory Id="INSTALLXXX2024DIR_pt_BR_UI" Name="UI">
						<Directory Id="INSTALLXXX2024DIR_pt_BR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2024DIR_pt_BR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2024DIR_zh_CN" Name="zh-CN">
					<Directory Id="INSTALLXXX2024DIR_zh_CN_UI" Name="UI">
						<Directory Id="INSTALLXXX2024DIR_zh_CN_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2024DIR_zh_CN_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
			</Directory>
			<Directory Id="MAX2025DIR">
				<Directory Id="INSTALLYYY2025DIR_UI_ln" Name="UI_ln">
					<Directory Id="INSTALLYYY2025DIR_UI_ln_Resources" Name="Resources">
						<Directory Id="INSTALLYYY2025DIR_UI_ln_Resources_Dark" Name="Dark">
							<Directory Id="INSTALLYYY2025DIR_UI_ln_Resources_Dark_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2025DIR_UI_ln_Resources_Dark_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
						<Directory Id="INSTALLYYY2025DIR_UI_ln_Resources_Light" Name="Light">
							<Directory Id="INSTALLYYY2025DIR_UI_ln_Resources_Light_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2025DIR_UI_ln_Resources_Light_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2025DIR_de_DE" Name="de-DE">
					<Directory Id="INSTALLXXX2025DIR_de_DE_UI" Name="UI">
						<Directory Id="INSTALLXXX2025DIR_de_DE_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2025DIR_de_DE_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2025DIR_en_US" Name="en-US">
					<Directory Id="INSTALLXXX2025DIR_en_US_UI" Name="UI">
						<Directory Id="INSTALLXXX2025DIR_en_US_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2025DIR_en_US_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2025DIR_fr_FR" Name="fr-FR">
					<Directory Id="INSTALLXXX2025DIR_fr_FR_UI" Name="UI">
						<Directory Id="INSTALLXXX2025DIR_fr_FR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2025DIR_fr_FR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2025DIR_ja_JP" Name="ja-JP">
					<Directory Id="INSTALLXXX2025DIR_ja_JP_UI" Name="UI">
						<Directory Id="INSTALLXXX2025DIR_ja_JP_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2025DIR_ja_JP_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2025DIR_ko_KR" Name="ko-KR">
					<Directory Id="INSTALLXXX2025DIR_ko_KR_UI" Name="UI">
						<Directory Id="INSTALLXXX2025DIR_ko_KR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2025DIR_ko_KR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2025DIR_pt_BR" Name="pt-BR">
					<Directory Id="INSTALLXXX2025DIR_pt_BR_UI" Name="UI">
						<Directory Id="INSTALLXXX2025DIR_pt_BR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2025DIR_pt_BR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2025DIR_zh_CN" Name="zh-CN">
					<Directory Id="INSTALLXXX2025DIR_zh_CN_UI" Name="UI">
						<Directory Id="INSTALLXXX2025DIR_zh_CN_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2025DIR_zh_CN_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
			</Directory>
			<Directory Id="MAX2026DIR">
				<Directory Id="INSTALLYYY2026DIR_UI_ln" Name="UI_ln">
					<Directory Id="INSTALLYYY2026DIR_UI_ln_Resources" Name="Resources">
						<Directory Id="INSTALLYYY2026DIR_UI_ln_Resources_Dark" Name="Dark">
							<Directory Id="INSTALLYYY2026DIR_UI_ln_Resources_Dark_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2026DIR_UI_ln_Resources_Dark_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
						<Directory Id="INSTALLYYY2026DIR_UI_ln_Resources_Light" Name="Light">
							<Directory Id="INSTALLYYY2026DIR_UI_ln_Resources_Light_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2026DIR_UI_ln_Resources_Light_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2026DIR_de_DE" Name="de-DE">
					<Directory Id="INSTALLXXX2026DIR_de_DE_UI" Name="UI">
						<Directory Id="INSTALLXXX2026DIR_de_DE_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2026DIR_de_DE_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2026DIR_en_US" Name="en-US">
					<Directory Id="INSTALLXXX2026DIR_en_US_UI" Name="UI">
						<Directory Id="INSTALLXXX2026DIR_en_US_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2026DIR_en_US_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2026DIR_fr_FR" Name="fr-FR">
					<Directory Id="INSTALLXXX2026DIR_fr_FR_UI" Name="UI">
						<Directory Id="INSTALLXXX2026DIR_fr_FR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2026DIR_fr_FR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2026DIR_ja_JP" Name="ja-JP">
					<Directory Id="INSTALLXXX2026DIR_ja_JP_UI" Name="UI">
						<Directory Id="INSTALLXXX2026DIR_ja_JP_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2026DIR_ja_JP_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2026DIR_ko_KR" Name="ko-KR">
					<Directory Id="INSTALLXXX2026DIR_ko_KR_UI" Name="UI">
						<Directory Id="INSTALLXXX2026DIR_ko_KR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2026DIR_ko_KR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2026DIR_pt_BR" Name="pt-BR">
					<Directory Id="INSTALLXXX2026DIR_pt_BR_UI" Name="UI">
						<Directory Id="INSTALLXXX2026DIR_pt_BR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2026DIR_pt_BR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2026DIR_zh_CN" Name="zh-CN">
					<Directory Id="INSTALLXXX2026DIR_zh_CN_UI" Name="UI">
						<Directory Id="INSTALLXXX2026DIR_zh_CN_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2026DIR_zh_CN_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
			</Directory>
			<Directory Id="MAX2027DIR">
				<Directory Id="INSTALLYYY2027DIR_UI_ln" Name="UI_ln">
					<Directory Id="INSTALLYYY2027DIR_UI_ln_Resources" Name="Resources">
						<Directory Id="INSTALLYYY2027DIR_UI_ln_Resources_Dark" Name="Dark">
							<Directory Id="INSTALLYYY2027DIR_UI_ln_Resources_Dark_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2027DIR_UI_ln_Resources_Dark_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
						<Directory Id="INSTALLYYY2027DIR_UI_ln_Resources_Light" Name="Light">
							<Directory Id="INSTALLYYY2027DIR_UI_ln_Resources_Light_Icons" Name="Icons">
								<Directory Id="INSTALLYYY2027DIR_UI_ln_Resources_Light_Icons_Datasmith" Name="Datasmith">
								</Directory>
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2027DIR_de_DE" Name="de-DE">
					<Directory Id="INSTALLXXX2027DIR_de_DE_UI" Name="UI">
						<Directory Id="INSTALLXXX2027DIR_de_DE_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2027DIR_de_DE_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2027DIR_en_US" Name="en-US">
					<Directory Id="INSTALLXXX2027DIR_en_US_UI" Name="UI">
						<Directory Id="INSTALLXXX2027DIR_en_US_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2027DIR_en_US_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2027DIR_fr_FR" Name="fr-FR">
					<Directory Id="INSTALLXXX2027DIR_fr_FR_UI" Name="UI">
						<Directory Id="INSTALLXXX2027DIR_fr_FR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2027DIR_fr_FR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2027DIR_ja_JP" Name="ja-JP">
					<Directory Id="INSTALLXXX2027DIR_ja_JP_UI" Name="UI">
						<Directory Id="INSTALLXXX2027DIR_ja_JP_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2027DIR_ja_JP_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2027DIR_ko_KR" Name="ko-KR">
					<Directory Id="INSTALLXXX2027DIR_ko_KR_UI" Name="UI">
						<Directory Id="INSTALLXXX2027DIR_ko_KR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2027DIR_ko_KR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2027DIR_pt_BR" Name="pt-BR">
					<Directory Id="INSTALLXXX2027DIR_pt_BR_UI" Name="UI">
						<Directory Id="INSTALLXXX2027DIR_pt_BR_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2027DIR_pt_BR_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
				<Directory Id="INSTALLXXX2027DIR_zh_CN" Name="zh-CN">
					<Directory Id="INSTALLXXX2027DIR_zh_CN_UI" Name="UI">
						<Directory Id="INSTALLXXX2027DIR_zh_CN_UI_Ribbon" Name="Ribbon">
							<Directory Id="INSTALLXXX2027DIR_zh_CN_UI_Ribbon_Extensions" Name="Extensions">
							</Directory>
						</Directory>
					</Directory>
				</Directory>
			</Directory>

			<Directory Id="TempFolder" />
		</Directory>
	</Fragment>

	<Fragment>

		<!-- Declaration of a empty binaries folder so that the engine accept our remote location for the engine dir -->
		<Component Id="EMPTYENGINEBINARIESDIR" Directory="BINARIESFOLDER" Guid="{9EE7B75B-AD08-46F6-B149-0F061982985E}">
			<CreateFolder/>
		</Component>

		<Component Id="EXPORTERCOMMON" Directory="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax" Guid="{48435313-DE64-46EB-9264-D67353E57DFE}">
			<File Id="EXPORTERCOMMON_PackageContents.xml" Source="..\Plugin\PackageContents.xml"/>
		</Component>

		<Component Id="EXPORTERCOMMONUI" Directory="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_UI" Guid="{6146B792-3F5E-400B-8ECA-58E9662DFA00}">
			<File Id="EXPORTERCOMMONUI_datasmith_icons.rcc" Source="..\Extras\3dsMaxScripts\UI\datasmith_icons.rcc"/>
			<File Id="EXPORTERCOMMONUI_DatasmithDialog.ms" Source="..\Extras\3dsMaxScripts\UI\DatasmithPostStartUp.ms"/>
			<File Id="EXPORTERCOMMONUI_DatasmithMacros.mcr" Source="..\Extras\3dsMaxScripts\UI\DatasmithMacros.mcr"/>
			<File Id="EXPORTERCOMMONUI_pre_start_up.ms" Source="..\Extras\3dsMaxScripts\UI\DatasmithPreStartUp.ms"/>
			<File Id="EXPORTERCOMMONUI_UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" Source="..\Extras\3dsMaxScripts\UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms"/>
		</Component>
		
		<Component Id="EXPORTERCOMMONUI_EN_US" Directory="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_UI_en_Us" Guid="{22E83604-4064-468F-89F1-103517CDE9C7}">
			<File Id="EXPORTERCOMMONUI_EN_US_DatasmithMacros.mcr.res" Source="..\Extras\3dsMaxScripts\UI\en-Us\DatasmithMacros.mcr.res"/>
		</Component>

		<Component Id="EXPORTER2022" Directory="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2022" Guid="{2F3CB956-A564-4F66-85C3-74ED435B9FA9}">
			<RegistryKey Root="HKCU" Key="Software\Epic Games\DatasmithMaxPlugin\2022">
				<RegistryValue Name="Installed" Value="1" KeyPath="yes" Type="integer" />
			</RegistryKey>
			<File Id="EXPORTER2022_DatasmithMax2022.gup" Source="$(var.BinaryDir)\2022\DatasmithMax2022.gup"/>
			<File Id="EXPORTER2022_tbbmalloc.dll" Source="$(var.BinaryDir)\2022\tbbmalloc.dll"/>
		</Component>

		<Component Id="EXPORTER2022RIBBON_de_DE" Directory="INSTALLXXX2022DIR_de_DE_UI_Ribbon_Extensions" Guid="{7E4F3A7B-2D58-4DA7-A834-7E23A169DC75}">
			<File Id="EXPORTER2022RIBBON_de_DE_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\de-DE\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2022RIBBON_en_US" Directory="INSTALLXXX2022DIR_en_US_UI_Ribbon_Extensions" Guid="{CB534292-9CBE-4A55-B36D-64BA750A8B2E}">
			<File Id="EXPORTER2022RIBBON_en_US_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\en-US\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2022RIBBON_fr_FR" Directory="INSTALLXXX2022DIR_fr_FR_UI_Ribbon_Extensions" Guid="{8BBD62D2-7014-42C8-89E8-370FFDC6E4B3}">
			<File Id="EXPORTER2022RIBBON_fr_FR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\fr-FR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2022RIBBON_ja_JP" Directory="INSTALLXXX2022DIR_ja_JP_UI_Ribbon_Extensions" Guid="{A73543EF-75EE-4E0F-81E3-CD39774B9CD0}">
			<File Id="EXPORTER2022RIBBON_ja_JP_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ja-JP\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2022RIBBON_ko_KR" Directory="INSTALLXXX2022DIR_ko_KR_UI_Ribbon_Extensions" Guid="{99249CD7-F437-4203-AAAE-13FDC267BC3D}">
			<File Id="EXPORTER2022RIBBON_ko_KR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ko-KR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2022RIBBON_pt_BR" Directory="INSTALLXXX2022DIR_pt_BR_UI_Ribbon_Extensions" Guid="{C545FA77-DF66-46BE-9F06-560D7068AF52}">
			<File Id="EXPORTER2022RIBBON_pt_BR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\pt-BR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2022RIBBON_zh_CN" Directory="INSTALLXXX2022DIR_zh_CN_UI_Ribbon_Extensions" Guid="{179DA6CA-4256-4294-9E76-15024F4A77F7}">
			<File Id="EXPORTER2022RIBBON_zh_CN_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\zh-CN\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2022DARKICONS" Directory="INSTALLYYY2022DIR_UI_ln_Resources_Dark_Icons_Datasmith" Guid="{E6A15909-3170-4651-99B0-E43FF2083919}">
			<File Id="EXPORTER2022DARKICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2022DARKICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2022DARKICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2022DARKICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2022DARKICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2022DARKICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2022DARKICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2022DARKICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2022LIGHTICONS" Directory="INSTALLYYY2022DIR_UI_ln_Resources_Light_Icons_Datasmith" Guid="{F9BA2F7C-C808-4C8F-BF42-2D5EE4958A0C}">
			<File Id="EXPORTER2022LIGHTICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2022LIGHTICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2022LIGHTICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2022LIGHTICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2022LIGHTICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2022LIGHTICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2022LIGHTICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2022LIGHTICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2023" Directory="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2023" Guid="{73DDE815-C9A6-467F-A35A-83AC7C43A375}">
			<RegistryKey Root="HKCU" Key="Software\Epic Games\DatasmithMaxPlugin\2023">
				<RegistryValue Name="Installed" Value="1" KeyPath="yes" Type="integer" />
			</RegistryKey>
			<File Id="EXPORTER2023_DatasmithMax2023.gup" Source="$(var.BinaryDir)\2023\DatasmithMax2023.gup"/>
			<File Id="EXPORTER2023_tbbmalloc.dll" Source="$(var.BinaryDir)\2023\tbbmalloc.dll"/>
		</Component>

		<Component Id="EXPORTER2023RIBBON_de_DE" Directory="INSTALLXXX2023DIR_de_DE_UI_Ribbon_Extensions" Guid="{BCD3AF4B-8033-4678-BB0E-12E2CA0EA571}">
			<File Id="EXPORTER2023RIBBON_de_DE_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\de-DE\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2023RIBBON_en_US" Directory="INSTALLXXX2023DIR_en_US_UI_Ribbon_Extensions" Guid="{069DEC47-025A-48D8-970C-F49EAB66B5A5}">
			<File Id="EXPORTER2023RIBBON_en_US_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\en-US\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2023RIBBON_fr_FR" Directory="INSTALLXXX2023DIR_fr_FR_UI_Ribbon_Extensions" Guid="{09DC582E-1F8E-41D8-8A0B-7B21BBCBC225}">
			<File Id="EXPORTER2023RIBBON_fr_FR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\fr-FR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2023RIBBON_ja_JP" Directory="INSTALLXXX2023DIR_ja_JP_UI_Ribbon_Extensions" Guid="{A63BAA04-2780-415E-A407-99CC2958F93D}">
			<File Id="EXPORTER2023RIBBON_ja_JP_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ja-JP\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2023RIBBON_ko_KR" Directory="INSTALLXXX2023DIR_ko_KR_UI_Ribbon_Extensions" Guid="{E7912ECE-AA59-4A10-BC1C-68D6698B3A35}">
			<File Id="EXPORTER2023RIBBON_ko_KR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ko-KR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2023RIBBON_pt_BR" Directory="INSTALLXXX2023DIR_pt_BR_UI_Ribbon_Extensions" Guid="{EDA63F2C-053D-4DE5-8086-0B62D02BE795}">
			<File Id="EXPORTER2023RIBBON_pt_BR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\pt-BR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2023RIBBON_zh_CN" Directory="INSTALLXXX2023DIR_zh_CN_UI_Ribbon_Extensions" Guid="{9925B720-65DF-4526-8BD4-7C55F1B8ACA4}">
			<File Id="EXPORTER2023RIBBON_zh_CN_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\zh-CN\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2023DARKICONS" Directory="INSTALLYYY2023DIR_UI_ln_Resources_Dark_Icons_Datasmith" Guid="{A73D6A77-F86C-4742-A5D1-7CBC0873EA68}">
			<File Id="EXPORTER2023DARKICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2023DARKICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2023DARKICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2023DARKICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2023DARKICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2023DARKICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2023DARKICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2023DARKICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2023LIGHTICONS" Directory="INSTALLYYY2023DIR_UI_ln_Resources_Light_Icons_Datasmith" Guid="{238024F4-D311-473E-8FF9-D1EC4BCAE11C}">
			<File Id="EXPORTER2023LIGHTICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2023LIGHTICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2023LIGHTICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2023LIGHTICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2023LIGHTICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2023LIGHTICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2023LIGHTICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2023LIGHTICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2024" Directory="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2024" Guid="{9E341752-8943-4115-995F-6F3AEAA07271}">
			<RegistryKey Root="HKCU" Key="Software\Epic Games\DatasmithMaxPlugin\2024">
				<RegistryValue Name="Installed" Value="1" KeyPath="yes" Type="integer" />
			</RegistryKey>
			<File Id="EXPORTER2024_DatasmithMax2024.gup" Source="$(var.BinaryDir)\2024\DatasmithMax2024.gup"/>
			<File Id="EXPORTER2024_tbbmalloc.dll" Source="$(var.BinaryDir)\2024\tbbmalloc.dll"/>
		</Component>

		<Component Id="EXPORTER2024RIBBON_de_DE" Directory="INSTALLXXX2024DIR_de_DE_UI_Ribbon_Extensions" Guid="{6566FFAF-EFA6-40E9-8107-D15B3EE49BF6}">
			<File Id="EXPORTER2024RIBBON_de_DE_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\de-DE\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2024RIBBON_en_US" Directory="INSTALLXXX2024DIR_en_US_UI_Ribbon_Extensions" Guid="{DE0B86B0-1FC9-4042-B8AB-024FCE47C9C6}">
			<File Id="EXPORTER2024RIBBON_en_US_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\en-US\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2024RIBBON_fr_FR" Directory="INSTALLXXX2024DIR_fr_FR_UI_Ribbon_Extensions" Guid="{C48050BD-7E40-47AE-9180-BA3C1165642E}">
			<File Id="EXPORTER2024RIBBON_fr_FR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\fr-FR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2024RIBBON_ja_JP" Directory="INSTALLXXX2024DIR_ja_JP_UI_Ribbon_Extensions" Guid="{5191F008-B50B-4D0E-86AE-6A5E09317A26}">
			<File Id="EXPORTER2024RIBBON_ja_JP_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ja-JP\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2024RIBBON_ko_KR" Directory="INSTALLXXX2024DIR_ko_KR_UI_Ribbon_Extensions" Guid="{EC028D0A-2B97-465A-9A7E-ADB36DC74631}">
			<File Id="EXPORTER2024RIBBON_ko_KR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ko-KR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2024RIBBON_pt_BR" Directory="INSTALLXXX2024DIR_pt_BR_UI_Ribbon_Extensions" Guid="{F10170E2-16C5-4612-8C2C-E351B04E0E00}">
			<File Id="EXPORTER2024RIBBON_pt_BR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\pt-BR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2024RIBBON_zh_CN" Directory="INSTALLXXX2024DIR_zh_CN_UI_Ribbon_Extensions" Guid="{F9057F96-EE51-4BF6-AC28-C1C920F1E45D}">
			<File Id="EXPORTER2024RIBBON_zh_CN_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\zh-CN\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2024DARKICONS" Directory="INSTALLYYY2024DIR_UI_ln_Resources_Dark_Icons_Datasmith" Guid="{CF0BA097-ED48-48FA-8996-C130F34EAF3E}">
			<File Id="EXPORTER2024DARKICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2024DARKICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2024DARKICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2024DARKICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2024DARKICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2024DARKICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2024DARKICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2024DARKICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2024LIGHTICONS" Directory="INSTALLYYY2024DIR_UI_ln_Resources_Light_Icons_Datasmith" Guid="{9D5A7655-A3F4-4505-B999-A80DD06264E3}">
			<File Id="EXPORTER2024LIGHTICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2024LIGHTICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2024LIGHTICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2024LIGHTICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2024LIGHTICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2024LIGHTICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2024LIGHTICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2024LIGHTICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2025" Directory="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2025" Guid="{98A2623F-EC58-4ABC-9003-2C344BEE51D1}">
			<RegistryKey Root="HKCU" Key="Software\Epic Games\DatasmithMaxPlugin\2025">
				<RegistryValue Name="Installed" Value="1" KeyPath="yes" Type="integer" />
			</RegistryKey>
			<File Id="EXPORTER2025_DatasmithMax2025.gup" Source="$(var.BinaryDir)\2025\DatasmithMax2025.gup"/>
			<File Id="EXPORTER2025_tbbmalloc.dll" Source="$(var.BinaryDir)\2025\tbbmalloc.dll"/>
		</Component>

		<Component Id="EXPORTER2025RIBBON_de_DE" Directory="INSTALLXXX2025DIR_de_DE_UI_Ribbon_Extensions" Guid="{909369B6-3583-44D8-B575-BBEBED621FBB}">
			<File Id="EXPORTER2025RIBBON_de_DE_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\de-DE\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2025RIBBON_en_US" Directory="INSTALLXXX2025DIR_en_US_UI_Ribbon_Extensions" Guid="{A45B59C0-E59A-494D-A019-27849EEB8D85}">
			<File Id="EXPORTER2025RIBBON_en_US_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\en-US\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2025RIBBON_fr_FR" Directory="INSTALLXXX2025DIR_fr_FR_UI_Ribbon_Extensions" Guid="{34A31786-0E77-4467-9315-9179F50BA746}">
			<File Id="EXPORTER2025RIBBON_fr_FR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\fr-FR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2025RIBBON_ja_JP" Directory="INSTALLXXX2025DIR_ja_JP_UI_Ribbon_Extensions" Guid="{4C591720-8699-4F1D-A54D-58CCD1EDF64A}">
			<File Id="EXPORTER2025RIBBON_ja_JP_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ja-JP\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2025RIBBON_ko_KR" Directory="INSTALLXXX2025DIR_ko_KR_UI_Ribbon_Extensions" Guid="{6CE885F4-D658-441A-A5B4-7B97A06FF532}">
			<File Id="EXPORTER2025RIBBON_ko_KR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ko-KR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2025RIBBON_pt_BR" Directory="INSTALLXXX2025DIR_pt_BR_UI_Ribbon_Extensions" Guid="{ED1A775E-059D-4D99-B9B0-197B14A9D5BC}">
			<File Id="EXPORTER2025RIBBON_pt_BR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\pt-BR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2025RIBBON_zh_CN" Directory="INSTALLXXX2025DIR_zh_CN_UI_Ribbon_Extensions" Guid="{3023283C-9570-4688-9FE0-27A2554582AE}">
			<File Id="EXPORTER2025RIBBON_zh_CN_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\zh-CN\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2025DARKICONS" Directory="INSTALLYYY2025DIR_UI_ln_Resources_Dark_Icons_Datasmith" Guid="{E62727C4-C010-4EF6-B5BE-52361A5405A6}">
			<File Id="EXPORTER2025DARKICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2025DARKICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2025DARKICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2025DARKICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2025DARKICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2025DARKICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2025DARKICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2025DARKICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2025LIGHTICONS" Directory="INSTALLYYY2025DIR_UI_ln_Resources_Light_Icons_Datasmith" Guid="{98B95CB0-E862-45BC-B181-11B72665B659}">
			<File Id="EXPORTER2025LIGHTICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2025LIGHTICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2025LIGHTICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2025LIGHTICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2025LIGHTICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2025LIGHTICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2025LIGHTICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2025LIGHTICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2026" Directory="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2026" Guid="{55DC06BE-D69F-415D-86EB-C82255B07E1D}">
			<RegistryKey Root="HKCU" Key="Software\Epic Games\DatasmithMaxPlugin\2026">
				<RegistryValue Name="Installed" Value="1" KeyPath="yes" Type="integer" />
			</RegistryKey>
			<File Id="EXPORTER2026_DatasmithMax2026.gup" Source="$(var.BinaryDir)\2026\DatasmithMax2026.gup"/>
			<File Id="EXPORTER2026_tbbmalloc.dll" Source="$(var.BinaryDir)\2026\tbbmalloc.dll"/>
		</Component>

		<Component Id="EXPORTER2026RIBBON_de_DE" Directory="INSTALLXXX2026DIR_de_DE_UI_Ribbon_Extensions" Guid="{D0E9AD3B-D4AC-4A51-9B85-484F2D841C57}">
			<File Id="EXPORTER2026RIBBON_de_DE_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\de-DE\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2026RIBBON_en_US" Directory="INSTALLXXX2026DIR_en_US_UI_Ribbon_Extensions" Guid="{44E787EA-1B9A-45DB-81A9-6C0A42101230}">
			<File Id="EXPORTER2026RIBBON_en_US_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\en-US\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2026RIBBON_fr_FR" Directory="INSTALLXXX2026DIR_fr_FR_UI_Ribbon_Extensions" Guid="{8D07353A-64A2-44DC-9F81-2D8F6426DDCE}">
			<File Id="EXPORTER2026RIBBON_fr_FR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\fr-FR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2026RIBBON_ja_JP" Directory="INSTALLXXX2026DIR_ja_JP_UI_Ribbon_Extensions" Guid="{26AB4681-8AD1-4EBF-B4D8-2C94641F82EE}">
			<File Id="EXPORTER2026RIBBON_ja_JP_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ja-JP\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2026RIBBON_ko_KR" Directory="INSTALLXXX2026DIR_ko_KR_UI_Ribbon_Extensions" Guid="{09511B78-CBAE-4A0D-9A47-C0DFC1D27053}">
			<File Id="EXPORTER2026RIBBON_ko_KR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ko-KR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2026RIBBON_pt_BR" Directory="INSTALLXXX2026DIR_pt_BR_UI_Ribbon_Extensions" Guid="{CB0A713A-1805-43DB-B84C-3054719EB703}">
			<File Id="EXPORTER2026RIBBON_pt_BR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\pt-BR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2026RIBBON_zh_CN" Directory="INSTALLXXX2026DIR_zh_CN_UI_Ribbon_Extensions" Guid="{6B337118-173A-495D-8A36-27509E183672}">
			<File Id="EXPORTER2026RIBBON_zh_CN_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\zh-CN\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2026DARKICONS" Directory="INSTALLYYY2026DIR_UI_ln_Resources_Dark_Icons_Datasmith" Guid="{9B5887A5-F681-4B65-90B3-08C108CD4C56}">
			<File Id="EXPORTER2026DARKICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2026DARKICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2026DARKICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2026DARKICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2026DARKICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2026DARKICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2026DARKICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2026DARKICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2026LIGHTICONS" Directory="INSTALLYYY2026DIR_UI_ln_Resources_Light_Icons_Datasmith" Guid="{CA01E20B-0FF1-479F-9748-268BA447DB07}">
			<File Id="EXPORTER2026LIGHTICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2026LIGHTICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2026LIGHTICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2026LIGHTICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2026LIGHTICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2026LIGHTICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2026LIGHTICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2026LIGHTICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2027" Directory="AutodeskApplicationPluginsUserFolder_UnrealDatasmithMax_2027" Guid="{8C4C4FDF-726A-41CA-AAC4-75482B6C85A3}">
			<RegistryKey Root="HKCU" Key="Software\Epic Games\DatasmithMaxPlugin\2027">
				<RegistryValue Name="Installed" Value="1" KeyPath="yes" Type="integer" />
			</RegistryKey>
			<File Id="EXPORTER2027_DatasmithMax2027.gup" Source="$(var.BinaryDir)\2027\DatasmithMax2027.gup"/>
			<File Id="EXPORTER2027_tbbmalloc.dll" Source="$(var.BinaryDir)\2027\tbbmalloc.dll"/>
		</Component>

		<Component Id="EXPORTER2027RIBBON_de_DE" Directory="INSTALLXXX2027DIR_de_DE_UI_Ribbon_Extensions" Guid="{625FF510-9A17-436F-9195-F72A506EC942}">
			<File Id="EXPORTER2027RIBBON_de_DE_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\de-DE\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2027RIBBON_en_US" Directory="INSTALLXXX2027DIR_en_US_UI_Ribbon_Extensions" Guid="{BD689F1C-F5B2-46B2-BD37-D9BBF8BBC085}">
			<File Id="EXPORTER2027RIBBON_en_US_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\en-US\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2027RIBBON_fr_FR" Directory="INSTALLXXX2027DIR_fr_FR_UI_Ribbon_Extensions" Guid="{4DEB3DB8-DF21-488E-94A9-F177B744DE77}">
			<File Id="EXPORTER2027RIBBON_fr_FR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\fr-FR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2027RIBBON_ja_JP" Directory="INSTALLXXX2027DIR_ja_JP_UI_Ribbon_Extensions" Guid="{C4D66C4F-97E5-46D3-AAC9-A1623093ADEE}">
			<File Id="EXPORTER2027RIBBON_ja_JP_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ja-JP\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2027RIBBON_ko_KR" Directory="INSTALLXXX2027DIR_ko_KR_UI_Ribbon_Extensions" Guid="{9E6D1BFE-F147-4EEF-BF61-006CD51E665C}">
			<File Id="EXPORTER2027RIBBON_ko_KR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\ko-KR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2027RIBBON_pt_BR" Directory="INSTALLXXX2027DIR_pt_BR_UI_Ribbon_Extensions" Guid="{6F6A524C-3E83-4286-9EC4-E31C2B279C1F}">
			<File Id="EXPORTER2027RIBBON_pt_BR_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\pt-BR\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2027RIBBON_zh_CN" Directory="INSTALLXXX2027DIR_zh_CN_UI_Ribbon_Extensions" Guid="{A384DB79-7C25-4350-8C75-D0619D7C7768}">
			<File Id="EXPORTER2027RIBBON_zh_CN_DatasmithUI.xaml" Source="..\Extras\3dsMaxScripts\UI\zh-CN\DatasmithUI.xaml"/>
		</Component>

		<Component Id="EXPORTER2027DARKICONS" Directory="INSTALLYYY2027DIR_UI_ln_Resources_Dark_Icons_Datasmith" Guid="{6F282752-4C2F-4EB7-B7DB-2230DF13D1C9}">
			<File Id="EXPORTER2027DARKICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2027DARKICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2027DARKICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2027DARKICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2027DARKICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2027DARKICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2027DARKICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2027DARKICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

		<Component Id="EXPORTER2027LIGHTICONS" Directory="INSTALLYYY2027DIR_UI_ln_Resources_Light_Icons_Datasmith" Guid="{173B86F4-2ED7-4220-8505-38FF29CF3B19}">
			<File Id="EXPORTER2027LIGHTICONS_DatasmithAutoSyncIconOff32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOff32.png"/>
			<File Id="EXPORTER2027LIGHTICONS_DatasmithAutoSyncIconOn32.png" Source="..\Extras\Icons\DatasmithAutoSyncIconOn32.png"/>
			<File Id="EXPORTER2027LIGHTICONS_DatasmithIcon32.png" Source="..\Extras\Icons\DatasmithIcon32.png"/>
			<File Id="EXPORTER2027LIGHTICONS_DatasmithLogIcon32.png" Source="..\Extras\Icons\DatasmithLogIcon32.png"/>
			<File Id="EXPORTER2027LIGHTICONS_DatasmithManageConnectionsIcon32.png" Source="..\Extras\Icons\DatasmithManageConnectionsIcon32.png"/>
			<File Id="EXPORTER2027LIGHTICONS_DatasmithSettingsIcon32.png" Source="..\Extras\Icons\DatasmithSettingsIcon32.png"/>
			<File Id="EXPORTER2027LIGHTICONS_DatasmithSyncIcon32.png" Source="..\Extras\Icons\DatasmithSyncIcon32.png"/>
			<File Id="EXPORTER2027LIGHTICONS_DatasmithToolsIcon32.png" Source="..\Extras\Icons\DatasmithToolsIcon32.png"/>
		</Component>

	</Fragment>
</Wix>
