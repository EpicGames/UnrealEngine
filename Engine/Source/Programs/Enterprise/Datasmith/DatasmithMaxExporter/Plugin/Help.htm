<head>
	<meta http-equiv="refresh" content="1; URL=https://docs.unrealengine.com/en-US/Engine/Content/Importing/Datasmith/Overview/index.html" />
</head>
<body>
	<p>Redirecting to Unreal Datasmith help page, if you are not redirected <a href="https://docs.unrealengine.com/en-US/Engine/Content/Importing/Datasmith/Overview/index.html">click here</a></p>
</body>
