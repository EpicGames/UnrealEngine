<?xml version="1.0" encoding="utf-8"?>
<ApplicationPackage SchemaVersion="1.0" 
	ProductType="Application" 
	AutodeskProduct="3ds Max" 


	Name="Unreal Datasmith Exporter for Autodesk 3ds Max" 
	Description="Unreal Datasmith Exporter for Autodesk 3ds Max" 
	AppVersion="Unreal Engine 5.0" 
	Author="EpicGames, Inc."     
	ProductCode="{1FB940D3-A706-4F24-954A-E6DC3146BEE8}" 
	UpgradeCode="{BCBC53E8-6938-4C51-B60E-F857B1D4EEAF}" 
	OnlineDocumentation="https://www.unrealengine.com/en-US/datasmith/plugins" 
	HelpFile="./Help.htm" 
	>
	<!--  todo: Icon="./DatasmithIcon32.png" -->
    <CompanyDetails Name="EpicGames, Inc." Url="www.epicgames.com"/>

    <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2017" SeriesMax="2027" />

	<Components Description="plugins parts">
		<RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2027" SeriesMax="2027" />
		<ComponentEntry ModuleName="./2027/DatasmithMax2027.gup" />
	</Components>

	<Components Description="plugins parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2026" SeriesMax="2026" />
        <ComponentEntry ModuleName="./2026/DatasmithMax2026.gup" />
    </Components>

    <Components Description="plugins parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2025" SeriesMax="2025" />
        <ComponentEntry ModuleName="./2025/DatasmithMax2025.gup" />
    </Components>

    <Components Description="plugins parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2024" SeriesMax="2024" />
        <ComponentEntry ModuleName="./2024/DatasmithMax2024.gup" />
    </Components>

    <Components Description="plugins parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2023" SeriesMax="2023" />
        <ComponentEntry ModuleName="./2023/DatasmithMax2023.gup" />
    </Components>

    <Components Description="plugins parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2022" SeriesMax="2022" />
        <ComponentEntry ModuleName="./2022/DatasmithMax2022.gup" />
    </Components>

    <Components Description="plugins parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2021" SeriesMax="2021" />
        <ComponentEntry ModuleName="./2021/DatasmithMax2021.gup" />
    </Components>

    <Components Description="plugins parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2020" SeriesMax="2020" />
        <ComponentEntry ModuleName="./2020/DatasmithMax2020.gup" />
    </Components>

    <Components Description="plugins parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2019" SeriesMax="2019" />
        <ComponentEntry ModuleName="./2019/DatasmithMax2019.gup" />
    </Components>

    <Components Description="plugins parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2018" SeriesMax="2018" />
        <ComponentEntry ModuleName="./2018/DatasmithMax2018.gup" />
    </Components>

    <Components Description="plugins parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2017" SeriesMax="2017" />
        <ComponentEntry ModuleName="./2017/DatasmithMax2017.gup" />
    </Components>

    <Components Description="pre-start-up scripts parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2017" SeriesMax="2027" />
        <ComponentEntry ModuleName="./UI/DatasmithPreStartUp.ms" />
        <ComponentEntry ModuleName="./UI/UnrealStudio_Datasmith_Attributes_Holder_Modifier.ms" />
    </Components>

    <Components Description="macroscripts parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2017" SeriesMax="2027" />
        <ComponentEntry ModuleName="./UI/DatasmithMacros.mcr" />
    </Components>

    <Components Description="post-start-up scripts parts">
        <RuntimeRequirements OS="Win64" Platform="3ds Max" SeriesMin="2017" SeriesMax="2027" />
        <ComponentEntry ModuleName="./UI/DatasmithPostStartUp.ms" />
    </Components>

</ApplicationPackage>