// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxWriter.h"

#include "DatasmithSceneFactory.h"
#include "DatasmithMaxSceneHelper.h"

#include "Misc/Paths.h"

void FDatasmithMaxMatWriter::ExportPhysicalMaterialProperty(TSharedRef< IDatasmithScene > DatasmithScene, Texmap* Texture, bool bTextureEnabled, Texmap* TextureWeight, bool bTextureWeightEnabled, BMM_Color_fl Color, float Weight, TSharedPtr<IDatasmithCompositeTexture>& CompTex, FString TextureAliasName, FString ColorAliasName, bool bForceInvert, bool bIsGrayscale)
{
	// there is nothing to write
	if ((Texture == NULL || bTextureEnabled == false) && (Color.r <= 0 && Color.g <= 0 && Color.b <= 0))
	{
		return;
	}

	// if they are weighted we init the mix
	if (TextureWeight != NULL)
	{
		CompTex->SetMode(EDatasmithCompMode::Mix);
		TextureAliasName = FString(DATASMITH_TEXTURENAME);
		ColorAliasName = FString(DATASMITH_COLORNAME);

		TSharedPtr<IDatasmithCompositeTexture> MaskTextureMap = FDatasmithSceneFactory::CreateCompositeTexture();

		DumpTexture(DatasmithScene, MaskTextureMap, TextureWeight, DATASMITH_MASKNAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
		CompTex->AddMaskSurface(MaskTextureMap);
	}

	if (Texture != NULL && bTextureEnabled)
	{

		if (Weight == 1)
		{
			DumpTexture(DatasmithScene, CompTex, Texture, *TextureAliasName, *ColorAliasName, bForceInvert, bIsGrayscale);
		}
		else
		{
			DumpWeightedTexture(DatasmithScene, CompTex, Texture, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), Weight, *TextureAliasName, *ColorAliasName, bForceInvert, bIsGrayscale);
		}

	}
	else
	{
		if (Weight == 1)
		{
			CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
		}
		else
		{
			DumpWeightedColor(CompTex, Color, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), Weight, *ColorAliasName);
		}
	}

	// if they are weighted we finish the mix
	if (TextureWeight != NULL)
	{
		CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(BMM_Color_fl(0.0, 0.0, 0.0, 0.0)));
	}
}

void FDatasmithMaxMatWriter::ExportPhysicalMaterialCoat(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	const int NumParamBlocks = Material->NumParamBlocks();

	bool bCoatMapOn = true;
	bool bCoatColorMapOn = true;
	bool bCoatRoughnessMapOn = true;
	bool bCoatRoughnessInverted = true;

	float CoatAmount = 0.f;
	float CoatRoughness = 0.f;
	float CoatIOR = 1.5f;

	Texmap* CoatWeightMap = nullptr;
	Texmap* CoatColorMap = nullptr;
	Texmap* CoatRoughnessMap = nullptr;

	BMM_Color_fl CoatColor;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID(j);
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bCoatMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_color_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bCoatColorMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_rough_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bCoatRoughnessMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_roughness_inv")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bCoatRoughnessInverted = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Coating_Weight_Map")) == 0)
			{
				CoatWeightMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Coating_Color_Map")) == 0)
			{
				CoatColorMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Coating_Roughness_Map")) == 0)
			{
				CoatRoughnessMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coating")) == 0)
			{
				CoatAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_roughness")) == 0)
			{
				CoatRoughness = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_ior")) == 0)
			{
				CoatIOR = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_color")) == 0)
			{
				CoatColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}

		ParamBlock2->ReleaseDesc();
	}

	// There is no coating effect
	if (CoatAmount <= 0.0f || ((CoatColorMap == nullptr || bCoatColorMapOn == false) && CoatColor.r == 0.0f && CoatColor.g == 0.0f && CoatColor.b == 0))
	{
		return;
	}

	FString CoatShaderName = FString(Material->GetName().data()) + FString(_T("_coat"));
	TSharedPtr< IDatasmithShaderElement > MaterialShaderCoat = FDatasmithSceneFactory::CreateShader(*CoatShaderName);

	MaterialShaderCoat->SetBlendMode(EDatasmithBlendMode::ClearCoat);
	MaterialShaderCoat->SetIsStackedLayer(true);
	MaterialShaderCoat->SetIOR(CoatIOR);;

	ExportPhysicalMaterialProperty(DatasmithScene, CoatColorMap, bCoatColorMapOn, CoatWeightMap, bCoatMapOn, CoatColor, CoatAmount, MaterialShaderCoat->GetRefleComp(), DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME,false,false);

	if (CoatRoughnessMap != NULL)
	{
		ExportPhysicalMaterialProperty(DatasmithScene, CoatRoughnessMap, bCoatRoughnessMapOn, NULL, false, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), CoatRoughness, MaterialShaderCoat->GetRoughnessComp(), DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, bCoatRoughnessInverted, true);
	}
	else
	{
		MaterialShaderCoat->GetRoughnessComp()->AddParamVal1(IDatasmithCompositeTexture::ParamVal(CoatRoughness, TEXT("Roughness")));
	}

	MaterialElement->AddShader(MaterialShaderCoat);
}

void FDatasmithMaxMatWriter::ExportPhysicalMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	TSharedPtr< IDatasmithShaderElement > MaterialShader = FDatasmithSceneFactory::CreateShader((TCHAR*)Material->GetName().data());
	int NumParamBlocks = Material->NumParamBlocks();

	bool bDiffuseWeightMapOn = true, bDiffuseColorMapOn = true, bReflectivityMapOn = true, bReflectivityColorMapOn = true;
	bool bRoughnessMapOn = true, bMetalnessMapOn = true;
	bool bTransparencyMapOn = true, bTransparencyColorMapOn = true, bTransparencyRoughessMapOn = true;
	bool bEmittanceMapOn = true, bEmittanceColorMapOn = true;
	bool bRoughnessInverted = true;
	bool bThinWalled = true;
	bool bBumpmapOn = true, bCutoutMapOn = true;

	float DiffuseWeight = 0, Roughness = 0, Metalness = 0, Ior = 0;
	float Transparency = 0;
	float Reflectivity = 0;
	float EmittanceMultiplier = 0, EmittanceLuminance = 0, EmittanceTemperature = 0;
	float BumpMapAmount = 0, DisplacementMapAmount = 0;

	BMM_Color_fl DiffuseColor, TransparencyColor, EmittanceColor, ReflectionColor;

	Texmap* DiffuseWeightMap = NULL, *DiffuseColorMap = NULL;
	Texmap* ReflectivityMap = NULL, *RoughnessMap = NULL, *MetalnessMap = NULL, *ReflectionColorMap = NULL;
	Texmap* TransparencyMap = NULL, *TransparencyColorMap = NULL;
	Texmap* EmittanceMap = NULL, *EmittanceColorMap = NULL;
	Texmap* BumpMap = NULL, *CutoutMap = NULL;

	int MaterialMode = 0; //0 means simple, just metalness, 1 means specular + metalness



	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2 *ParamBlock2 = Material->GetParamBlockByID(j);
		// The the descriptor to 'decode'
		ParamBlockDesc2 *ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];


			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("material_mode")) == 0)
			{
				MaterialMode = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}

			// booleans
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("base_weight_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bDiffuseWeightMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("base_color_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bDiffuseColorMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectivity_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bReflectivityMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_color_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bReflectivityColorMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughness_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bRoughnessMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("metalness_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bMetalnessMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("transparency_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bTransparencyMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trans_color_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bTransparencyColorMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trans_rough_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bTransparencyRoughessMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emission_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bEmittanceMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_color_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bEmittanceColorMapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughness_inv")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bRoughnessInverted = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("thin_walled")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bThinWalled = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bBumpmapOn = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("cutout_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bCutoutMapOn = false;
				}
			}

			// float values
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("base_weight")) == 0)
			{
				DiffuseWeight = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughness")) == 0)
			{
				Roughness = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("metalness")) == 0)
			{
				Metalness = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trans_ior")) == 0)
			{
				Ior = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("transparency")) == 0)
			{
				Transparency = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectivity")) == 0)
			{
				Reflectivity = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emission")) == 0)
			{
				EmittanceMultiplier = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_luminance")) == 0)
			{
				EmittanceLuminance = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_kelvin")) == 0)
			{
				EmittanceTemperature = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map_amt")) == 0)
			{
				BumpMapAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("displacement_map_amt")) == 0)
			{
				DisplacementMapAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}

			// plain colors
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("base_color")) == 0)
			{
				DiffuseColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trans_color")) == 0)
			{
				TransparencyColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_color")) == 0)
			{
				EmittanceColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_color")) == 0)
			{
				ReflectionColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}

			// textures
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Base_Weight_Map")) == 0)
			{
				DiffuseWeightMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Base_Color_Map")) == 0)
			{
				DiffuseColorMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Reflectivity_Map")) == 0)
			{
				ReflectivityMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Roughness_Map")) == 0)
			{
				RoughnessMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Metalness_Map")) == 0)
			{
				MetalnessMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Refl_Color_Map")) == 0)
			{
				ReflectionColorMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Transparency_Map")) == 0)
			{
				TransparencyMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trans_color_map")) == 0)
			{
				TransparencyColorMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Emission_Map")) == 0)
			{
				EmittanceMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Emission_Color_Map")) == 0)
			{
				EmittanceColorMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map")) == 0)
			{
				BumpMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("cutout_map")) == 0)
			{
				CutoutMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}

		ParamBlock2->ReleaseDesc();
	}

	ExportPhysicalMaterialProperty(DatasmithScene, DiffuseColorMap, bDiffuseColorMapOn, DiffuseWeightMap, bDiffuseWeightMapOn, DiffuseColor, DiffuseWeight, MaterialShader->GetDiffuseComp(), DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
	ExportPhysicalMaterialProperty(DatasmithScene, TransparencyColorMap, bTransparencyColorMapOn, TransparencyMap, bTransparencyMapOn, TransparencyColor, Transparency, MaterialShader->GetTransComp(), DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, false, false);
	ExportPhysicalMaterialProperty(DatasmithScene, EmittanceColorMap, bEmittanceColorMapOn, EmittanceMap, bEmittanceMapOn, EmittanceColor, EmittanceMultiplier, MaterialShader->GetEmitComp(), DATASMITH_EMITTEXNAME, DATASMITH_EMITCOLNAME, false, false);
	ExportPhysicalMaterialProperty(DatasmithScene, CutoutMap, bCutoutMapOn, NULL, NULL, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), 1.0, MaterialShader->GetMaskComp(), DATASMITH_CLIPTEXNAME, DATASMITH_CLIPTEXNAME, false, true);

	if (MetalnessMap != NULL && bMetalnessMapOn)
	{
		ExportPhysicalMaterialProperty(DatasmithScene, MetalnessMap, bMetalnessMapOn, NULL, false, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), Metalness, MaterialShader->GetMetalComp(), DATASMITH_METALTEXNAME, DATASMITH_METALTEXNAME, false, true);
	}
	else
	{
		MaterialShader->SetMetal(Metalness);
	}

	if (MaterialMode == 1)
	{
		ExportPhysicalMaterialProperty(DatasmithScene, ReflectionColorMap, bReflectivityColorMapOn, ReflectivityMap, bReflectivityMapOn, ReflectionColor, Reflectivity, MaterialShader->GetRefleComp(), DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
	}

	if (RoughnessMap != NULL)
	{
		ExportPhysicalMaterialProperty(DatasmithScene, RoughnessMap, true, NULL, NULL, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), Roughness, MaterialShader->GetRoughnessComp(), DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, bRoughnessInverted, true);
	}
	else
	{
		if ((MetalnessMap != NULL && bMetalnessMapOn) || Metalness > 0 || (MaterialMode == 1 && ((ReflectionColorMap != NULL && bReflectivityColorMapOn) || Reflectivity > 0)))
		{
			if (bRoughnessInverted == false)
			{
				MaterialShader->GetRoughnessComp()->AddParamVal1(IDatasmithCompositeTexture::ParamVal(Roughness, TEXT("Roughness")));
			}
			else
			{
				MaterialShader->GetRoughnessComp()->AddParamVal1(IDatasmithCompositeTexture::ParamVal(1-Roughness, TEXT("Roughness")));
			}
		}
		else
		{
			MaterialShader->GetRoughnessComp()->AddParamVal1(IDatasmithCompositeTexture::ParamVal(0.75, TEXT("Roughness")));
		}
	}

	if (BumpMap != NULL && bBumpmapOn == true)
	{
		MaterialShader->SetBumpAmount(BumpMapAmount);

		if (FDatasmithMaxMatHelper::GetTextureClass(BumpMap) == EDSBitmapType::NormalMap)
		{
			ExportPhysicalMaterialProperty(DatasmithScene, BumpMap, bBumpmapOn, NULL, NULL, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), 1.0, MaterialShader->GetNormalComp(), DATASMITH_NORMALTEXNAME, DATASMITH_NORMALTEXNAME, false, false);
			ExportPhysicalMaterialProperty(DatasmithScene, BumpMap, bBumpmapOn, NULL, NULL, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), 1.0, MaterialShader->GetBumpComp(), DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
		}
		else
		{
			ExportPhysicalMaterialProperty(DatasmithScene, BumpMap, bBumpmapOn, NULL, NULL, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), 1.0, MaterialShader->GetBumpComp(), DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
		}
	}

	MaterialShader->SetIOR(Ior);
	if (bThinWalled)
	{
		MaterialShader->SetIORRefra(1.0);
	}

	MaterialElement->AddShader(MaterialShader);

	ExportPhysicalMaterialCoat(DatasmithScene, MaterialElement, Material);
}
