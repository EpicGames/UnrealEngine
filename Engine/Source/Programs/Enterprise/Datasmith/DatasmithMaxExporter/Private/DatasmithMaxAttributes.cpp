// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxAttributes.h"

#include "DatasmithMaxExporterDefines.h"
#include "DatasmithMaxClassIDs.h"

#define NOMINMAX

MAX_INCLUDES_START
	#include "inode.h"
	#include "iparamb2.h"
	#include "modstack.h"
MAX_INCLUDES_END

namespace DatasmithAttributesUtils
{
	Modifier* GetDatasmithAttributesModifer(INode* Node)
	{	
		if (Node == nullptr)
		{
			return nullptr;
		}

		Object* ObjectPtr = Node->GetObjectRef();

		while (ObjectPtr)
		{
			if (ObjectPtr && ObjectPtr->SuperClassID() == GEN_DERIVOB_CLASS_ID)
			{
				IDerivedObject* DerivedObj = (IDerivedObject*)ObjectPtr;

				const int ModifiersCount = DerivedObj->NumModifiers();
				for (int i = 0; i < ModifiersCount; i++)
				{
					Modifier* ModifierPtr = DerivedObj->GetModifier(i);

					// Don't use the modifier if it's not enabled
					if (ModifierPtr->ClassID() == DATASMITHUNREALATTRIBUTEMODIFIER_CLASS_ID && ModifierPtr->IsEnabled() != 0)
					{
						return ModifierPtr;
					}
				}

				// Goes down in the pipeline (used to search the source of a referenced object)
				Object* NewObjectPtr = DerivedObj->GetObjRef();

				if (ObjectPtr == NewObjectPtr)
				{
					ObjectPtr = nullptr;
				}
				else 
				{
					ObjectPtr = NewObjectPtr;
				}
			}
			else
			{
				break;
			}
		}

		return nullptr;
	}
}

TOptional<FDatasmithMaxStaticMeshAttributes> FDatasmithMaxStaticMeshAttributes::ExtractStaticMeshAttributes(INode* Node)
{
	Modifier* DatasmithAttributesModifier = DatasmithAttributesUtils::GetDatasmithAttributesModifer(Node);

	if (!DatasmithAttributesModifier)
	{
		return TOptional<FDatasmithMaxStaticMeshAttributes>();
	}

	int32 LightmapChannel = -2;
	INode* CustomCollisionNode = nullptr;
	EStaticMeshExportMode ExportMode = EStaticMeshExportMode::Default;
	
	const int NumParamBlocks = DatasmithAttributesModifier->NumParamBlocks();
	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = DatasmithAttributesModifier->GetParamBlockByID((short)j);
		if (ParamBlock2)
		{
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			if (ParamBlockDesc != nullptr)
			{
				for (int i = 0; i < ParamBlockDesc->count; i++)
				{
					const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];
					
					if (FCString::Stricmp(ParamDefinition.int_name, TEXT("param_int_LightmapChannel")) == 0)
					{
						Interval Unused;
						LightmapChannel = ParamBlock2->GetInt(ParamDefinition.ID, 0, Unused);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("param_node_CollisionMeshObject")) == 0)
					{
						Interval Unused;
						CustomCollisionNode = ParamBlock2->GetINode(ParamDefinition.ID, 0, Unused);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("param_int_StaticMeshExportMode")) == 0)
					{
						Interval Unused;

						switch (ParamBlock2->GetInt(ParamDefinition.ID, 0, Unused))
						{
						case 1:
							ExportMode = EStaticMeshExportMode::BoundingBox;
							break;
						default:
							ExportMode = EStaticMeshExportMode::Default;
							break;
						}
					}
				}
			}

			ParamBlock2->ReleaseDesc();
		}
	}
	
	return FDatasmithMaxStaticMeshAttributes(LightmapChannel, CustomCollisionNode, ExportMode);
}

FDatasmithMaxStaticMeshAttributes::FDatasmithMaxStaticMeshAttributes(int32 LightmapUVChannel, INode* CustomCollisionNode, EStaticMeshExportMode ExportMode)
	: LightmapUVChannel(LightmapUVChannel)
	, CustomCollisionNode(CustomCollisionNode)
	, ExportMode(ExportMode)
{
}
