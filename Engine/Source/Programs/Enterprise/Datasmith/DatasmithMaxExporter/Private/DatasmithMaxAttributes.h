// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class INode;

enum EStaticMeshExportMode : uint8
{
	/** Export the static mesh (Default) */
	Default,
	/** Export a simplified geometry for the mesh */
	BoundingBox,
};

/**
 * Attributes extracted from the datasmith attributes modifier for a static mesh
 */
class FDatasmithMaxStaticMeshAttributes
{
public:

	static TOptional<FDatasmithMaxStaticMeshAttributes> ExtractStaticMeshAttributes(INode* Node);

	FDatasmithMaxStaticMeshAttributes(int32 LightmapUVChannel, INode* CustomCollisionNode, EStaticMeshExportMode ExportMode);

	int32 GetLightmapUVChannel() const
	{
		return LightmapUVChannel;
	};

	INode* GetCustomCollisonNode() const
	{
		return CustomCollisionNode;
	}

	EStaticMeshExportMode GetExportMode() const
	{
		return ExportMode;
	}

private:
	const int32 LightmapUVChannel;
	INode* CustomCollisionNode;
	const EStaticMeshExportMode ExportMode;
};

