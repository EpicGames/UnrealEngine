// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithMaxCameraExporter.h"

#include "DatasmithMaxDirectLink.h"
#include "DatasmithMaxExporterDefines.h"
#include "DatasmithMaxHelper.h"
#include "DatasmithMaxSceneHelper.h"
#include "DatasmithMaxWriter.h"
#include "DatasmithMaxLogger.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneExporter.h"
#include "VRayLights.h"

#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "icustattribcontainer.h"
	#include "IFileResolutionManager.h"
	#include "iparamb2.h"
	#include "decomp.h"
	#include "lslights.h"
	#include "ilayer.h"
	#include "Scene/IPhysicalCamera.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

namespace DatasmithMaxCameraExporterImpl
{
	struct FMaxCameraParameters
	{
		bool bUseVignette = false;
		float VignettingAmount = 0.f;

		bool bTargeted = false;
		bool bUseDepthOfField = false;

		float TargetDistance = 0.f;

		float FilmWidth = 0.f;
		float FocalLength = 0.f;
	};

	struct FMaxWhiteBalanceParameters
	{
		int32 WhiteBalanceType = 0;
		int32 WhiteBalanceIlluminant = 6500;
		float WhiteBalanceKelvin = 6500.f;
		FLinearColor WhiteBalanceCustom = FLinearColor::White;
	};

	struct FMaxPhysicalCameraParameters
	{
		int32 ShutterUnitType = 0;
		float ShutterLengthSeconds = 0.f;
		float ShutterLengthFrames = 0.f;

		float ExposureValue = 6.f;
		float FNumber = 0.f;

		FMaxWhiteBalanceParameters WhiteBalance;
	};

	struct FVRayWhiteBalanceParameters
	{
		int32 Preset = 0;
		float Temperature = 6500.f;
		FLinearColor Custom = FLinearColor::White;
	};

	struct FVRayPhysicalCameraParameters
	{
		bool bSpecifyFocus = false;
		bool bUseExposure = false;

		float ISO = 0.f;
		float ShutterSpeed = 0.f;
		float FNumber = 0.f;

		float FocusDistance = 0.f;
		float TargetDistance = 0.f;

		FVRayWhiteBalanceParameters WhiteBalance;
	};

	FMaxCameraParameters ParseCameraParameters(Animatable& MaxObject)
	{
		const float CustomUnit = FMath::Abs( (float)GetSystemUnitScale(UNITS_CENTIMETERS) );
		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		FMaxCameraParameters CameraParameters;

		for (int j = 0; j < MaxObject.NumParamBlocks(); j++)
		{
			IParamBlock2* ParamBlock2 = MaxObject.GetParamBlockByID((short)j);
			if (ParamBlock2)
			{
				ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

				for (int i = 0; i < ParamBlockDesc->count; i++)
				{
					const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

					Interval ValidInterval = FOREVER;

					if (FCString::Stricmp(ParamDefinition.int_name, TEXT("vignetting_enabled")) == 0)
					{
						CameraParameters.bUseVignette = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime, ValidInterval) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("vignetting_amount")) == 0)
					{
						CameraParameters.VignettingAmount = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime, ValidInterval);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("target_distance")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("targetDistance")) == 0)
					{
						CameraParameters.TargetDistance = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime, ValidInterval) * CustomUnit;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("film_width_mm")) == 0)
					{
						CameraParameters.FilmWidth = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime, ValidInterval);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("focal_length_mm")) == 0)
					{
						CameraParameters.FocalLength = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime, ValidInterval);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("use_dof")) == 0)
					{
						CameraParameters.bUseDepthOfField = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime, ValidInterval) != 0 );
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("targeted")) == 0)
					{
						CameraParameters.bTargeted = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime, ValidInterval) != 0 );
					}
				}
				ParamBlock2->ReleaseDesc();
			}
		}

		return CameraParameters;
	}

	FMaxWhiteBalanceParameters ParseMaxWhiteBalanceParameters( Animatable& Object )
	{
		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		FMaxWhiteBalanceParameters WhiteBalanceParameters;

		for ( int j = 0; j < Object.NumParamBlocks(); ++j )
		{	
			if ( IParamBlock2* ParamBlock2 = Object.GetParamBlockByID( (short)j ) )
			{
				ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
				for ( int i = 0; i < ParamBlockDesc->count; ++i )
				{
					const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

					Interval ValidInterval = FOREVER;

					if (FCString::Stricmp(ParamDefinition.int_name, TEXT("white_balance_type")) == 0)
					{
						WhiteBalanceParameters.WhiteBalanceType = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime, ValidInterval);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("white_balance_illuminant")) == 0)
					{
						WhiteBalanceParameters.WhiteBalanceIlluminant = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime, ValidInterval);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("white_balance_kelvin")) == 0)
					{
						WhiteBalanceParameters.WhiteBalanceKelvin = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime, ValidInterval);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("white_balance_custom")) == 0)
					{
						BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, CurrentTime, ValidInterval);
						WhiteBalanceParameters.WhiteBalanceCustom = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( Color );
					}
				}
				ParamBlock2->ReleaseDesc();
			}
		}

		return WhiteBalanceParameters;
	}

	FMaxPhysicalCameraParameters ParsePhysicalCameraParameters(Animatable& MaxObject)
	{
		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		FMaxPhysicalCameraParameters PhysicalCameraParameters;
		PhysicalCameraParameters.WhiteBalance = ParseMaxWhiteBalanceParameters( MaxObject );

		for (int j = 0; j < MaxObject.NumParamBlocks(); j++)
		{
			IParamBlock2* ParamBlock2 = MaxObject.GetParamBlockByID((short)j);
			if (ParamBlock2)
			{
				ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

				for (int i = 0; i < ParamBlockDesc->count; i++)
				{
					const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

					Interval ValidInterval = FOREVER;

					if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("shutter_unit_type") ) == 0 )
					{
						PhysicalCameraParameters.ShutterUnitType = ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime, ValidInterval );
					}
					else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("shutter_length_seconds") ) == 0 )
					{
						PhysicalCameraParameters.ShutterLengthSeconds = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
					}
					else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("shutter_length_frames") ) == 0 )
					{
						PhysicalCameraParameters.ShutterLengthFrames = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
					}
					else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("exposure_value") ) == 0 )
					{
						PhysicalCameraParameters.ExposureValue = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
					}
					else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("f_number")) == 0 )
					{
						PhysicalCameraParameters.FNumber = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
					}
				}
				ParamBlock2->ReleaseDesc();
			}
		}

		return PhysicalCameraParameters;
	}

	FVRayWhiteBalanceParameters ParseVRayWhiteBalanceParameters( Animatable& Object )
	{
		DatasmithMaxCameraExporterImpl::FVRayWhiteBalanceParameters WhiteBalanceParameters;

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		for ( int j = 0; j < Object.NumParamBlocks(); ++j )
		{	
			if ( IParamBlock2* ParamBlock2 = Object.GetParamBlockByID( (short)j ) )
			{
				ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
				for ( int i = 0; i < ParamBlockDesc->count; ++i )
				{
					const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

					Interval ValidInterval = FOREVER;

					if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("temperature")) == 0 )
					{
						WhiteBalanceParameters.Temperature = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
					}
					else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("wb_preset")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("whitebalance_preset")) == 0 )
					{
						WhiteBalanceParameters.Preset = ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime, ValidInterval );
					}
					else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("whitebalance")) == 0 )
					{
						BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, CurrentTime, ValidInterval );
						WhiteBalanceParameters.Custom = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( Color );
					}
				}
				ParamBlock2->ReleaseDesc();
			}
		}

		return WhiteBalanceParameters;
	}

	FVRayPhysicalCameraParameters ParseVRayPhysicalCameraParameters(Animatable& Camera)
	{
		DatasmithMaxCameraExporterImpl::FVRayPhysicalCameraParameters CameraParameters;

		CameraParameters.WhiteBalance = ParseVRayWhiteBalanceParameters( Camera );

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		for ( int j = 0; j < Camera.NumParamBlocks(); ++j )
		{	
			if ( IParamBlock2* ParamBlock2 = Camera.GetParamBlockByID( (short)j ) )
			{
				ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
				for ( int i = 0; i < ParamBlockDesc->count; ++i )
				{
					const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

					Interval ValidInterval = FOREVER;

					if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("specify_focus") ) == 0 )
					{
						CameraParameters.bSpecifyFocus = ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime, ValidInterval ) != 0;
					}
					else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("exposure") ) == 0 )
					{
						CameraParameters.bUseExposure = ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime, ValidInterval ) != 0;
					}
					else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("focus_distance")) == 0 )
					{
						CameraParameters.FocusDistance = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS );
					}
					else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("target_distance") ) == 0 )
					{
						CameraParameters.TargetDistance = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS );
					}
					else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("ISO") ) == 0 )
					{
						CameraParameters.ISO = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
					}
					else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("shutter_speed") ) == 0 || FCString::Stricmp( ParamDefinition.int_name, TEXT("shutter") ) == 0 )
					{
						CameraParameters.ShutterSpeed = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("f_number")) == 0)
					{
						CameraParameters.FNumber = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
					}
				}
				ParamBlock2->ReleaseDesc();
			}
		}

		return CameraParameters;
	}

	void SetPostProcessWhiteBalance( const FMaxWhiteBalanceParameters& WhiteBalanceParameters, TSharedRef< IDatasmithPostProcessElement > PostProcess )
	{
												//daylight	Sunlight	Shade		Overcast	Incand.		Fluorescent
		const int TemperaturePresets[36] = {	6500,		5200,		7000,		6000,		3200,		4000,
												//CIE A		CIE D50		CIE D55		CIE D65		CIE D75		
												2856,		5003,		5503,		6504,		7504,
												//CIE F1	CIE F2		CIE F3		CIE F4		CIE F5		CIE F6		
												6430,		4230,		3450,		2940,		6350,		4150,
												//CIE F7	CIE F8		CIE F9		CIE F10		CIE F11		CIE F12		
												6500,		5000,		4150,		5000,		4000,		3000,
												//HALOGEN W	HALOGEN		HALOGEN C
												2800,		3200,		4000,		
												//CERAMIC W	HALOGEN	C
												3000,		4200,		
												//HALIDE W	HALIDE		HALIDE C
												3200,		4000,		6000,		
												//MERCURY	PHOSPHOR	XENON		H SODIUM	L SODIUM		
												3900,		4000,		6000,		2100,		1800
											};

		switch ( WhiteBalanceParameters.WhiteBalanceType )
		{
			case 0://temp tables
			{
				if ( WhiteBalanceParameters.WhiteBalanceIlluminant < UE_ARRAY_COUNT( TemperaturePresets ) )
				{
					int Value = TemperaturePresets[ WhiteBalanceParameters.WhiteBalanceIlluminant ];
					if (Value != 6500)
					{
						PostProcess->SetTemperature( (float)Value );
					}
				}
				break;
			}
			case 1://temperature
				PostProcess->SetTemperature( WhiteBalanceParameters.WhiteBalanceKelvin );
				break;
			case 2:
				PostProcess->SetColorFilter( WhiteBalanceParameters.WhiteBalanceCustom );
				break;
			default:
				break;
		}
	}

	void SetPostProcessWhiteBalance( const FVRayWhiteBalanceParameters& WhiteBalanceParameters, TSharedRef< IDatasmithPostProcessElement > PostProcess )
	{
										//	Custom		Neutral		Daylight	CIE D75		CIE D65		CIE D55		CIE D50		Temperature
		const int TemperaturePresets[8] = {	0000,		6500,		6500,		7504,		6504,		5503,		5003,		0000 };

		switch ( WhiteBalanceParameters.Preset )
		{
		case 0: // Custom
			PostProcess->SetColorFilter( WhiteBalanceParameters.Custom );
			break;
		case 7: // Temperature
			PostProcess->SetTemperature( WhiteBalanceParameters.Temperature );
			break;
		default:

			if ( WhiteBalanceParameters.Preset >= 0 && WhiteBalanceParameters.Preset < UE_ARRAY_COUNT( TemperaturePresets ) )
			{
				PostProcess->SetTemperature( TemperaturePresets[ WhiteBalanceParameters.Preset ] );
			}
			break;
		}
	}
}

bool FDatasmithMaxCameraExporter::ExportCamera(TimeValue CurrentTime, INode& Node, TSharedRef< IDatasmithCameraActorElement > Camera)
{
	ObjectState ObjState = Node.EvalWorldState(0);
	CameraObject* CamObj = (CameraObject*)ObjState.obj;
	if (CamObj == nullptr)
	{
		return false;
	}

	Camera->SetSensorWidth( GetCOREInterface()->GetRendApertureWidth() );
	Camera->SetSensorAspectRatio( (float)GetCOREInterface()->GetRendWidth() / (float)GetCOREInterface()->GetRendHeight() );
	Camera->SetFocalLength( GetCOREInterface()->GetRendApertureWidth() / (2.f * FMath::Tan(CamObj->GetFOV(CurrentTime) * 0.5f)) );

	int NumParamBlocks = CamObj->NumParamBlocks();
	float CustomUnit = FMath::Abs((float)GetSystemUnitScale(UNITS_CENTIMETERS));

	Class_ID ClassID = CamObj->ClassID();

	if ((ClassID == Class_ID(LOOKAT_CAM_CLASS_ID, 0)) || (ClassID == Class_ID(SIMPLE_CAM_CLASS_ID, 0)))
	{
		GenCamera* GenCam = (GenCamera*)CamObj;

		float FocusDistance = GenCam->GetTDist(CurrentTime) * DatasmithMaxDirectLink::FDatasmithConverter().UnitToCentimeter;
		Camera->SetFocusDistance( FocusDistance );
		Camera->SetEnableDepthOfField( GenCam->GetDOFEnable(CurrentTime) );
		// SensorWidth defaulted from GetRendApertureWidth above
	}
	else
	{
		DatasmithMaxCameraExporterImpl::FMaxCameraParameters MaxCameraParameters = DatasmithMaxCameraExporterImpl::ParseCameraParameters( *CamObj );
		Camera->SetFocusDistance( MaxCameraParameters.TargetDistance );
		Camera->SetSensorWidth( MaxCameraParameters.FilmWidth );
		Camera->SetFocalLength( MaxCameraParameters.FocalLength );
		Camera->SetEnableDepthOfField( MaxCameraParameters.bUseDepthOfField );

		if ( MaxCameraParameters.bUseVignette )
		{
			Camera->GetPostProcess()->SetVignette( MaxCameraParameters.VignettingAmount );
		}
	}

	// It's sufficient to check for the node's target to see if the camera has a look-at behavior
	INode* CameraTargetNode = Node.GetTarget();
	if ( CameraTargetNode )
	{
		Camera->SetLookAtActor( *FString::FromInt( CameraTargetNode->GetHandle() ) );
	}

	if ( ClassID == PHYSICALCAMERA_CLASS )
	{
		ExportPhysicalCamera( *CamObj, Camera );
	}
	else if ( ClassID == VRAY_PHYSICALCAMERA_CLASS )
	{
		ExportVRayPhysicalCamera( *CamObj, Camera );
	}

	return true;
}

bool FDatasmithMaxCameraExporter::ExportPhysicalCamera( CameraObject& Camera, TSharedRef< IDatasmithCameraActorElement > CameraElement )
{
	MaxSDK::IPhysicalCamera& PhysicalCamera = static_cast< MaxSDK::IPhysicalCamera& >( Camera );

	Interval ValidInterval = FOREVER;
	const float ISO = PhysicalCamera.GetEffectiveISO( GetCOREInterface()->GetTime(), ValidInterval );
	CameraElement->GetPostProcess()->SetCameraISO( ISO );

	DatasmithMaxCameraExporterImpl::FMaxPhysicalCameraParameters PhysicalCameraParameters = DatasmithMaxCameraExporterImpl::ParsePhysicalCameraParameters( Camera );

	DatasmithMaxCameraExporterImpl::SetPostProcessWhiteBalance( PhysicalCameraParameters.WhiteBalance, CameraElement->GetPostProcess().ToSharedRef() );

	CameraElement->SetFStop( PhysicalCameraParameters.FNumber );
	CameraElement->GetPostProcess()->SetDepthOfFieldFstop( PhysicalCameraParameters.FNumber );

	// Compute the shutter speed based on the EV, F-Stop and ISO
	if ( !FMath::IsNearlyZero( ISO ) )
	{
		float ShutterSpeedLog2 = PhysicalCameraParameters.ExposureValue - FMath::Log2( 100.f / ISO ) - FMath::Log2( FMath::Pow( CameraElement->GetFStop(), 2.f ) ) ;
		CameraElement->GetPostProcess()->SetCameraShutterSpeed( FMath::Pow( 2.f, ShutterSpeedLog2 ) );
	}

	return true;
}

bool FDatasmithMaxCameraExporter::ExportVRayPhysicalCamera( CameraObject& Camera, TSharedRef< IDatasmithCameraActorElement > CameraElement )
{
	DatasmithMaxCameraExporterImpl::FVRayPhysicalCameraParameters CameraParameters = DatasmithMaxCameraExporterImpl::ParseVRayPhysicalCameraParameters( Camera );

	DatasmithMaxCameraExporterImpl::SetPostProcessWhiteBalance( CameraParameters.WhiteBalance, CameraElement->GetPostProcess().ToSharedRef() );

	if ( CameraParameters.bSpecifyFocus )
	{
		CameraElement->SetFocusDistance( CameraParameters.FocusDistance );
	}
	else
	{
		CameraElement->SetFocusDistance( CameraParameters.TargetDistance );
	}

	if ( CameraParameters.bUseExposure )
	{
		CameraElement->GetPostProcess()->SetCameraISO( CameraParameters.ISO );
		CameraElement->GetPostProcess()->SetCameraShutterSpeed( CameraParameters.ShutterSpeed );

		CameraElement->SetFStop( CameraParameters.FNumber );
		CameraElement->GetPostProcess()->SetDepthOfFieldFstop( CameraParameters.FNumber );
	}

	return true;
}

bool FDatasmithMaxCameraExporter::ExportToneOperator( ToneOperator& ToneOp, TSharedRef< IDatasmithPostProcessVolumeElement > PostProcessVolumeElement )
{
	PostProcessVolumeElement->SetEnabled( ToneOp.Active( GetCOREInterface()->GetTime() ) != 0 );
	PostProcessVolumeElement->SetUnbound( true );

	const Class_ID ToneOpClassID = ToneOp.ClassID();

	if ( ToneOpClassID == PHYSICAL_CRTL_CLASSID )
	{
		return ExportPhysicalExposureControl( ToneOp, PostProcessVolumeElement );
	}
	else if ( ToneOpClassID == VRAY_CRTL_CLASSID )
	{
		return ExportVRayExposureControl( ToneOp, PostProcessVolumeElement );
	}

	return false;
}

bool FDatasmithMaxCameraExporter::ExportPhysicalExposureControl( ToneOperator& ToneOp, TSharedRef< IDatasmithPostProcessVolumeElement > PostProcessVolumeElement )
{
	DatasmithMaxCameraExporterImpl::FMaxWhiteBalanceParameters WhiteBalanceParameters = DatasmithMaxCameraExporterImpl::ParseMaxWhiteBalanceParameters( ToneOp );

	DatasmithMaxCameraExporterImpl::SetPostProcessWhiteBalance( WhiteBalanceParameters, PostProcessVolumeElement->GetSettings() );

	TOptional< float > GlobalEV;

	int NumParamBlocks = ToneOp.NumParamBlocks();
	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = ToneOp.GetParamBlockByID((short)j);
		
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			Interval ValidInterval = FOREVER;

			if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("global_ev")) == 0 )
			{
				GlobalEV = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime(), ValidInterval );
			}
		}

		ParamBlock2->ReleaseDesc();
	}

	// Compute the shutter speed based on the EV, F-Stop and ISO
	if ( GlobalEV )
	{
		const float ISO = 200.f;
		const float Fstop = 8.f;

		float ShutterSpeedLog2 = GlobalEV.GetValue() - FMath::Log2( 100.f / ISO ) - FMath::Log2( FMath::Pow( Fstop, 2.f ) ) ;
		PostProcessVolumeElement->GetSettings()->SetCameraShutterSpeed( FMath::Pow( 2.f, ShutterSpeedLog2 ) );

		PostProcessVolumeElement->GetSettings()->SetCameraISO( ISO );
		PostProcessVolumeElement->GetSettings()->SetDepthOfFieldFstop( Fstop );
	}

	return true;
}

bool FDatasmithMaxCameraExporter::ExportVRayExposureControl( ToneOperator& ToneOp, TSharedRef< IDatasmithPostProcessVolumeElement > PostProcessVolumeElement )
{
	enum class EVRayExposureMode : int
	{
		FromVRayCamera = 105,
		FromEVParameter = 106,
		Photographic = 107
	};

	int ExposureMode = (int)EVRayExposureMode::Photographic;
	TOptional< float > GlobalEV;
	TOptional< float > ISO;
	TOptional< float > Shutter;
	TOptional< float > FNumber;
	TOptional< INode* > CameraNode;

	const TimeValue CurrentTime = GetCOREInterface()->GetTime();

	const int NumParamBlocks = ToneOp.NumParamBlocks();
	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = ToneOp.GetParamBlockByID((short)j);
		
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			Interval ValidInterval = FOREVER;

			if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("mode")) == 0 )
			{
				ExposureMode = ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime, ValidInterval );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("camnode")) == 0 )
			{
				CameraNode = ParamBlock2->GetINode( ParamDefinition.ID, CurrentTime, ValidInterval );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("EV")) == 0 )
			{
				GlobalEV = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("iso")) == 0 )
			{
				ISO = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("shutter")) == 0 )
			{
				Shutter = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("f_number")) == 0 )
			{
				FNumber = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, ValidInterval );
			}
		}

		ParamBlock2->ReleaseDesc();
	}

	DatasmithMaxCameraExporterImpl::FVRayWhiteBalanceParameters WhiteBalanceParameters = DatasmithMaxCameraExporterImpl::ParseVRayWhiteBalanceParameters( ToneOp );

	switch ( ExposureMode )
	{
	case (int)EVRayExposureMode::FromVRayCamera:
		if ( CameraNode && CameraNode.GetValue() )
		{
			ObjectState ObjState = CameraNode.GetValue()->EvalWorldState(0);
			CameraObject* CamObj = (CameraObject*)ObjState.obj;
			if (CamObj)
			{
				DatasmithMaxCameraExporterImpl::FVRayPhysicalCameraParameters VRayCameraParameters = DatasmithMaxCameraExporterImpl::ParseVRayPhysicalCameraParameters( *CamObj );
				FNumber = VRayCameraParameters.FNumber;
				ISO = VRayCameraParameters.ISO;
				Shutter = VRayCameraParameters.ShutterSpeed;

				WhiteBalanceParameters = VRayCameraParameters.WhiteBalance;
			}
		}
		break;

	case (int)EVRayExposureMode::FromEVParameter:
	{
		if ( GlobalEV )
		{
			ISO = 200.f;
			FNumber = 8.f;

			float ShutterSpeedLog2 = GlobalEV.GetValue() - FMath::Log2( 100.f / ISO.GetValue() ) - FMath::Log2( FMath::Pow( FNumber.GetValue(), 2.f ) ) ;
			Shutter = FMath::Pow( 2.f, ShutterSpeedLog2 );
		}
		break;
	}
	case (int)EVRayExposureMode::Photographic:
	default:
		break;
	}

	if ( ISO )
	{
		PostProcessVolumeElement->GetSettings()->SetCameraISO( ISO.GetValue() );
	}

	if ( Shutter )
	{
		PostProcessVolumeElement->GetSettings()->SetCameraShutterSpeed( Shutter.GetValue() );
	}

	if ( FNumber )
	{
		PostProcessVolumeElement->GetSettings()->SetDepthOfFieldFstop( FNumber.GetValue() );
	}
	
	DatasmithMaxCameraExporterImpl::SetPostProcessWhiteBalance( WhiteBalanceParameters, PostProcessVolumeElement->GetSettings() );

	return true;
}
