// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef NEW_DIRECTLINK_PLUGIN

#include "DatasmithMaxConverters.h"

#include "DatasmithMaxDirectLink.h"

#include "DatasmithSceneFactory.h"

#include "DatasmithMaxSceneExporter.h"
#include "DatasmithMaxCameraExporter.h"
#include "DatasmithMaxSceneHelper.h"
#include "DatasmithMaxLogger.h"
#include "DatasmithMaxHelper.h"

#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"

namespace  DatasmithMaxDirectLink
{

void FHelperNodeConverter::Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
}

void FHelperNodeConverter::ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	// note: this is how baseline exporter derives names
	FString UniqueName = FString::FromInt(NodeTracker.Node->GetHandle());
	NodeTracker.CreateConverted().DatasmithActorElement = FDatasmithSceneFactory::CreateActor((const TCHAR*)*UniqueName);
	SceneTracker.SetupActor(NodeTracker);

	// todo: update validity from node
	NodeTracker.Validity.NarrowValidityToInterval(FOREVER);
}

void FHelperNodeConverter::RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
}

void FCameraNodeConverter::Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
}

void FCameraNodeConverter::ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	// note: this is how baseline exporter derives names
	FString UniqueName = FString::FromInt(NodeTracker.Node->GetHandle());
	NodeTracker.CreateConverted().DatasmithActorElement = FDatasmithSceneFactory::CreateCameraActor((const TCHAR*)*UniqueName);

	FDatasmithMaxCameraExporter::ExportCamera(SceneTracker.CurrentSyncPoint.Time, *NodeTracker.Node, StaticCastSharedPtr<IDatasmithCameraActorElement>(NodeTracker.GetConverted().DatasmithActorElement).ToSharedRef());

	SceneTracker.SetupActor(NodeTracker);

	// Max camera view direction is Z-, Unreal's X+
	// Max camera Up is Y+,  Unrela's Z+
	FQuat Rotation = NodeTracker.GetConverted().DatasmithActorElement->GetRotation();
	Rotation *= FQuat(0.0, 0.707107, 0.0, 0.707107);
	Rotation *= FQuat(0.707107, 0.0, 0.0, 0.707107);
	NodeTracker.GetConverted().DatasmithActorElement->SetRotation(Rotation);

	// todo: update validity from node
	NodeTracker.Validity.NarrowValidityToInterval(FOREVER);
}

void FCameraNodeConverter::RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
}

void FLightNodeConverter::Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
}

void FLightNodeConverter::ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	TSharedPtr<IDatasmithLightActorElement> LightElement;
	{
		// note: this is how baseline exporter derives names
		FString UniqueName = FString::FromInt(NodeTracker.Node->GetHandle());

		LightElement = FDatasmithMaxSceneExporter::CreateLightElementForNode(NodeTracker.Node, *UniqueName);

		if (!LightElement)
		{
			if (FDatasmithMaxSceneHelper::GetLightClass(NodeTracker.Node) == EMaxLightClass::SkyEquivalent)
			{
				SceneTracker.GetDatasmithSceneRef()->SetUsePhysicalSky(true);
			}
			else
			{
				LogWarning(TEXT("Unsupported light: ") + DatasmithMaxLogger::Get().GetLightDescription(NodeTracker.Node));
			}
			return;
		}
		else
		{
			if ( !FDatasmithMaxSceneExporter::ParseLight(*this, NodeTracker.Node, LightElement.ToSharedRef(), SceneTracker.GetDatasmithSceneRef()) )
			{
				return;
			}
		}

		NodeTracker.CreateConverted().DatasmithActorElement = LightElement;
	}
	SceneTracker.SetupActor(NodeTracker);

	//Cylinder shaped lights don't have the same default orientations, so we recalculate their transform and add the shape information.
	if (LightElement->IsA(EDatasmithElementType::AreaLight)
		&& StaticCastSharedPtr<IDatasmithAreaLightElement>(LightElement)->GetLightShape() == EDatasmithLightShape::Cylinder)
	{
		FVector Translation, Scale;
		FQuat Rotation;

		const float UnitMultiplier = (float)GetSystemUnitScale(UNITS_CENTIMETERS);
		const FMaxLightCoordinateConversionParams LightParams = FMaxLightCoordinateConversionParams(NodeTracker.Node, EDatasmithLightShape::Cylinder);

		Interval ValidityInterval;
		ValidityInterval.SetInfinite();

		if (NodeTracker.Node->GetWSMDerivedObject() != nullptr)
		{
			FDatasmithMaxSceneExporter::MaxToUnrealCoordinates(NodeTracker.Node->GetObjTMAfterWSM(SceneTracker.CurrentSyncPoint.Time, &ValidityInterval), Translation, Rotation, Scale, UnitMultiplier, LightParams);
		}
		else
		{
			FDatasmithMaxSceneExporter::MaxToUnrealCoordinates(NodeTracker.Node->GetObjectTM(SceneTracker.CurrentSyncPoint.Time, &ValidityInterval), Translation, Rotation, Scale, UnitMultiplier, LightParams);
		}

		Rotation.Normalize();
		LightElement->SetTranslation(Translation);
		LightElement->SetScale(Scale);
		LightElement->SetRotation(Rotation);

		// todo: update validity from node
		NodeTracker.Validity.NarrowValidityToInterval(ValidityInterval);
	}

	if (IsIesProfileValid())
	{
		if (const TCHAR* TexturePath = SceneTracker.AcquireIesTexture(IesFilePath))
		{
			LightElement->SetIesTexturePathName(TexturePath);
		}
		else
		{
			LightElement->SetUseIes(false);  // Force disable Ies if ies profile not found
		}
	}
}

void FLightNodeConverter::RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	SceneTracker.ReleaseIesTexture(IesFilePath);
}

void FLightNodeConverter::ApplyIesProfile(const TCHAR* InIesFilePath)
{
	IesFilePath = InIesFilePath;
}

const TCHAR* FLightNodeConverter::GetIesProfile()
{
	return *IesFilePath;
}

bool FLightNodeConverter::IsIesProfileValid()
{
	return FPlatformFileManager::Get().GetPlatformFile().FileExists(*IesFilePath);
}

void FMeshNodeConverter::Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	ObjectState ObjState = NodeTracker.Node->EvalWorldState(SceneTracker.CurrentSyncPoint.Time);
	Object* Obj = ObjState.obj;

	// AnimHandle is unique and never reused for new objects
	InstanceHandle = Animatable::GetHandleByAnim(Obj); // Record instance handle into Converter to identify it later

	SceneTracker.AddGeometryNodeInstance(NodeTracker, *this, Obj);
}

void FMeshNodeConverter::RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	FMeshNodeConverter& Converter = static_cast<FMeshNodeConverter&>(NodeTracker.GetConverter());

	// todo: registerNodeForMaterial is done in UpdateGeometryNode(separate from finding instances for Node)
	SceneTracker.UnregisterNodeForMaterial(NodeTracker);
	SceneTracker.RemoveGeometryNodeInstance(NodeTracker);
}

void FMeshNodeConverter::ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	SceneTracker.ConvertGeometryNodeToDatasmith(NodeTracker, *this);

	// Mark node as updated as soon as it is - in order for next nodes to be able to use its DatasmithActor
	// todo: update validity from node
	NodeTracker.Validity.NarrowValidityToInterval(FOREVER);
}

void FHismNodeConverter::RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	// remove static meshes used by the RailClone
	SceneTracker.UnregisterNodeForMaterial(NodeTracker);
	
	for (FMeshConverted& Mesh : Meshes)
	{
		SceneTracker.ReleaseMeshElement(Mesh);
	}
}

void FRailCloneNodeConverter::Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
}

void FRailCloneNodeConverter::ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	GeomUtils::ConvertRailClone(SceneTracker, NodeTracker);
}

void FForestNodeConverter::Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
}

void FForestNodeConverter::ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	GeomUtils::ConvertForest(SceneTracker, NodeTracker);
}
	
}

#endif // NEW_DIRECTLINK_PLUGIN
