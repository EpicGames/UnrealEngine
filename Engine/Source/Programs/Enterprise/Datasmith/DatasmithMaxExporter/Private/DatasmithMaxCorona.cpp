// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxWriter.h"

#include "DatasmithSceneFactory.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithMaxSceneHelper.h"
#include "DatasmithMaxSceneExporter.h"

#include "DatasmithMaterialsUtils.h"

#include "Misc/Paths.h"

float GetCoronaTexmapGamma(BitmapTex* InBitmapTex)
{
	int NumParamBlocks = InBitmapTex->NumParamBlocks();

	float Gamma = -1;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InBitmapTex->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("gamma")) == 0)
			{
				Gamma = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				break;
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	return Gamma;
}

FString FDatasmithMaxMatWriter::DumpBitmapCorona(TSharedPtr<IDatasmithCompositeTexture>& CompTex, BitmapTex* InBitmapTex, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale)
{
	DatasmithMaxTexmapParser::FCoronaBitmapParameters CoronaBitmapParameters = DatasmithMaxTexmapParser::ParseCoronaBitmap( InBitmapTex );

	float Gamma = GetCoronaTexmapGamma(InBitmapTex);
	FString OrigBase = FPaths::GetBaseFilename(CoronaBitmapParameters.Path) + FString("_") + FString::SanitizeFloat(Gamma).Replace(TEXT("."), TEXT("_"));
	FString Base = OrigBase + TextureSuffix;

	float IntegerPart;
	float UOffset = (CoronaBitmapParameters.OffsetU * CoronaBitmapParameters.TileU);
	UOffset += (-0.5f + 0.5f * CoronaBitmapParameters.TileU);
	UOffset = 1.0f - UOffset;
	UOffset = FMath::Modf(UOffset, &IntegerPart);

	float VOffset = (CoronaBitmapParameters.OffsetV * CoronaBitmapParameters.TileV);
	VOffset += (0.5f - 0.5f * CoronaBitmapParameters.TileV);
	VOffset = FMath::Modf(VOffset, &IntegerPart);

	float Multiplier = 1.0;
	int Slot = 0;
	FDatasmithTextureSampler TextureSampler(CoronaBitmapParameters.UVCoordinate, CoronaBitmapParameters.TileU, CoronaBitmapParameters.TileV, 
		UOffset, VOffset, -(CoronaBitmapParameters.RotW) / (2.f * PI), Multiplier, bForceInvert, Slot, false,
		CoronaBitmapParameters.MirrorU, CoronaBitmapParameters.MirrorV);

	CompTex->AddSurface(*OrigBase, TextureSampler);

	return *Base;
}

void FDatasmithMaxMatWriter::ExportCoronaMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	TSharedPtr< IDatasmithShaderElement > MaterialShader = FDatasmithSceneFactory::CreateShader((TCHAR*)Material->GetName().data());
	int NumParamBlocks = Material->NumParamBlocks();

	bool bDiffuseTexEnable = true;
	bool bReflectanceTexEnable = true;
	bool bRefractTexEnable = true;
	bool bEmitTexEnable = true;
	bool bGlossyTexEnable = true;
	bool bBumpTexEnable = true;
	bool bOpacityTexEnable = true;

	float DiffuseTexAmount = 0.f;
	float ReflectanceTexAmount = 0.f;
	float RefractTexAmount = 0.f;
	float GlossyTexAmount = 0.f;

	float BumpAmount = 0.f;

	float LevelDiffuse = 1.f;
	float LevelReflect = 1.f;
	float LevelRefract = 0.f;
	float LevelGlossy = 1.f;
	float LevelOpacity = 1.f;

	bool bThinRefraction = false;

	BMM_Color_fl ColorDiffuse;
	BMM_Color_fl ColorReflection;
	BMM_Color_fl ColorRefraction;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapDiffuse")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bDiffuseTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapReflect")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bReflectanceTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapRefract")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bRefractTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOpacity")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bOpacityTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapSelfIllum")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bEmitTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapReflectGlossiness")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bGlossyTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapBump")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bBumpTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapamountBump")) == 0)
			{
				BumpAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnDiffuse")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bDiffuseTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapAmountDiffuse")) == 0)
			{
				DiffuseTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnReflect")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bReflectanceTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapamountReflect")) == 0)
			{
				ReflectanceTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnRefract")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bRefractTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapamountRefract")) == 0)
			{
				RefractTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnReflectGlossiness")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bGlossyTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapamountReflectGlossiness")) == 0)
			{
				GlossyTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnBump")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bBumpTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnOpacity")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bOpacityTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnSelfIllum")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bEmitTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("LevelDiffuse")) == 0)
			{
				LevelDiffuse = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("ColorDiffuse")) == 0)
			{
				ColorDiffuse = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("colorReflect")) == 0)
			{
				ColorReflection = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("colorRefract")) == 0)
			{
				ColorRefraction = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("LevelReflect")) == 0)
			{
				LevelReflect = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("LevelRefract")) == 0)
			{
				LevelRefract = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("LevelOpacity")) == 0)
			{
				LevelOpacity = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectGlossiness")) == 0)
			{
				LevelGlossy = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("thin")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 1)
				{
					bThinRefraction = true;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("ior")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("refraction_ior")) == 0) //Name has changed between versions
			{
				MaterialShader->SetIORRefra( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	// corona matconverter creates low refraction values and ior = 1.0
	// in plants and other things, it is preferred to disable refraction totally
	bool bDisableRefraction = false;
	if (MaterialShader->GetIORRefra() == 1.00f && bThinRefraction == false)
	{
		bDisableRefraction = true;
	}

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			//Name has changed between versions
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("ior")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("refraction_ior")) == 0)
			{
				if (bThinRefraction == false)
				{
					MaterialShader->SetIORRefra( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
				}
				else
				{
					MaterialShader->SetIORRefra( 1.02f );
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("fresnelIor")) == 0)
			{
				MaterialShader->SetIOR( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapDiffuse")) == 0 && bDiffuseTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (DiffuseTexAmount == 1 && LevelDiffuse == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetDiffuseComp(), LocalTex, DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
				}
				else
				{
					if (LevelDiffuse != 1)
					{
						DumpWeightedTexture(DatasmithScene, MaterialShader->GetDiffuseComp(), LocalTex, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), LevelDiffuse, DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
					}
					else
					{
						DumpWeightedTexture(DatasmithScene, MaterialShader->GetDiffuseComp(), LocalTex, ColorDiffuse, DiffuseTexAmount, DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
					}
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("ColorDiffuse")) == 0 && bDiffuseTexEnable == false)
			{
				if (ColorDiffuse.r > 0 || ColorDiffuse.g > 0 || ColorDiffuse.b > 0)
				{
					if (LevelDiffuse == 1.0)
					{
						MaterialShader->GetDiffuseComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(ColorDiffuse));
					}
					else
					{
						DumpWeightedColor(MaterialShader->GetDiffuseComp(), ColorDiffuse, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), LevelDiffuse, DATASMITH_DIFFUSETEXNAME);
					}
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapReflect")) == 0 && bReflectanceTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());

				if (ReflectanceTexAmount == 1 && LevelReflect == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetRefleComp(), LocalTex, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
				}
				else
				{
					if (LevelReflect != 1)
					{
						DumpWeightedTexture(DatasmithScene, MaterialShader->GetRefleComp(), LocalTex, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), LevelReflect, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
					}
					else
					{
						DumpWeightedTexture(DatasmithScene, MaterialShader->GetRefleComp(), LocalTex, ColorReflection, ReflectanceTexAmount, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
					}
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("colorReflect")) == 0 && bReflectanceTexEnable == false)
			{
				if (ColorReflection.r > 0 || ColorReflection.g > 0 || ColorReflection.b > 0)
				{
					if (LevelReflect == 1.0)
					{
						MaterialShader->GetRefleComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(ColorReflection));
					}
					else
					{
						DumpWeightedColor(MaterialShader->GetRefleComp(), ColorReflection, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), LevelReflect, DATASMITH_REFLETEXNAME);
					}
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapReflectGlossiness")) == 0 && bGlossyTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());

				if (LevelGlossy == 1 && GlossyTexAmount == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, true, true);
				}
				else
				{
					DumpWeightedTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, BMM_Color_fl(1.0f - LevelGlossy, 1.0f - LevelGlossy, 1.0f - LevelGlossy, 1.0f - LevelGlossy), GlossyTexAmount,
						DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, true, true);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectGlossiness")) == 0 && bGlossyTexEnable == false)
			{
				if (bReflectanceTexEnable == false && ColorReflection.r == 0 && ColorReflection.g == 0 && ColorReflection.b == 0)
				{
					MaterialShader->GetRoughnessComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(0.75f, TEXT("roughness")) );
				}
				else
				{
					MaterialShader->GetRoughnessComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(1.0f - LevelGlossy, TEXT("roughness")) );
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapRefract")) == 0 && bRefractTexEnable == true && LevelRefract > 0 && bDisableRefraction == false)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());

				if (RefractTexAmount == 1 && LevelRefract == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetTransComp(), LocalTex, DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, false, false);
				}
				else
				{
					if (LevelRefract != 1)
					{
						DumpWeightedTexture(DatasmithScene, MaterialShader->GetTransComp(), LocalTex, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), LevelRefract, DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, false, false);
					}
					else
					{
						DumpWeightedTexture(DatasmithScene, MaterialShader->GetTransComp(), LocalTex, ColorRefraction, RefractTexAmount, DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, false, false);
					}
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("colorRefract")) == 0 && bRefractTexEnable == false && LevelRefract > 0 && bDisableRefraction == false)
			{
				if (ColorRefraction.r > 0 || ColorRefraction.g > 0 || ColorRefraction.b > 0)
				{
					if (LevelRefract == 1.0)
					{
						MaterialShader->GetTransComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(ColorRefraction));
					}
					else
					{
						DumpWeightedColor(MaterialShader->GetTransComp(), ColorRefraction, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), LevelRefract, DATASMITH_TRANSPTEXNAME);
					}
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOpacity")) == 0 && bOpacityTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				DumpTexture(DatasmithScene, MaterialShader->GetMaskComp(), LocalTex, DATASMITH_CLIPTEXNAME, DATASMITH_CLIPTEXNAME, false, true);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapSelfIllum")) == 0 && bOpacityTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				DumpTexture(DatasmithScene, MaterialShader->GetEmitComp(), LocalTex, DATASMITH_EMITTEXNAME, DATASMITH_EMITTEXNAME, false, false);
				MaterialShader->SetEmitPower(100.0);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapBump")) == 0)
			{
				if (bBumpTexEnable == true && BumpAmount > 0)
				{
					MaterialShader->SetBumpAmount(BumpAmount / 100.0);
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					if (FDatasmithMaxMatHelper::GetTextureClass(LocalTex) == EDSBitmapType::NormalMap)
					{
						DumpTexture(DatasmithScene, MaterialShader->GetNormalComp(), LocalTex, DATASMITH_NORMALTEXNAME, DATASMITH_NORMALTEXNAME, false, false);
						DumpTexture(DatasmithScene, MaterialShader->GetBumpComp(), LocalTex, DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
					}
					else
					{
						DumpTexture(DatasmithScene, MaterialShader->GetBumpComp(), LocalTex, DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
					}
				}
			}
		}
		ParamBlock2->ReleaseDesc();
	}
	MaterialElement->AddShader(MaterialShader);
}

void FDatasmithMaxMatWriter::ExportCoronaLightMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	TSharedPtr< IDatasmithShaderElement > MaterialShader = FDatasmithSceneFactory::CreateShader((TCHAR*)Material->GetName().data());

	int NumParamBlocks = Material->NumParamBlocks();

	bool bTexEnabled = true;
	bool bClipTexEnabled = true;
	Texmap* EmitTexture = NULL;
	Texmap* ClipTexture = NULL;
	BMM_Color_fl EmitColor;
	float Multiplier = 1.0f;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap")) == 0)
			{
				EmitTexture = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacityTexmap")) == 0)
			{
				ClipTexture = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOn")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bTexEnabled = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacityTexmapOn")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bClipTexEnabled = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("color")) == 0)
			{
				EmitColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("twosidedEmission")) == 0)
			{
				// int twoSided = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("intensity")) == 0)
			{
				Multiplier = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	if (bTexEnabled && EmitTexture != NULL)
	{
		DumpTexture(DatasmithScene, MaterialShader->GetEmitComp(), EmitTexture, DATASMITH_EMITTEXNAME, DATASMITH_EMITCOLNAME, false, false);
	}
	else
	{
		MaterialShader->GetEmitComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(EmitColor));
	}

	if (bClipTexEnabled && ClipTexture != NULL)
	{
		DumpTexture(DatasmithScene, MaterialShader->GetMaskComp(), ClipTexture, DATASMITH_CLIPTEXNAME, DATASMITH_CLIPTEXNAME, false, true);
	}

	MaterialShader->SetEmitPower(2.0 * Multiplier);
	MaterialShader->SetLightOnly(true);
	MaterialShader->SetUseEmissiveForDynamicAreaLighting(true);

	MaterialElement->AddShader(MaterialShader);
}

void FDatasmithMaxMatWriter::ExportCoronaBlendMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	FDatasmithMaxMatExport::WriteXMLMaterial(DatasmithScene, MaterialElement, Material->GetSubMtl(0));

	int NumParamBlocks = Material->NumParamBlocks();

	Mtl* CoatMaterials[10];
	Texmap* Mask[10];
	float MixAmount[10];

	for (int i = 0; i < 10; i++)
	{
		CoatMaterials[i] = NULL;
		Mask[i] = NULL;
		MixAmount[i] = 0.5f;
	}

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Layers")) == 0)
			{
				for (int s = 0; s < 10; s++)
				{
					CoatMaterials[s] = ParamBlock2->GetMtl(ParamDefinition.ID, GetCOREInterface()->GetTime(), s);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mixmaps")) == 0)
			{
				for (int s = 0; s < 10; s++)
				{
					Mask[s] = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), s);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("amounts")) == 0)
			{
				for (int s = 0; s < 10; s++)
				{
					MixAmount[s] = (float)ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime(), s);
				}
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	for (int s = 0; s < 10; s++)
	{
		if (CoatMaterials[s])
		{
			FDatasmithMaxMatExport::WriteXMLMaterial(DatasmithScene, MaterialElement, CoatMaterials[s]);

			TSharedPtr< IDatasmithShaderElement >& Shader = MaterialElement->GetShader( MaterialElement->GetShadersCount() - 1 );
			Shader->SetBlendMode(EDatasmithBlendMode::Alpha);
			Shader->SetIsStackedLayer(true);

			if (Mask[s])
			{
				DumpTexture(DatasmithScene, Shader->GetWeightComp(), Mask[s], DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
			}
			else
			{
				Shader->GetWeightComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(MixAmount[s], TEXT("MixAmount")) );
			}
		}
	}
}

FString FDatasmithMaxMatWriter::DumpCoronaColor(TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert)
{
	FString ActualPrefix = FString(Prefix) + FString(TEXT("comp"));

	DatasmithMaxTexmapParser::FCoronaColorParameters ColorParameters = DatasmithMaxTexmapParser::ParseCoronaColor( InTexmap );

	FLinearColor CoronaColor;

	switch ( ColorParameters.Method )
	{
	case 1:
		CoronaColor.R = ColorParameters.ColorHdr.X;
		CoronaColor.G = ColorParameters.ColorHdr.Y;
		CoronaColor.B = ColorParameters.ColorHdr.Z;
		break;
	case 2:
		CoronaColor = DatasmithMaterialsUtils::TemperatureToColor( ColorParameters.Temperature );
		ColorParameters.bInputIsLinear = true;
		break;
	case 3:
		if ( ColorParameters.HexColor.Len() == 7 )
		{
			FString Red = TEXT("0x") + ColorParameters.HexColor.Mid(1, 2);
			FString Green = TEXT("0x") + ColorParameters.HexColor.Mid(3, 2);
			FString Blue = TEXT("0x") + ColorParameters.HexColor.Mid(5, 2);

			CoronaColor.R = FCString::Strtoi(*Red, nullptr, 16) / 255.0f;
			CoronaColor.G = FCString::Strtoi(*Green, nullptr, 16) / 255.0f;
			CoronaColor.B = FCString::Strtoi(*Blue, nullptr, 16) / 255.0f;
			break;
		}
	default:
		CoronaColor = ColorParameters.RgbColor;
		break;
	}

	CoronaColor.R *= ColorParameters.Multiplier;
	CoronaColor.G *= ColorParameters.Multiplier;
	CoronaColor.B *= ColorParameters.Multiplier;

	if ( !ColorParameters.bInputIsLinear )
	{
		const bool bConvertToSRGB = false; // CoronaColor is already in SRGB
		CoronaColor = FLinearColor::FromSRGBColor( CoronaColor.ToFColor( bConvertToSRGB ) );
	}

	CompTex->AddSurface( CoronaColor );

	return FString();
}

FString FDatasmithMaxMatWriter::DumpCoronaMix(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale)
{
	FString ActualPrefix = FString(Prefix) + FString(TEXT("comp"));
	CompTex->SetMode( EDatasmithCompMode::Composite );

	int NumParamBlocks = InTexmap->NumParamBlocks();

	Texmap* TextureSlot1 = NULL;
	Texmap* TextureSlot2 = NULL;
	Texmap* TextureMask = NULL;
	bool bUseTexmap1 = true;
	bool bUseTexmap2 = true;
	bool bUseMask = true;
	BMM_Color_fl Color1;
	BMM_Color_fl Color2;
	double MixAmount = 0.0;
	int MixOperation = 0;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapTop")) == 0)
			{
				TextureSlot1 = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapBottom")) == 0)
			{
				TextureSlot2 = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapMix")) == 0)
			{
				TextureMask = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("colorTop")) == 0)
			{
				Color1 = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("colorBottom")) == 0)
			{
				Color2 = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("MixAmount")) == 0)
			{
				MixAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("MixOperation")) == 0)
			{
				MixOperation = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapTopOn")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bUseTexmap1 = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapBottomOn")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bUseTexmap2 = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapMixOn")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bUseMask = false;
				}
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	FString Result = TEXT("");
	if (TextureSlot1 != NULL && bUseTexmap1)
	{
		Result = DumpTexture(DatasmithScene, CompTex, TextureSlot1, DATASMITH_TEXTURENAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
	}
	else
	{
		CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color1));
	}

	if (TextureSlot2 != NULL && bUseTexmap2)
	{
		DumpTexture(DatasmithScene, CompTex, TextureSlot2, DATASMITH_TEXTURENAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
	}
	else
	{
		CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color2));
	}

	// Base layer
	CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(1.f, TEXT("BaseLayerWeight")) ); // weight
	CompTex->AddParamVal2( IDatasmithCompositeTexture::ParamVal(0.f, TEXT("Mode")) );   // Mode

	// top layer
	// weight
	if (TextureMask != NULL && bUseMask)
	{
		TSharedPtr<IDatasmithCompositeTexture> MaskTextureMap = FDatasmithSceneFactory::CreateCompositeTexture();

		DumpTexture(DatasmithScene, MaskTextureMap, TextureMask, DATASMITH_MASKNAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
		CompTex->AddMaskSurface(MaskTextureMap);
		CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(-1.f, TEXT("WeightUsesMask")) );
	}
	else
	{
		CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal((float)MixAmount, TEXT("Weight")) );
	}

	float Mode = 0.0f;
	switch (MixOperation)
	{
	case 0:
		Mode = float(EDatasmithCompositeCompMode::Add);
		break;
	case 1:
		Mode = float(EDatasmithCompositeCompMode::Sub);
		break;
	case 2:
		Mode = float(EDatasmithCompositeCompMode::Mult);
		break;
	case 4:
		Mode = float(EDatasmithCompositeCompMode::Darken);
		break;
	case 5:
		Mode = float(EDatasmithCompositeCompMode::Lighten);
		break;
	case 6:
		Mode = float(EDatasmithCompositeCompMode::Alpha);
		break;
	case 8:
		Mode = float(EDatasmithCompositeCompMode::Difference);
		break;
	case 9:
		Mode = float(EDatasmithCompositeCompMode::Screen);
		break;
	case 10:
		Mode = float(EDatasmithCompositeCompMode::Overlay);
		break;
	case 11:
		Mode = float(EDatasmithCompositeCompMode::Dodge);
		break;
	case 12:
		Mode = float(EDatasmithCompositeCompMode::Burn);
		break;
	case 13:
		Mode = float(EDatasmithCompositeCompMode::LinearBurn);
		break;
	case 14:
		Mode = float(EDatasmithCompositeCompMode::LinearLight);
		break;
	case 15:
		Mode = float(EDatasmithCompositeCompMode::Darken);
		break;
	case 16:
		Mode = float(EDatasmithCompositeCompMode::Lighten);
		break;
	case 17:
		Mode = float(EDatasmithCompositeCompMode::SoftLight);
		break;
	case 18:
		Mode = float(EDatasmithCompositeCompMode::HardLight);
		break;
	case 19:
		Mode = float(EDatasmithCompositeCompMode::PinLight);
		break;
	case 21:
		Mode = float(EDatasmithCompositeCompMode::Exclusion);
		break;
	default:
		Mode = float(EDatasmithCompositeCompMode::Alpha);
		break;
	}

	CompTex->AddParamVal2( IDatasmithCompositeTexture::ParamVal(Mode, TEXT("Mode")) );

	return Result;
}

FString FDatasmithMaxMatWriter::DumpCoronaMultitex(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, const TCHAR* ColorPrefix, bool bForceInvert, bool bIsGrayscale)
{
	FString ActualPrefix = FString(Prefix) + FString(TEXT("comp"));

	int NumParamBlocks = InTexmap->NumParamBlocks();
	Texmap* Texture0 = NULL;
	BMM_Color_fl Color0;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmaps")) == 0)
			{
				Texture0 = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 0);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("colors")) == 0)
			{
				Color0 = (BMM_Color_fl)ParamBlock2->GetAColor(ParamDefinition.ID, GetCOREInterface()->GetTime(), 0);
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	if (Texture0 == NULL)
	{
		CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color0));
		return FString();
	}
	else
	{
		return DumpTexture(DatasmithScene, CompTex, Texture0, Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
	}
}

bool FDatasmithMaxMatWriter::GetCoronaFixNormal(Texmap* InTexmap)
{
	if (InTexmap->ClassID() != CORONANORMALCLASS)
	{
		return false;
	}

	bool bFixNormalGamma = false;
	int NumParamBlocks = InTexmap->NumParamBlocks();

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("addGamma")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0)
				{
					bFixNormalGamma = true;
				}

				continue;
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	return bFixNormalGamma;
}