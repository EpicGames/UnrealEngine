// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef NEW_DIRECTLINK_PLUGIN

#include "DatasmithMaxDirectLink.h"

#include "DatasmithSceneFactory.h"


#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "impexp.h"
	#include "max.h"
MAX_INCLUDES_END

#if WITH_ITOO_INTERFACE
#pragma warning( push )
#pragma warning( disable: 4238 )
//#include "itoo/forestitreesinterface.h"
#include "itreesinterface.H"
#pragma warning( pop )

#include "ircinterface.H"
#endif // WITH_ITOO_INTERFACE

namespace DatasmithMaxDirectLink
{

namespace GeomUtils
{

bool ConvertRailClone(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker)
{
	INode* RailCloneNode = NodeTracker.Node;

	if (RailCloneNode == nullptr)
	{
		return false;
	}

	TimeValue CurrentTime = GetCOREInterface()->GetTime();

	IRCStaticInterface* RCStaticInterface = GetRCStaticInterface();
	if (!RCStaticInterface)
	{
		return false;
	}

	RCStaticInterface->IRCRegisterEngine();
	Interval ObjectValidity = RailCloneNode->GetObjectRef()->ObjectValidity(CurrentTime);

	NodeTracker.Validity.NarrowValidityToInterval(ObjectValidity);

	IRCInterface* RCInterface = GetRCInterface(RailCloneNode->GetObjectRef());
	if (!RCInterface)
	{
		return false;
	}
	TRCEngineFeatures RCFeatures;
	RCFeatures.rcAPIversion;
	RCFeatures.renderAPIversion;
	RCFeatures.supportNoGeomObjects = false; // todo: support non-mesh objects
	RCFeatures.disableMaterialBaking; // todo: might handle this ourselves

	if (RCStaticInterface->functions.Count() > 2) // Check that RC version is at least 3(see SDK)
	{
		RCStaticInterface->IRCSetEngineFeatures(INT_PTR(&RCFeatures)); // fills rcAPIversion
	}

	LogDebug(FString::Printf(TEXT("RCEngine:  sdk version=%d, runtime version:%d"), RCFeatures.renderAPIversion, RCFeatures.rcAPIversion));

	RCInterface->IRCRenderBegin(CurrentTime);

	int NumInstances;
	TRCInstance* RCInstance = RCGetInstances(RCInterface, RCFeatures, NumInstances);

	if (RCInstance && NumInstances > 0) // Required to check for null by the SDK
	{
		// todo: materials
		// MaterialEnum(RailCloneNode->GetMtl(), true);
		int32 NextMeshIndex = 0;

		struct FHismInstances
		{
			// todo: for forest - custom mesh node
			TUniquePtr<Mesh> MaxMesh;
			TArray<Matrix3> Transforms;
		};


		TArray<FHismInstances> InstancesForMesh;
		InstancesForMesh.Reserve(NumInstances);
		TMap<Mesh*, int32> RenderableNodeIndicesMap;

		for (int j = 0; j < NumInstances; j++, RCInstance = RCGetNextInstance(RCInstance, RCFeatures))
		{
			if (RCInstance && RCInstance->mesh)
			{
				if (int32* RenderableNodeIndexPtr = RenderableNodeIndicesMap.Find(RCInstance->mesh))
				{
					InstancesForMesh[*RenderableNodeIndexPtr].Transforms.Emplace(RCInstance->tm);
				}
				else
				{
					RenderableNodeIndicesMap.Add(RCInstance->mesh, InstancesForMesh.Num());
					FHismInstances& RenderableNode = InstancesForMesh.Emplace_GetRef();
					RenderableNode.MaxMesh = MakeUnique<Mesh>(*RCInstance->mesh);
					RenderableNode.Transforms.Emplace(RCInstance->tm);
				}
			}
		}

		// note: this is how baseline exporter derives names
		FString UniqueName = FString::FromInt(NodeTracker.Node->GetHandle());
		NodeTracker.CreateConverted().DatasmithActorElement = FDatasmithSceneFactory::CreateActor((const TCHAR*)*UniqueName);

		SceneTracker.SetupActor(NodeTracker);

		int32 MeshIndex = 0;
		for (FHismInstances& Instances: InstancesForMesh)
		{
			FMeshConverterSource MeshSource = {
				NodeTracker.Node, TEXT(""),
				FRenderMeshForConversion(NodeTracker.Node, Instances.MaxMesh.Get(), false), false,
				FRenderMeshForConversion()
			};

			SceneTracker.SetupDatasmithHISMForNode(NodeTracker, MeshSource, NodeTracker.Node->GetMtl(), MeshIndex, Instances.Transforms);
			MeshIndex++;
		}
	}

	RCInterface->IRCClearInstances();
	RCInterface->IRCClearMeshes();
	RCInterface->IRCRenderEnd(CurrentTime);

	return true;

}

bool ConvertForest(ISceneTracker& Scene, FNodeTracker& NodeTracker)
{
	INode* ForestNode = NodeTracker.Node;

	TimeValue CurrentTime = GetCOREInterface()->GetTime();

	IForestPackInterface* ForestPackInterface = GetForestPackInterface();
	ForestPackInterface->IForestRegisterEngine();

	TForestEngineFeatures EngineFeatures;
	EngineFeatures.edgeMode = FALSE;
	EngineFeatures.meshesSupport = FALSE;
	ForestPackInterface->IForestSetEngineFeatures((INT_PTR) &EngineFeatures);

	ITreesInterface* ITrees = GetTreesInterface(ForestNode->GetObjectRef());
	ITrees->IForestRenderBegin(CurrentTime);

	int NumInstances;
	TForestInstance* ForestInstance = (TForestInstance*)ITrees->IForestGetRenderNodes(NumInstances);

	ulong ForestHandle = ForestNode->GetHandle();
	FString ForestName = ForestNode->GetName();
	Matrix3 ForestMatrix = ForestNode->GetNodeTM(CurrentTime);
	if (ForestInstance && NumInstances)
	{
		struct FHismInstances
		{
			INode* GeometryNode;
			TMap<Mtl*, TArray<Matrix3>> InstancesTransformsForMaterial;
		};


		TArray<FHismInstances> InstancesForMesh;
		InstancesForMesh.Reserve(NumInstances); // Reserve to avoid reallocations
		TMap<int, int32> RenderableNodeIndicesMap;

		for (int i = 0; i < NumInstances; i++, ForestInstance++)
		{
			if (ForestInstance->node != NULL)
			{
				int VirtualMaster = ITrees->IForestGetSpecID(i);

				if (int32* RenderableNodeIndexPtr = RenderableNodeIndicesMap.Find(VirtualMaster))
				{
					InstancesForMesh[*RenderableNodeIndexPtr].InstancesTransformsForMaterial.FindOrAdd(ForestInstance->mtl).Emplace(ForestInstance->tm);
				}
				else
				{
					RenderableNodeIndicesMap.Add(VirtualMaster, InstancesForMesh.Num());
					FHismInstances& RenderableNode = InstancesForMesh.Emplace_GetRef();

					RenderableNode.GeometryNode = ForestInstance->node;
					RenderableNode.InstancesTransformsForMaterial.FindOrAdd(ForestInstance->mtl).Emplace(ForestInstance->tm);
				}
			}
		}

		// note: this is how baseline exporter derives names
		FString UniqueName = FString::FromInt(NodeTracker.Node->GetHandle());
		NodeTracker.CreateConverted().DatasmithActorElement = FDatasmithSceneFactory::CreateActor((const TCHAR*)*UniqueName);

		Scene.SetupActor(NodeTracker);

		int32 MeshIndex = 0;
		for (FHismInstances& Instances: InstancesForMesh)
		{
			INode* GeometryNode = Instances.GeometryNode;

			FMeshConverterSource MeshSource = {
				GeometryNode, TEXT(""),
				GetMeshForGeomObject(CurrentTime, GeometryNode), false,
				FRenderMeshForConversion()
			};

			if (MeshSource.IsValid())
			{
				for(const TPair<Mtl*, TArray<Matrix3>>& MaterialAndTransforms: Instances.InstancesTransformsForMaterial)
				{
					Scene.SetupDatasmithHISMForNode(NodeTracker, MeshSource, MaterialAndTransforms.Key, MeshIndex, MaterialAndTransforms.Value);
				}

				MeshIndex ++;
			}
		}
	}

	ITrees->IForestClearRenderNodes();
	ITrees->IForestRenderEnd(CurrentTime);

	return true;

}

}
}


#include "Windows/HideWindowsPlatformTypes.h"

#endif // NEW_DIRECTLINK_PLUGIN
