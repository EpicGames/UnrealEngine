// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef NEW_DIRECTLINK_PLUGIN

#include "DatasmithMaxDirectLink.h"

#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY(LogDatasmithMaxExporter);

#include "Async/Async.h"

#include "Misc/OutputDeviceRedirector.h" // For GLog

#include "Logging/LogMacros.h"


#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "max.h"

	#include "maxscript/maxscript.h"

	#include "ISceneEventManager.h"
MAX_INCLUDES_END


namespace DatasmithMaxDirectLink
{

void LogDebugImpl(const FString& Msg)
{
#ifdef LOG_DEBUG_ENABLE
	// mprintf(L"[%s]%s\n", *FDateTime::UtcNow().ToString(TEXT("%Y.%m.%d-%H.%M.%S:%s")), *Msg);
	UE_LOGF(LogDatasmithMaxExporter, Error, "%ls", *Msg);
	LogFlush();
	LogDebugDialog(Msg);
#endif
}

void LogInfo(const TCHAR* Msg)
{
	UE_LOGF(LogDatasmithMaxExporter, Display, "%ls", Msg);
	LogInfoDialog(Msg);
}

void LogWarning(const TCHAR* Msg)
{
	UE_LOGF(LogDatasmithMaxExporter, Warning, "%ls", Msg);
	LogWarningDialog(Msg);
}

void LogCompletion(const TCHAR* Msg)
{
	UE_LOGF(LogDatasmithMaxExporter, Display, "%ls", Msg);
	LogCompletionDialog(Msg);
}

void LogError(const TCHAR* Msg)
{
	UE_LOGF(LogDatasmithMaxExporter, Display, "%ls", Msg);
	LogErrorDialog(Msg);
}

void LogError(const FString& Msg)
{
	LogError(*Msg);
}

void LogWarning(const FString& Msg)
{
	LogWarning(*Msg);
}

void LogCompletion(const FString& Msg)
{
	LogCompletion(*Msg);
}

void LogInfo(const FString& Msg)
{
	LogInfo(*Msg);
}

void LogFlush()
{
	Async(EAsyncExecution::TaskGraphMainThread,
		[]()
		{
			GLog->FlushThreadedLogs();
			GLog->Flush();
		});
}

FString GetNodeDesc(INode* Node)
{
	return Node ? FString::Printf(TEXT("%s(%llu)"), Node->GetName(), NodeEventNamespace::GetKeyByNode(Node)) : TEXT("<null>");
}

void LogDebugNode(const FString& Name, INode* Node)
{
#ifdef LOG_DEBUG_HEAVY_ENABLE

	LogDebug(FString::Printf(TEXT("%s: %s - %s, parent: %s")
		, *Name
		, *GetNodeDesc(Node)
		, (Node && Node->IsNodeHidden(TRUE))? TEXT("HIDDEN") : TEXT("")
		, *GetNodeDesc(Node ? Node->GetParentNode() : nullptr)
		));
	if (Node)
	{
		LogDebug(FString::Printf(TEXT("    NumberOfChildren: %d "), Node->NumberOfChildren()));
		
		if (Object* ObjectRef = Node->GetObjectRef())
		{
			Class_ID ClassId = ObjectRef->ClassID();
			LogDebug(FString::Printf(TEXT("    Class_ID: 0x%lx, 0x%lx "), ClassId.PartA(), ClassId.PartB()));
		}
	}
#endif
}

void LogNodeEvent(const MCHAR* Name, INodeEventCallback::NodeKeyTab& Nodes)
{
#ifdef LOG_DEBUG_HEAVY_ENABLE
	LogDebug(FString::Printf(TEXT("NodeEventCallback:%s"), Name));
	for (int NodeIndex = 0; NodeIndex < Nodes.Count(); ++NodeIndex)
	{
		FNodeKey NodeKey = Nodes[NodeIndex];

		Animatable* anim = Animatable::GetAnimByHandle(NodeKey);
		if (INode* Node = NodeEventNamespace::GetNodeByKey(NodeKey)) // Node sometimes is null. Not sure why
		{
			LogDebug(FString::Printf(TEXT("   %s, parent: %s"), *GetNodeDesc(Node), *GetNodeDesc(Node->GetParentNode())));
		}
		else
		{
			LogDebug(FString::Printf(TEXT("   <null>(%llu)"), NodeKey));
		}
	}
#endif
}

}
#include "Windows/HideWindowsPlatformTypes.h"

#endif // NEW_DIRECTLINK_PLUGIN
