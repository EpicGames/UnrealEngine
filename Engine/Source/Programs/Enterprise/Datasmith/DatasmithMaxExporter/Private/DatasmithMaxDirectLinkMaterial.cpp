// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef NEW_DIRECTLINK_PLUGIN

#include "DatasmithMaxDirectLink.h"

#include "DatasmithMaxHelper.h"
#include "DatasmithMaxWriter.h"
#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxSceneExporter.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxMaterialsToUEPbr.h"


#include "DatasmithSceneFactory.h"
#include "DatasmithExportOptions.h"

#include "Misc/Paths.h"
#include "Misc/SecureHash.h"


#include "Logging/LogMacros.h"


#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "impexp.h"
	#include "max.h"
MAX_INCLUDES_END

namespace DatasmithMaxDirectLink
{

void FMaterialsCollectionTracker::AddActualMaterial(FMaterialTracker& MaterialTracker, Mtl* Material)
{
	// Record relationships between tracked material and actual materials used on geometry(e.g. submaterials of a tracked multisubobj material)
	MaterialTracker.AddActualMaterial(Material);
	TSet<FMaterialTracker*>& MaterialTrackersForMaterial = ActualMaterialToMaterialTracker.FindOrAdd(Material);
	MaterialTrackersForMaterial.Add(&MaterialTracker);
}

void FMaterialsCollectionTracker::ReleaseActualMaterial(FMaterialTracker& MaterialTracker, Mtl* Material)
{
	TSet<FMaterialTracker*>* Found = ActualMaterialToMaterialTracker.Find(Material);
	if (!Found)
	{
		return;
	}

	TSet<FMaterialTracker*>& MaterialTrackersForMaterial = *Found;
	MaterialTrackersForMaterial.Remove(&MaterialTracker);
	if (!MaterialTrackersForMaterial.Num())
	{
		//Clean up material when it's not used by other tracked materials
		ActualMaterialToMaterialTracker.Remove(Material);

		if (FString Name; UsedMaterialToDatasmithMaterialName.RemoveAndCopyValue(Material, Name))
		{
			MaterialNameProvider.RemoveExistingName(Name);
		}

		RemoveDatasmithMaterialForUsedMaterial(Material);
	}
}


const TCHAR* FMaterialsCollectionTracker::GetMaterialName(Mtl* Material)
{
	if (FString* NamePtr = UsedMaterialToDatasmithMaterialName.Find(Material))
	{
		return **NamePtr;
	}

	// Using material's Name as Max doesn't have another persistent id for materials(like INode has with its GetHandle)
	FString SourceName = Material->GetName().data();
	FString SanitizedName = FDatasmithUtils::SanitizeObjectName(SourceName);

	// Limit Datasmith name size to FName length max
	if (SanitizedName.Len() > (NAME_SIZE-1))
	{
		// In case we have such long material names make sure that their shortened versions don't match when their source names are different
		// We could rely GenerateUniqueName but since we can build a stable name better do it because we can.
		// GenerateUniqueName is the last resort when materials have identical names
		FString NameHash = FMD5::HashBytes(reinterpret_cast<const uint8*>(*SourceName), SourceName.Len()*sizeof(TCHAR));
		SanitizedName = SanitizedName.Left(NAME_SIZE-1-NameHash.Len()) + NameHash;
	}

	return *UsedMaterialToDatasmithMaterialName.Add(Material, MaterialNameProvider.GenerateUniqueName(SanitizedName));
}

bool FMaterialsCollectionTracker::IsMaterialUsed(Mtl* Material)
{
	return (UsedMaterialToMeshElementSlot.Contains(Material) || UsedMaterialToMeshActorSlot.Contains(Material));
}

void FMaterialsCollectionTracker::SetMaterialForMeshElement(const TSharedPtr<IDatasmithMeshElement>& MeshElement, Mtl* MaterialAssignedToNode, Mtl* ActualMaterial, uint16 SlotId)
{
	MeshElement->SetMaterial(GetMaterialName(ActualMaterial), SlotId);

	if (!UsedMaterialToDatasmithMaterial.Contains(ActualMaterial))
	{
		// Invalidate material assigned directly to node to make it build its submaterials
		// Rebuilding individual submaterials looks much more complex especially taking into account that Update can be cancelled/resumed
		InvalidateMaterial(MaterialAssignedToNode);
	}

	UsedMaterialToMeshElementSlot.FindOrAdd(ActualMaterial).FindOrAdd(MeshElement.Get()).Add(SlotId);
	MeshElementToUsedMaterialSlot.FindOrAdd(MeshElement.Get()).FindOrAdd(ActualMaterial).Add(SlotId);
}

void FMaterialsCollectionTracker::SetMaterialForMeshActorElement(
	const TSharedPtr<IDatasmithMeshActorElement>& MeshActor, Mtl* MaterialAssignedToNode, Mtl* Material, int32 SlotId)
{
	MeshActor->AddMaterialOverride(GetMaterialName(Material), SlotId);

	UsedMaterialToMeshActorSlot.FindOrAdd(Material).FindOrAdd(MeshActor.Get()).Add(SlotId);
	MeshActorToUsedMaterialSlot.FindOrAdd(MeshActor.Get()).FindOrAdd(Material).Add(SlotId);
}

void FMaterialsCollectionTracker::UnSetMaterialsForMeshElement(const TSharedPtr<IDatasmithMeshElement>& MeshElement)
{
	if (!MeshElement)
	{
		return;
	}

	if (!MeshElementToUsedMaterialSlot.Contains(MeshElement.Get()))
	{
		return;
	}

	TArray<Mtl*> Materials;
	MeshElementToUsedMaterialSlot[MeshElement.Get()].GetKeys(Materials);
	for (Mtl* Material : Materials)
	{
		UsedMaterialToMeshElementSlot[Material].Remove(MeshElement.Get());
		if (UsedMaterialToMeshElementSlot[Material].IsEmpty())
		{
			if (!IsMaterialUsed(Material))
			{
				RemoveDatasmithMaterialForUsedMaterial(Material);
			}
			UsedMaterialToMeshElementSlot.Remove(Material);
		}
	}
	MeshElementToUsedMaterialSlot.Remove(MeshElement.Get());
}

void FMaterialsCollectionTracker::UnSetMaterialsForMeshActor(const TSharedPtr<IDatasmithMeshActorElement>& MeshActor)
{
	if (!MeshActor)
	{
		return;
	}

	if (!MeshActorToUsedMaterialSlot.Contains(MeshActor.Get()))
	{
		return;
	}

	TArray<Mtl*> Materials;
	MeshActorToUsedMaterialSlot[MeshActor.Get()].GetKeys(Materials);
	for (Mtl* Material : Materials)
	{
		UsedMaterialToMeshActorSlot[Material].Remove(MeshActor.Get());
		if (UsedMaterialToMeshActorSlot[Material].IsEmpty())
		{
			if (!IsMaterialUsed(Material))
			{
				RemoveDatasmithMaterialForUsedMaterial(Material);
			}
			UsedMaterialToMeshActorSlot.Remove(Material);
		}
	}
	MeshActorToUsedMaterialSlot.Remove(MeshActor.Get());
}

void FMaterialsCollectionTracker::SetMaterialsForMeshElement(FMeshConverted& MeshConverted, Mtl* Material)
{
	SetMaterialsForMeshElement(MeshConverted.GetDatasmithMeshElement(), Material, MeshConverted.SupportedChannels);
}

void FMaterialsCollectionTracker::SetMaterialsForMeshElement(const TSharedPtr<IDatasmithMeshElement>& MeshElement, Mtl* Material, const TSet<uint16>& SupportedChannels)
{
	if (FDatasmithMaxMatHelper::GetMaterialClass(Material) == EDSMaterialType::XRefMat)
	{
		SetMaterialsForMeshElement(MeshElement, FDatasmithMaxMatHelper::GetRenderedXRefMaterial(Material), SupportedChannels);
		return;
	}
	if (Material != nullptr)
	{
		TArray<uint16> ChannelsSorted = SupportedChannels.Array();
		ChannelsSorted.Sort();
		for (uint16 Channel : ChannelsSorted)
		{
			//Max's channel UI is not zero-based, so we register an incremented ChannelID for better visual consistency after importing in Unreal.
			uint16 DisplayedChannelID = Channel + 1;

			// Multi mat
			if (FDatasmithMaxMatHelper::GetMaterialClass(Material) == EDSMaterialType::MultiMat)
			{
				// Replicate the 3ds max behavior where material ids greater than the number of sub-materials wrap around and are mapped to the existing sub-materials
				if ( Mtl* SubMaterial = Material->GetSubMtl(Channel % Material->NumSubMtls()) )
				{
					if (FDatasmithMaxMatHelper::GetMaterialClass(SubMaterial) == EDSMaterialType::TheaRandom)
					{
						SetMaterialForMeshElement(MeshElement, Material, FDatasmithMaxSceneExporter::GetRandomSubMaterial(SubMaterial, MeshElement->GetDimensions()), DisplayedChannelID);
					}
					else
					{
						SetMaterialForMeshElement(MeshElement, Material, SubMaterial, DisplayedChannelID);
					}
				}
			}
			else if (FDatasmithMaxMatHelper::GetMaterialClass(Material) == EDSMaterialType::TheaRandom)
			{
				SetMaterialForMeshElement(MeshElement, Material, FDatasmithMaxSceneExporter::GetRandomSubMaterial(Material, MeshElement->GetDimensions()), DisplayedChannelID);
			}
			// Single material
			else
			{
				SetMaterialForMeshElement(MeshElement, Material, Material, DisplayedChannelID);
			}
		}
	}
}

void FMaterialsCollectionTracker::SetMaterialsForMeshActor(const TSharedPtr<IDatasmithMeshActorElement>& MeshActor, Mtl* Material, TSet<uint16>& SupportedChannels, const FVector3f& RandomSeed)
{
	if (FDatasmithMaxMatHelper::GetMaterialClass(Material) == EDSMaterialType::XRefMat)
	{
		SetMaterialsForMeshActor(MeshActor, FDatasmithMaxMatHelper::GetRenderedXRefMaterial(Material), SupportedChannels, RandomSeed);
		return;
	}

	if (Material != nullptr)
	{
		if (SupportedChannels.Num() <= 1)
		{
			if (FDatasmithMaxMatHelper::GetMaterialClass(Material) != EDSMaterialType::MultiMat)
			{
				if (FDatasmithMaxMatHelper::GetMaterialClass(Material) != EDSMaterialType::TheaRandom)
				{
					SetMaterialForMeshActorElement(MeshActor, Material, Material, -1);
				}
				else
				{
					SetMaterialForMeshActorElement(MeshActor, Material, FDatasmithMaxSceneExporter::GetRandomSubMaterial(Material, RandomSeed), -1 );
				}
			}
			else
			{
				int Mid = 0;

				//Find the lowest supported material id.
				SupportedChannels.Sort([](const uint16& A, const uint16& B) { return (A < B); });
				for (const uint16 SupportedMid : SupportedChannels)
				{
					Mid = SupportedMid;
				}

				// Replicate the 3ds max behavior where material ids greater than the number of sub-materials wrap around and are mapped to the existing sub-materials
				if (Mtl* SubMaterial = Material->GetSubMtl(Mid % Material->NumSubMtls()))
				{
					if (FDatasmithMaxMatHelper::GetMaterialClass(SubMaterial) != EDSMaterialType::TheaRandom)
					{
						SetMaterialForMeshActorElement(MeshActor, Material, SubMaterial, -1);
					}
					else
					{
						SetMaterialForMeshActorElement(MeshActor, Material, FDatasmithMaxSceneExporter::GetRandomSubMaterial(Material, RandomSeed), -1 );
					}
				}
			}
		}
		else
		{
			int ActualSubObj = 1;
			SupportedChannels.Sort([](const uint16& A, const uint16& B) { return (A < B); });
			for (const uint16 Mid : SupportedChannels)
			{
				if (FDatasmithMaxMatHelper::GetMaterialClass(Material) != EDSMaterialType::MultiMat)
				{
					if (FDatasmithMaxMatHelper::GetMaterialClass(Material) != EDSMaterialType::TheaRandom)
					{
						SetMaterialForMeshActorElement(MeshActor, Material, Material, ActualSubObj);
					}
					else
					{
						SetMaterialForMeshActorElement(MeshActor, Material, FDatasmithMaxSceneExporter::GetRandomSubMaterial(Material, RandomSeed), -1);
					}
				}
				else
				{
					// Replicate the 3ds max behavior where material ids greater than the number of sub-materials wrap around and are mapped to the existing sub-materials
					int32 MaterialIndex = Mid % Material->NumSubMtls();

					Mtl* SubMaterial = Material->GetSubMtl(MaterialIndex);
					if (SubMaterial != nullptr)
					{
						if (FDatasmithMaxMatHelper::GetMaterialClass(SubMaterial) != EDSMaterialType::TheaRandom)
						{
							//Material slots in Max are not zero-based, so we serialize our SlotID starting from 1 for better visual consistency.
							SetMaterialForMeshActorElement(MeshActor, Material, SubMaterial, Mid + 1);
						}
						else
						{
							SetMaterialForMeshActorElement(MeshActor, Material, FDatasmithMaxSceneExporter::GetRandomSubMaterial(SubMaterial, RandomSeed), MaterialIndex + 1);
						}
					}
				}
				ActualSubObj++;
			}
		}
	}
}


// Collects actual materials that are used by the top-level material(assigned to node)
class FMaterialEnum
{
public:
	FMaterialsCollectionTracker& MaterialsCollectionTracker;
	FMaterialTracker& MaterialTracker;

	FMaterialEnum(FMaterialsCollectionTracker& InMaterialsCollectionTracker, FMaterialTracker& InMaterialTracker): MaterialsCollectionTracker(InMaterialsCollectionTracker), MaterialTracker(InMaterialTracker) {}

	void MaterialEnum(Mtl* Material, bool bAddMaterial)
	{
		if (Material == NULL)
		{
			return;
		}

		if (FDatasmithMaxMatHelper::GetMaterialClass(Material) == EDSMaterialType::XRefMat)
		{
			MaterialEnum(FDatasmithMaxMatHelper::GetRenderedXRefMaterial(Material), true);
		}
		else if (FDatasmithMaxMatHelper::GetMaterialClass(Material) == EDSMaterialType::MultiMat)
		{
			for (int i = 0; i < Material->NumSubMtls(); i++)
			{
				MaterialEnum(Material->GetSubMtl(i), true);
			}
		}
		else
		{
			if (bAddMaterial)
			{
				MaterialsCollectionTracker.AddActualMaterial(MaterialTracker, Material);
			}

			bool bAddRecursively = Material->ClassID() == THEARANDOMCLASS || Material->ClassID() == VRAYBLENDMATCLASS || Material->ClassID() == CORONALAYERMATCLASS || Material->ClassID() == BLENDMATCLASS;
			for (int i = 0; i < Material->NumSubMtls(); i++)
			{
				MaterialEnum(Material->GetSubMtl(i), bAddRecursively);
			}
		}
	}
};

void FMaterialsCollectionTracker::Reset()
{
	MaterialTrackers.Reset();
	InvalidatedMaterialTrackers.Reset();

	ActualMaterialToMaterialTracker.Reset();

	UsedMaterialToDatasmithMaterial.Reset();
	UsedMaterialToDatasmithMaterialName.Reset();
	UsedMaterialToMeshElementSlot.Reset();
	MeshElementToUsedMaterialSlot.Reset();

	UsedMaterialToMeshActorSlot.Reset();
	MeshActorToUsedMaterialSlot.Reset();

	MaterialNameProvider.Clear();

	UsedTextureToMaterialTracker.Reset();
	UsedTextureToDatasmithElement.Reset();
	TextureElementToTexmap.Reset();

	TextureElementsAddedToScene.Reset();
}

void FMaterialsCollectionTracker::UpdateMaterial(FMaterialTracker* MaterialTracker)
{
	RemoveConvertedMaterial(*MaterialTracker);
	FMaterialEnum(*this, *MaterialTracker).MaterialEnum(MaterialTracker->Material, true);
}

void FMaterialsCollectionTracker::AddDatasmithMaterialForUsedMaterial(TSharedRef<IDatasmithScene> DatasmithScene, Mtl* Material, TSharedPtr<IDatasmithBaseMaterialElement> DatasmithMaterial)
{
	if (DatasmithMaterial)
	{
		SCENE_UPDATE_STAT_INC(UpdateMaterials, Converted);
		DatasmithScene->AddMaterial(DatasmithMaterial);
		UsedMaterialToDatasmithMaterial.Add(Material, DatasmithMaterial);
		SceneTracker.RemapConvertedMaterialUVChannels(Material, DatasmithMaterial);
	}
}

void FMaterialsCollectionTracker::ConvertMaterial(Mtl* Material, TSharedRef<IDatasmithScene> DatasmithScene, const TCHAR* AssetsPath, TSet<Texmap*>& TexmapsConverted)
{
	SCENE_UPDATE_STAT_INC(UpdateMaterials, Total);

	if (UsedMaterialToDatasmithMaterial.Contains(Material))
	{
		// Material might have been already converted - if it's present in UsedMaterialToDatasmithMaterial this means that it(or multisubobj it's part of) wasn't changed
		// e.g. this happens when another multisubobj material is added with existing (and already converted) material as submaterial
		return;
	}

	if (!IsMaterialUsed(Material))
	{
		// Don't convert materials that don't have visible geometry exported
		// This happens when a submaterial of MultiSubobj material is unused on a mesh
		return;
	}

	TSet<Texmap*> TexmapsUsedByMaterial;
	FMaterialConversionContext MaterialConversionContext = {TexmapsUsedByMaterial, *this};
	TGuardValue MaterialConversionContextGuard(FDatasmithMaxMaterialsToUEPbrManager::Context, &MaterialConversionContext);

	TSharedPtr<IDatasmithBaseMaterialElement> DatasmithMaterial;
	if (FDatasmithMaxMaterialsToUEPbr* MaterialConverter = FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter(Material))
	{
		MaterialConverter->Convert(DatasmithScene, DatasmithMaterial, Material, AssetsPath);
		AddDatasmithMaterialForUsedMaterial(DatasmithScene, Material, DatasmithMaterial);
	}
	else
	{
		MSTR Classname;
		Material->GetClassName(Classname);
		FString MaterialDesc = FString::Printf(TEXT("\"%s\" of type <%s> (0x%08x-0x%08x)"), Material->GetName().data(), Classname.data(), Material->ClassID().PartA(), Material->ClassID().PartB());
		LogWarning(FString(TEXT("Material not supported: ")) + MaterialDesc);
	}

	// Tie texture used by an actual material to tracked material
	for (Texmap* Tex : TexmapsUsedByMaterial)
	{
		for (FMaterialTracker* MaterialTracker : ActualMaterialToMaterialTracker[Material])
		{
			MaterialTracker->AddActualTexture(Tex);
			UsedTextureToMaterialTracker.FindOrAdd(Tex).Add(MaterialTracker);
		}

		TexmapsConverted.Add(Tex);
	}

}

void FMaterialsCollectionTracker::ReleaseMaterial(FMaterialTracker& MaterialTracker)
{
	RemoveConvertedMaterial(MaterialTracker);
	MaterialTrackers.Remove(MaterialTracker.Material);
	InvalidatedMaterialTrackers.Remove(&MaterialTracker);
}

void FMaterialsCollectionTracker::RemoveDatasmithMaterialForUsedMaterial(Mtl* Material)
{
	TSharedPtr<IDatasmithBaseMaterialElement> DatasmithMaterial;
	if(UsedMaterialToDatasmithMaterial.RemoveAndCopyValue(Material, DatasmithMaterial))
	{
		SceneTracker.RemoveMaterial(DatasmithMaterial);
	}
}

void FMaterialsCollectionTracker::RemoveConvertedMaterial(FMaterialTracker& MaterialTracker)
{
	for (Mtl* Material: MaterialTracker.GetActualMaterials())
	{
		ReleaseActualMaterial(MaterialTracker, Material);
	}

	for (Texmap* Tex: MaterialTracker.GetActualTexmaps())
	{
		TSet<FMaterialTracker*>& MaterialTrackersForTexture = UsedTextureToMaterialTracker[Tex];
		MaterialTrackersForTexture.Remove(&MaterialTracker);

		if (!MaterialTrackersForTexture.Num()) // No tracked materials are using this texture anymore
		{
			UsedTextureToMaterialTracker.Remove(Tex);

			for (const TSharedPtr<IDatasmithTextureElement>& TextureElement: UsedTextureToDatasmithElement[Tex])
			{
				TSet<Texmap*>& Texmaps = TextureElementToTexmap[TextureElement];
				Texmaps.Remove(Tex);

				if (Texmaps.IsEmpty())  // This was the last texmap that produced this element
				{
					RemoveTextureElement(TextureElement);
					TextureElementToTexmap.Remove(TextureElement);
				}
			}
			UsedTextureToDatasmithElement.Remove(Tex);
		}
	}

	MaterialTracker.ResetActualMaterialAndTextures();
}

void FMaterialsCollectionTracker::UpdateTexmap(Texmap* Texmap)
{
	if (UsedTextureToDatasmithElement.Contains(Texmap))
	{
		// Don't update texmap that wasn't released - this means that it doesn't need update
		// Texmap is released when every material that uses it is invalidated or removed
		// When Texmap wasn't released means that some materials using it weren't invalidated which implies texmap is up to date(or material would have received a change event)
		return;
	}

	TArray<TSharedPtr<IDatasmithTextureElement>> TextureElements;
	if (TSharedPtr<ITexmapToTextureElementConverter>* Found = TexmapConverters.Find(Texmap))
	{
		TSharedPtr<ITexmapToTextureElementConverter> Converter = *Found;
		TSharedPtr<IDatasmithTextureElement> TextureElement = Converter->Convert(*this, Converter->TextureElementName);
		if (TextureElement)
		{
			TextureElements.Add(TextureElement);
			AddTextureElement(TextureElement);
		}
	}

	UsedTextureToDatasmithElement.FindOrAdd(Texmap).Append(TextureElements);

	for (TSharedPtr<IDatasmithTextureElement> TextureElement : TextureElements)
	{
		TextureElementToTexmap.FindOrAdd(TextureElement).Add(Texmap);
	}
}

void FMaterialsCollectionTracker::AddTextureElement(const TSharedPtr<IDatasmithTextureElement>& TextureElement)
{
	if (TextureElementsAddedToScene.Contains(TextureElement))
	{
		return;
	}

	SceneTracker.GetDatasmithSceneRef()->AddTexture(TextureElement);
	TextureElementsAddedToScene.Add(TextureElement);
}

void FMaterialsCollectionTracker::RemoveTextureElement(const TSharedPtr<IDatasmithTextureElement>& TextureElement)
{
	TextureElementsAddedToScene.Remove(TextureElement);
	SceneTracker.RemoveTexture(TextureElement);
}

void FMaterialsCollectionTracker::AddTexmapForConversion(Texmap* Texmap, const FString& DesiredTextureElementName, const TSharedPtr<ITexmapToTextureElementConverter>& Converter)
{
	Converter->TextureElementName = DesiredTextureElementName;
	TexmapConverters.Add(Texmap, Converter);
}

FMaterialTracker* FMaterialsCollectionTracker::AddMaterial(Mtl* Material)
{
	if (FMaterialTrackerHandle* HandlePtr = MaterialTrackers.Find(Material))
	{
		return HandlePtr->GetMaterialTracker();
	}

	// Track material if not yet
	FMaterialTrackerHandle& MaterialTrackerHandle = MaterialTrackers.Emplace(Material, Material);
	InvalidatedMaterialTrackers.Add(MaterialTrackerHandle.GetMaterialTracker());
	return MaterialTrackerHandle.GetMaterialTracker();
}

void FMaterialsCollectionTracker::InvalidateMaterial(Mtl* Material)
{
	if (FMaterialTrackerHandle* MaterialTrackerHandle = MaterialTrackers.Find(Material))
	{
		InvalidatedMaterialTrackers.Add(MaterialTrackerHandle->GetMaterialTracker());
	}
}

}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // NEW_DIRECTLINK_PLUGIN
