// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef NEW_DIRECTLINK_PLUGIN

#pragma warning( push )
#pragma warning( disable: 5103 )

#include "DatasmithMaxDirectLink.h"
#include "DatasmithMaxLogger.h"

#include "Async/Async.h"
#include "Containers/Queue.h"

#include "Resources/Windows/resource.h"

#include "Windows/AllowWindowsPlatformTypes.h"

#include "UI/IDatasmith3dsMaxUI.h"


MAX_INCLUDES_START
	#include "impexp.h"
	#include "max.h"

	#include "maxscript/maxwrapper/mxsobjects.h"

	#include "maxscript/maxscript.h"
	#include "maxscript/foundation/numbers.h"
	#include "maxscript/foundation/arrays.h"
	#include "maxscript\macros\define_instantiation_functions.h"

	#include <maxicon.h>
MAX_INCLUDES_END

#include "windows.h"

extern HINSTANCE HInstanceMax;


namespace DatasmithMaxDirectLink
{

class FDummyDatasmith3dsMaxUI: public Ui::IMessagesWindow
{
public:
	virtual void OpenWindow() override
	{
	}

	virtual void AddError(const FString& Message)  override
	{
	}
	virtual void AddWarning(const FString& Message)  override
	{
	}
	virtual void AddInfo(const FString& Message)  override
	{
	}
	virtual void AddCompletion(const FString& Message)  override
	{
	}
	virtual void ClearMessages()  override
	{
	}
};

FDummyDatasmith3dsMaxUI DummyDatasmith3dsMaxUI;
Ui::IMessagesWindow* MessagesDialog = &DummyDatasmith3dsMaxUI;

/************************************* MaxScript exports *********************************/

Value* OnLoad_cf(Value**, int);
Primitive OnLoad_pf(_M("Datasmith_OnLoad"), OnLoad_cf);

Value* OnLoad_cf(Value **arg_list, int count)
{
	check_arg_count(OnLoad, 2, count);
	Value* pEnableUI= arg_list[0];
	Value* pEnginePath = arg_list[1];

	bool bEnableUI = pEnableUI->to_bool();

	const TCHAR* EnginePathUnreal = (const TCHAR*)pEnginePath->to_string();

	bool bResult = CreateExporter(bEnableUI, EnginePathUnreal);

	if (bResult)
	{
		// Create Slate UI only when CreateExporter returns true(it needs to successfully init  engine loop there for Slate to work)
		MessagesDialog = Ui::CreateMessagesWindow();
	}

	LogInfo(FString::Printf(TEXT("Initialized Datasmith SDK v.%s"), *FDatasmithUtils::GetEnterpriseVersionAsString(true)));

	return bool_result(bResult);
}

Value* OnUnload_cf(Value**, int);
Primitive OnUnload_pf(_M("Datasmith_OnUnload"), OnUnload_cf);

Value* OnUnload_cf(Value **arg_list, int count)
{
	check_arg_count(OnUnload, 0, count);

	ShutdownExporter();

	return bool_result(true);
}

Value* SetOutputPath_cf(Value**, int);
Primitive SetOutputPath_pf(_M("Datasmith_SetOutputPath"), SetOutputPath_cf);

Value* SetOutputPath_cf(Value** arg_list, int count)
{
	check_arg_count(CreateScene, 1, count);
	Value* pOutputPath = arg_list[0];

	GetExporter()->SetOutputPath(pOutputPath->to_string());

	return bool_result(true);
}

Value* CreateScene_cf(Value**, int);
Primitive CreateScene_pf(_M("Datasmith_CreateScene"), CreateScene_cf);

Value* CreateScene_cf(Value** arg_list, int count)
{
	check_arg_count(CreateScene, 1, count);
	Value* pName = arg_list[0];

	GetExporter()->SetName(pName->to_string());

	return bool_result(true);
}

Value* UpdateScene_cf(Value**, int);
Primitive UpdateScene_pf(_M("Datasmith_UpdateScene"), UpdateScene_cf);

Value* UpdateScene_cf(Value** arg_list, int count)
{
	check_arg_count_with_keys(UpdateScene, 0, count);

	bool bQuiet = key_arg_or_default(quiet, &false_value)->to_bool();

	if(!GetExporter())
	{
		return bool_result(false);
	}

	bool bResult = GetExporter()->UpdateScene(bQuiet);
	return bool_result(bResult);
}


Value* Export_cf(Value**, int);
Primitive Export_pf(_M("Datasmith_Export"), Export_cf);

Value* Export_cf(Value** arg_list, int count)
{
	check_arg_count_with_keys(Export, 2, count);
	Value* pName = arg_list[0];
	Value* pOutputPath = arg_list[1];

	bool bQuiet = key_arg_or_default(quiet, &false_value)->to_bool();
	bool bSelected = key_arg_or_default(selected, &false_value)->to_bool();

	bool bResult = Export(pName->to_string(), pOutputPath->to_string(), bQuiet, bSelected);;
	return bool_result(bResult);
}


Value* Reset_cf(Value**, int);
Primitive Reset_pf(_M("Datasmith_Reset"), Reset_cf);

Value* Reset_cf(Value** arg_list, int count)
{
	check_arg_count(Reset, 0, count);

	if (!GetExporter())
	{
		return bool_result(false);
	}

	GetExporter()->ResetSceneTracking();
	return bool_result(true);
}

Value* StartSceneChangeTracking_cf(Value**, int);
Primitive StartSceneChangeTracking_pf(_M("Datasmith_StartSceneChangeTracking"), StartSceneChangeTracking_cf);

Value* StartSceneChangeTracking_cf(Value** arg_list, int count)
{
	check_arg_count(StartSceneChangeTracking, 0, count);

	GetExporter()->StartSceneChangeTracking();

	return bool_result(true);
}

Value* DirectLinkInitializeForScene_cf(Value** arg_list, int count)
{
	check_arg_count(DirectLinkInitializeForScene, 0, count);

	GetExporter()->InitializeDirectLinkForScene();

	return bool_result(true);
}
Primitive DirectLinkInitializeForScene_pf(_M("Datasmith_DirectLinkInitializeForScene"), DirectLinkInitializeForScene_cf);


Value* DirectLinkUpdateScene_cf(Value** arg_list, int count)
{
	check_arg_count(DirectLinkUpdateScene, 0, count);
	LogDebug(TEXT("DirectLink::UpdateScene: start"));
	GetExporter()->UpdateDirectLinkScene();
	LogDebug(TEXT("DirectLink::UpdateScene: done"));

	return bool_result(true);
}
Primitive DirectLinkUpdateScene_pf(_M("Datasmith_DirectLinkUpdateScene"), DirectLinkUpdateScene_cf);

Value* ToggleAutoSync_cf(Value** arg_list, int count) 
{
	check_arg_count(ToggleAutoSync, 0, count);

	if (GetExporter())
	{
		return bool_result(GetExporter()->ToggleAutoSync());
	}

	return &false_value;
}
Primitive ToggleAutoSync_pf(_M("Datasmith_ToggleAutoSync"), ToggleAutoSync_cf);

Value* IsAutoSyncEnabled_cf(Value** arg_list, int count) 
{
	check_arg_count(IsAutoSyncEnabled, 0, count);

	if (GetExporter())
	{
		return bool_result(GetExporter()->IsAutoSyncEnabled());
	}

	return &false_value;
}
Primitive IsAutoSyncEnabled_pf(_M("Datasmith_IsAutoSyncEnabled"), IsAutoSyncEnabled_cf);

Value* SetAutoSyncDelay_cf(Value** arg_list, int count) 
{
	check_arg_count(SetAutoSyncDelay, 1, count);

	if (GetExporter())
	{
		GetExporter()->SetAutoSyncDelay(arg_list[0]->to_float());
		return &true_value;
	}

	return &false_value;
}
Primitive SetAutoSyncDelay_pf(_M("Datasmith_SetAutoSyncDelay"), SetAutoSyncDelay_cf);

Value* SetAutoSyncIdleDelay_cf(Value** arg_list, int count) 
{
	check_arg_count(SetAutoSyncIdleDelay, 1, count);

	if (GetExporter())
	{
		GetExporter()->SetAutoSyncIdleDelay(arg_list[0]->to_float());
		return &true_value;
	}

	return &false_value;
}
Primitive SetAutoSyncIdleDelay_pf(_M("Datasmith_SetAutoSyncIdleDelay"), SetAutoSyncIdleDelay_cf);

Value* OpenDirectlinkUi_cf(Value** arg_list, int count) 
{
	check_arg_count(OpenDirectlinkUi, 0, count);

	return bool_result(OpenDirectLinkUI());
}
Primitive OpenDirectlinkUi_pf(_M("Datasmith_OpenDirectlinkUi"), OpenDirectlinkUi_cf);

#define DefinePersistentExportOption(name) \
Value* GetExportOption_##name##_cf(Value** arg_list, int count) \
{ \
	check_arg_count(GetExportOption_##name, 0, count); \
 \
	return bool_result(GetPersistentExportOptions().Get##name##()); \
} \
Primitive GetExportOption_##name##_pf(_M("Datasmith_GetExportOption_" #name), GetExportOption_##name##_cf);\
\
Value* SetExportOption_##name##_cf(Value** arg_list, int count) \
{ \
	check_arg_count(SetExportOption_##name, 1, count); \
	Value* pValue= arg_list[0]; \
	GetPersistentExportOptions().Set##name##(pValue->to_bool()); \
	return &true_value; \
} \
Primitive SetExportOption_##name##_pf(_M("Datasmith_SetExportOption_" #name), SetExportOption_##name##_cf);\

#define DefinePersistentExportOptionInt(name) \
Value* GetExportOption_##name##_cf(Value** arg_list, int count) \
{ \
	check_arg_count(GetExportOption_##name, 0, count); \
 \
	one_value_local(result); \
	vl.result = Integer::intern(GetPersistentExportOptions().Get##name##()); \
	return_value(vl.result); \
} \
Primitive GetExportOption_##name##_pf(_M("Datasmith_GetExportOption_" #name), GetExportOption_##name##_cf);\
\
Value* SetExportOption_##name##_cf(Value** arg_list, int count) \
{ \
	check_arg_count(SetExportOption_##name, 1, count); \
	Value* pValue= arg_list[0]; \
	GetPersistentExportOptions().Set##name##(pValue->to_int()); \
	return &true_value; \
} \
Primitive SetExportOption_##name##_pf(_M("Datasmith_SetExportOption_" #name), SetExportOption_##name##_cf);\

//////////////////////////////////////////
DefinePersistentExportOption(AnimatedTransforms)

DefinePersistentExportOption(StatSync)
DefinePersistentExportOptionInt(TextureResolution)
DefinePersistentExportOption(XRefScenes)
//////////////////////////////////////////

#undef DefinePersistentExportOption
#undef DefinePersistentExportOptionInt

Value* GetDirectlinkCacheDirectory_cf(Value** arg_list, int count)
{
	check_arg_count(GetDirectlinkCacheDirectory, 0, count);
	const TCHAR* Path = GetDirectlinkCacheDirectory();
	if (Path)
	{
		return new String(Path);
	}

	return &undefined;
}

Primitive GetDirectlinkCacheDirectory_pf(_M("Datasmith_GetDirectlinkCacheDirectory"), GetDirectlinkCacheDirectory_cf);


Value* LogFlush_cf(Value** arg_list, int count)
{
	LogFlush();
	return &undefined;
}

Primitive LogFlush_pf(_M("Datasmith_LogFlush"), LogFlush_cf);


Value* Crash_cf(Value** arg_list, int count)
{
	volatile int* P;
	P = nullptr;

	*P = 666;
	return &undefined;
}

Primitive Crash_pf(_M("Datasmith_Crash"), Crash_cf);

Value* LogInfo_cf(Value** arg_list, int count)
{
	check_arg_count(LogInfo, 1, count);
	Value* Message = arg_list[0];

	LogInfo(Message->to_string());

	return bool_result(true);
}
Primitive LogInfo_pf(_M("Datasmith_LogInfo"), LogInfo_cf);

Value* LogWarning_cf(Value** arg_list, int count)
{
	check_arg_count(LogWarning, 1, count);
	Value* Message = arg_list[0];

	LogWarning(Message->to_string());

	return bool_result(true);
}
Primitive LogWarning_pf(_M("Datasmith_LogWarning"), LogWarning_cf);

Value* LogError_cf(Value** arg_list, int count)
{
	check_arg_count(LogError, 1, count);
	Value* Message = arg_list[0];

	LogError(Message->to_string());

	return bool_result(true);
}
Primitive LogError_pf(_M("Datasmith_LogError"), LogError_cf);

Value* LogCompletion_cf(Value** arg_list, int count)
{
	check_arg_count(LogCompletion, 1, count);
	Value* Message = arg_list[0];

	LogCompletion(Message->to_string());

	return bool_result(true);
}
Primitive LogCompletion_pf(_M("Datasmith_LogCompletion"), LogCompletion_cf);


class FMessagesDialog
{
public:

	~FMessagesDialog()
	{
		ensure(false);
		if (Thread.IsValid())
		{
			ThreadEvent->Trigger();
			Thread.Get();
		}
	}

	static INT_PTR CALLBACK DlgProc(HWND hDlg, UINT iMsg, WPARAM wParam, LPARAM lParam)
	{
		switch (iMsg)
		{
			case WM_INITDIALOG:
			{
				return TRUE;
			}

			case WM_CLOSE:
			{
				ShowWindow(hDlg, SW_HIDE);
				return FALSE;
			}
		};
		return FALSE;
	}

	void PumpWindowMessages()
	{
		MSG Message;
		// standard Windows message handling
		while(PeekMessage(&Message, NULL, 0, 0, PM_REMOVE))
		{
			if (!IsDialogMessage(DialogHwnd, &Message))
			{
				TranslateMessage(&Message);
				DispatchMessage(&Message);
			}
		}
	}

	void ProcessLogMessages()
	{
		if (Messages.IsEmpty())
		{
			return;
		}

		FString Message;
		while(Messages.Dequeue(Message))
		{
			SendDlgItemMessage(DialogHwnd, IDC_ERROR_MSG_LIST, LB_ADDSTRING, NULL, reinterpret_cast<LPARAM>(*Message));
		}

		// Scroll message list to bottom
		LRESULT ItemCount = SendDlgItemMessage(DialogHwnd, IDC_ERROR_MSG_LIST, LB_GETCOUNT,NULL,NULL);
		SendDlgItemMessage(DialogHwnd, IDC_ERROR_MSG_LIST, LB_SETTOPINDEX,ItemCount-1,NULL);
	}

	void Toggle()
	{
		if (!bDialogCreated)
		{
			Thread = Async(EAsyncExecution::Thread,
				[&, this]
				{
					DialogHwnd = CreateDialogParam(HInstanceMax, MAKEINTRESOURCE(IDD_ERROR_MSGS), GetCOREInterface()->GetMAXHWnd(), DlgProc, reinterpret_cast<LPARAM>(this));
					while (true)
					{
						PumpWindowMessages();

						ProcessLogMessages();

						if (ThreadEvent->Wait(FTimespan::FromMilliseconds(10)))
						{
							DestroyWindow(DialogHwnd);
							break;
						}
					}
				}
			);
			bDialogCreated = true;
		}
		else
		{
			ShowWindow(DialogHwnd, IsWindowVisible(DialogHwnd) ? SW_HIDE : SW_SHOW);
		}
	}

	void AddWarning(const TCHAR* Message)
	{
		Messages.Enqueue(FString(TEXT("WARNING:")) + Message);
	}

	void AddInfo(const TCHAR* Message)
	{
		Messages.Enqueue(Message);
	}

	bool bDialogCreated = false;
	HWND DialogHwnd = NULL;
	FEventRef ThreadEvent;
	TFuture<void> Thread;

	TQueue<FString> Messages;
};

void LogErrorDialog(const FString& Msg)
{
	MessagesDialog->AddError(Msg);
}

void LogWarningDialog(const FString& Msg)
{
	MessagesDialog->AddWarning(Msg);
}

void LogCompletionDialog(const FString& Msg)
{
	MessagesDialog->AddCompletion(Msg);
}

void LogInfoDialog(const FString& Msg)
{
	MessagesDialog->AddInfo(Msg);
}

void LogDebugDialog(const FString& Msg)
{
	MessagesDialog->AddInfo(Msg);
}

// Setup ActionTable with Datasmith commands exposed as actions
class FDatasmithActions
{
public:
	static const ActionTableId ActionTableId = 0x291356d8;
	static const ActionContextId ActionContextId = 0x291356d9;

	enum EActionIds
	{
		ID_SYNC_ACTION_ID,
		ID_AUTOSYNC_ACTION_ID,
		ID_CONNECTIONS_ACTION_ID,
		ID_EXPORT_ACTION_ID,
		ID_SHOWLOG_ACTION_ID,
		ID_EXPORT_SELECTED_ACTION_ID,
	};

	class FDatasmithActionTable : public ActionTable
	{
	public:
		explicit FDatasmithActionTable(MSTR& Name): ActionTable(ActionTableId, ActionContextId, Name)
		{
			
		}

		BOOL IsChecked(int ActionId)
		{
			switch (ActionId)
			{
			case ID_AUTOSYNC_ACTION_ID: return GetExporter() && GetExporter()->IsAutoSyncEnabled(); // Can't change AutoSync icon but able to set its state to Checked when it's on
			};
			return false;
		}

		TMap<int, TUniquePtr<MaxBmpFileIcon>> IconForAction;

		MaxIcon* GetIcon(int ActionId) override
		{
			TUniquePtr<MaxBmpFileIcon>& Icon = IconForAction.FindOrAdd(ActionId);

			if (Icon)
			{
				return Icon.Get();
			}

			switch (ActionId)
			{
			case ID_SYNC_ACTION_ID:
			{
				Icon.Reset(new MaxBmpFileIcon(_T(":/Datasmith/Icons/DatasmithSyncIcon")));
				break;
			}
			case ID_AUTOSYNC_ACTION_ID:
			{
				Icon.Reset(new MaxBmpFileIcon(_T(":/Datasmith/Icons/DatasmithAutoSyncIconOn")));
				break;
			}
			case ID_CONNECTIONS_ACTION_ID:
			{
				Icon.Reset(new MaxBmpFileIcon(_T(":/Datasmith/Icons/DatasmithManageConnectionsIcon")));
				break;
			}
			case ID_EXPORT_ACTION_ID:
			{
				Icon.Reset(new MaxBmpFileIcon(_T(":/Datasmith/Icons/DatasmithIcon")));
				break;
			}
			case ID_SHOWLOG_ACTION_ID:
			{
				Icon.Reset(new MaxBmpFileIcon(_T(":/Datasmith/Icons/DatasmithLogIcon")));
				break;
			}
			case ID_EXPORT_SELECTED_ACTION_ID:
			{
				Icon.Reset(new MaxBmpFileIcon(_T(":/Datasmith/Icons/DatasmithIcon")));
				break;
			}
			}
			return Icon.Get();
		}
	};

	class FDatasmithActionCallback : public ActionCallback
	{
	public:

		BOOL ExecuteAction (int ActionId)
		{
			LogDebug(FString::Printf(TEXT("Action: %d"), ActionId));

			switch (ActionId)
			{
			case ID_SYNC_ACTION_ID:
			{
				if (GetExporter())
				{
					GetExporter()->PerformSync(false);
				}
				return true;
			}
			case ID_AUTOSYNC_ACTION_ID:
			{
				if (GetExporter())
				{
					GetExporter()->ToggleAutoSync();
				}
				return true;
			}
			case ID_CONNECTIONS_ACTION_ID:
			{
				OpenDirectLinkUI();
				return true;
			}
			case ID_EXPORT_ACTION_ID:
			{
				if (GetExporter())
				{
					const MCHAR* ScriptCode = _T("Datasmith_ExportDialog selected:false");
#if MAX_PRODUCT_YEAR_NUMBER >= 2022
					 ExecuteMAXScriptScript(ScriptCode, MAXScript::ScriptSource::NonEmbedded);
#else
					 ExecuteMAXScriptScript(ScriptCode);
#endif
				}
				return true;
			}
			case ID_SHOWLOG_ACTION_ID:
			{
				MessagesDialog->OpenWindow();
				return true;
			}
			case ID_EXPORT_SELECTED_ACTION_ID:
			{
				if (GetExporter())
				{
					const MCHAR* ScriptCode = _T("Datasmith_ExportDialog selected:true");
#if MAX_PRODUCT_YEAR_NUMBER >= 2022
					 ExecuteMAXScriptScript(ScriptCode, MAXScript::ScriptSource::NonEmbedded);
#else
					 ExecuteMAXScriptScript(ScriptCode);
#endif
				}
				return true;
			}
			}
		    return false;
		}
	};

	FDatasmithActions()
		: Name(TEXT("Datasmith"))
	{
		// todo: localization of Name

#define DATASMITH_ACTION(name) \
			ID_##name##_ACTION_ID, \
			IDS_##name##_DESC, \
			IDS_##name##_NAME, \
			IDS_DATASMITH_CATEGORY 

		static ActionDescription ActionsDescriptions[] = {
			DATASMITH_ACTION(SYNC),
			DATASMITH_ACTION(AUTOSYNC),
			DATASMITH_ACTION(CONNECTIONS),
			DATASMITH_ACTION(EXPORT),
			DATASMITH_ACTION(SHOWLOG),
			DATASMITH_ACTION(EXPORT_SELECTED),
		};

		FDatasmithActionTable* Table = new FDatasmithActionTable(Name); // Table, registered with RegisterActionTable will be deallocated by Max

		Table->BuildActionTable(nullptr, sizeof(ActionsDescriptions) / sizeof(ActionsDescriptions[0]), ActionsDescriptions, HInstanceMax);
		GetCOREInterface()->GetActionManager()->RegisterActionContext(ActionContextId, Name.data());

		// Register table - this needs to be called explicitly when action table is not returned to Max with ClassDesc's GetActionTable method
		GetCOREInterface()->GetActionManager()->RegisterActionTable(Table); 

		GetCOREInterface()->GetActionManager()->ActivateActionTable(&ActionCallback, ActionTableId);
	}

private:
	TSTR Name;
	FDatasmithActionCallback ActionCallback;
};


TUniquePtr<FDatasmithActions> Actions;

void ShutdownScripts()
{
	Actions.Reset();
}

Value* SetupActions_cf(Value** arg_list, int count)
{
	if (!Actions)
	{
		Actions = MakeUnique<FDatasmithActions>();
	}

	return bool_result(true);
}
Primitive SetupActions_pf(_M("Datasmith_SetupActions"), SetupActions_cf);

}

#include "Windows/HideWindowsPlatformTypes.h"

#pragma warning( pop )

#endif // NEW_DIRECTLINK_PLUGIN
