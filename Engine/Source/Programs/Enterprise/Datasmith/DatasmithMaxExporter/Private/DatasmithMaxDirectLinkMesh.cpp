// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef NEW_DIRECTLINK_PLUGIN

#include "DatasmithMaxDirectLink.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithMesh.h"
#include "DatasmithUtils.h"
#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxMeshExporter.h"
#include "DatasmithMaxSceneExporter.h"
#include "DatasmithMaxAttributes.h"
#include "DatasmithMaxSceneHelper.h"
#include "DatasmithMaxLogger.h"

#include "Logging/LogMacros.h"


#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "max.h"

	#include "modstack.h"
	#include "iparamb2.h"
	#include "MeshNormalSpec.h"
MAX_INCLUDES_END


namespace DatasmithMaxDirectLink
{
namespace GeomUtils
{

class FNullView : public View
{
public:
	FNullView()
	{
		worldToView.IdentityMatrix(); screenW = 640.0f; screenH = 480.0f;
	}

	virtual Point2 ViewToScreen(Point3 p) override
	{
		return Point2(p.x, p.y);
	}
};

Object* GetBaseObject(INode* Node, TimeValue Time);
int SetObjectParamValue(TimeValue CurrentTime, Object *Obj, const FString& ParamName, int DesiredValue);

FRenderMeshForConversion GetMeshForGeomObject(TimeValue CurrentTime, INode* Node, const FTransform& Pivot)
{
	// todo: baseline exporter uses GetBaseObject which takes result of EvalWorldState
	// and searched down DerivedObject pipeline(by taking GetObjRef) 
	// This is STRANGE as EvalWorldState shouldn't return DerivedObject in the first place(it should return result of pipeline evaluation)

	BOOL bNeedsDelete = false;
	Interval ValidityInterval;
	ValidityInterval.SetInfinite();

	TriObject* ObjectToDelete = nullptr;

	Mesh* RenderMesh = nullptr;

	{
		Object* Obj = GetBaseObject(Node, CurrentTime);

		// Read validity interval before changing display settings
		// Else VRay will somehow react to a call to ObjectValidity and change RenderMesh contents to what it was originally.
		// E.g. display was set 'Box', we change display to 'Mesh' here before calling GetRenderMesh to retrieve actual mesh later(rather than a simple box)
		//  but if ObjectValidity is called after GetRenderMesh that RenderMesh will become box again!
		Interval ObjectValidity = Obj->ObjectValidity(CurrentTime);

		const Class_ID& ObjectClassID = Obj->ClassID();
		const FString VRayProxyParamName(TEXT("display"));
		const FString BodyObjectViewportMeshParamName(TEXT("RenderViewportMeshRA"));
		int PreviousMeshDisplayValue = 0;

		const int VRAY_PROXY_DISPLAY_AS_MESH = 4; // Value to set on a VRay Mesh Proxy to get the mesh
		if (ObjectClassID == VRAYPROXY_CLASS_ID)
		{
			// Need the high resolution render mesh associated with the VRay Mesh Proxy for the export
			PreviousMeshDisplayValue = SetObjectParamValue(CurrentTime, Obj, VRayProxyParamName, VRAY_PROXY_DISPLAY_AS_MESH);
		}
		else if(ObjectClassID == BODYOBJECT_CLASS_ID)
		{
			// Need to make sure we are using the viewport mesh on BodyObject, otherwise the RenderMesh gives a tessellated low resolution mesh.
			PreviousMeshDisplayValue = SetObjectParamValue(CurrentTime, Obj, BodyObjectViewportMeshParamName, 1);
		}

		GeomObject* GeomObj = static_cast<GeomObject*>(Obj);
		if (GeomObj)
		{
			if (GeomObj->CanConvertToType(Class_ID(TRIOBJ_CLASS_ID, 0)))
			{
				// GeomObj->GetRenderMesh for some objects types returns low quality mesh no matter settings
				// Example: a body cutter object, its GetRenderMesh gives away a coarse mesh even when its viewport settings are set to Fine(and it's displayed Fine in viewport)
				// Looks like TriObject should be used in the first place as a source of best quality mesh

				TriObject* TriObj = static_cast<TriObject*>(GeomObj->ConvertToType(CurrentTime, Class_ID(TRIOBJ_CLASS_ID, 0)));

				if (TriObj != GeomObj)
				{
					// In case tri object is a different object from node's GeomObj then use it to get mesh
					RenderMesh = &(TriObj->mesh);
					bNeedsDelete = false; // Mesh doesn't need to be deleted - TriObject does

					check(TriObj != GeomObj); // Extra check that we won't delete live object 
					ObjectToDelete = TriObj; // When converted tri object is different from th eobject itself it needs to be disposed
				}
			}

			if (!RenderMesh)  // Fallback to GetRenderMesh if TriObject fails
			{
				FNullView View;
				RenderMesh = GeomObj->GetRenderMesh(CurrentTime, Node, View, bNeedsDelete);
			}
		}

		// Restore display state if different from mesh display
		if (ObjectClassID == VRAYPROXY_CLASS_ID && PreviousMeshDisplayValue != VRAY_PROXY_DISPLAY_AS_MESH)
		{
			SetObjectParamValue(CurrentTime, Obj, VRayProxyParamName, PreviousMeshDisplayValue);
		}
		else if(ObjectClassID == BODYOBJECT_CLASS_ID && PreviousMeshDisplayValue != 1)
		{
			SetObjectParamValue(CurrentTime, Obj, BodyObjectViewportMeshParamName, PreviousMeshDisplayValue);
		}

		if (RenderMesh)
		{
			ValidityInterval &= ObjectValidity; // Update validity only when actual mesh is returned
		}
	}

	FRenderMeshForConversion Result(Node, RenderMesh, bNeedsDelete, ObjectToDelete);
	Result.SetValidityInterval(ValidityInterval);
	Result.SetPivot(Pivot);
	return MoveTemp(Result);
}

/** Convert Max to UE coordinates, handle scene master unit
 * @param MaxTransform      source transform
 * @param UnitMultiplier    Master scene unit
 */
FTransform FTransformFromMatrix3(const Matrix3& MaxTransform, float UnitMultiplier)
{
	FVector Translation;
	FQuat Rotation;
	FVector Scale;
	FDatasmithMaxSceneExporter::MaxToUnrealCoordinates( MaxTransform, Translation, Rotation, Scale, UnitMultiplier);
	return FTransform( Rotation, Translation, Scale );
}

INode* GetCollisionNode(ISceneTracker& SceneTracker, INode* Node, const FDatasmithMaxStaticMeshAttributes* DatasmithAttributes, bool& bOutFromDatasmithAttribute)
{
	if (DatasmithAttributes)
	{
		INode* ModifierSpecifiedCustomCollisionNode = DatasmithAttributes->GetCustomCollisonNode();
		if (ModifierSpecifiedCustomCollisionNode)
		{
			bOutFromDatasmithAttribute = true;
			return ModifierSpecifiedCustomCollisionNode;
		}
	}

	

	FString OriginalName = Node->GetName();
	for ( const FString& CollisionNodePrefix : FDatasmithMaxSceneHelper::CollisionNodesPrefixes )
	{
		if (FNodeTracker* CollisionNode = SceneTracker.GetNodeTrackerByNodeName(*( CollisionNodePrefix + TEXT("_") + OriginalName )))
		{
			bOutFromDatasmithAttribute = false;
			return CollisionNode->Node;
		}
	}

	return nullptr;
}


FRenderMeshForConversion GetMeshForCollision(TimeValue CurrentTime, ISceneTracker& SceneTracker, INode* Node, bool bBakePivot)
{
	// source: FDatasmithMaxMeshExporter::ExportMesh
	FDatasmithConverter Converter;
	bool bIsCollisionFromDatasmithAttributes;
	TOptional<FDatasmithMaxStaticMeshAttributes> DatasmithAttributes = FDatasmithMaxStaticMeshAttributes::ExtractStaticMeshAttributes(Node);
	INode* CollisionNode = GetCollisionNode(SceneTracker, Node, DatasmithAttributes ? &DatasmithAttributes.GetValue() : nullptr, bIsCollisionFromDatasmithAttributes);
	FTransform CollisionPivot;
	if (CollisionNode)
	{

		FTransform ColliderPivot = FDatasmithMaxSceneExporter::GetPivotTransform(CollisionNode, Converter.UnitToCentimeter);

		if (bIsCollisionFromDatasmithAttributes)
		{
			if (!bBakePivot)
			{
				FTransform RealPivot = FDatasmithMaxSceneExporter::GetPivotTransform(Node, Converter.UnitToCentimeter);
				ColliderPivot = ColliderPivot * RealPivot.Inverse();
			}
			CollisionPivot = ColliderPivot;
		}
		else
		{
			FTransform FTransformFromMatrix3(const Matrix3& MaxTransform, float UnitMultiplier); // todo: move to header and rename(F is for class!)
			FTransform NodeWTM = FTransformFromMatrix3(Node->GetNodeTM(CurrentTime), Converter.UnitToCentimeter);
			FTransform ColliderNodeWTM = FTransformFromMatrix3(CollisionNode->GetNodeTM(CurrentTime), Converter.UnitToCentimeter);

			// if object-offset has been baked into the mesh data, we want collision mesh data in the mesh's node space
			//   MeshVert_Node = RealPivot * max_vert_data
			//   MeshVert_world = NodeWTM * MeshVert_Node
			// Collision mesh vertices in world space:
			//   CollVert_node = ColliderPivot * CollVert_obj
			//   CollVert_world = ColliderNodeWTM * CollVert_node
			//   CollVert_mesh = NodeWTM-1 * CollVert_world
			FTransform BakedTransform = ColliderPivot * ColliderNodeWTM * NodeWTM.Inverse();

			if (!bBakePivot)
			{
				// if object-offset has not been baked, we want collision mesh data in the mesh's object space
				FTransform RealPivot = FDatasmithMaxSceneExporter::GetPivotTransform(Node, Converter.UnitToCentimeter);
				BakedTransform = BakedTransform * RealPivot.Inverse();
			}

			CollisionPivot = BakedTransform;
		}
	}
	if (!CollisionNode)
	{
		return FRenderMeshForConversion();
	}

	return GetMeshForGeomObject(CurrentTime, CollisionNode, CollisionPivot);
}



// todo: copied from baseline plugin(it has dependencies on converters that are not static in FDatasmithMaxMeshExporter)
void FillDatasmithMeshFromMaxMesh(FDatasmithMesh& DatasmithMesh, Mesh& MaxMesh, INode* ExportedNode, bool bForceSingleMat, TSet<uint16>& SupportedChannels, TMap<int32, int32>& UVChannelsMap, FTransform Pivot)
{
	FDatasmithConverter Converter;

	const int NumFaces = MaxMesh.getNumFaces();
	const int NumVerts = MaxMesh.getNumVerts();

	DatasmithMesh.SetVerticesCount(NumVerts);
	DatasmithMesh.SetFacesCount(NumFaces);

	// Vertices
	for (int i = 0; i < NumVerts; i++)
	{
		Point3 Point = MaxMesh.getVert(i);

		FVector Vertex = Converter.toDatasmithVector(Point);
		Vertex = Pivot.TransformPosition(Vertex); // Bake object-offset in the mesh data when possible

		DatasmithMesh.SetVertex(i, Vertex.X, Vertex.Y, Vertex.Z);
	}

	// Vertex Colors
	if (MaxMesh.curVCChan == 0 && MaxMesh.numCVerts > 0)
	{
		// Default vertex color channel
		for (int32 i = 0; i < NumFaces; i++)
		{
			TVFace& Face = MaxMesh.vcFace[i];
			DatasmithMesh.SetVertexColor(i * 3, Converter.toDatasmithColor(MaxMesh.vertCol[Face.t[0]]));
			DatasmithMesh.SetVertexColor(i * 3 + 1, Converter.toDatasmithColor(MaxMesh.vertCol[Face.t[1]]));
			DatasmithMesh.SetVertexColor(i * 3 + 2, Converter.toDatasmithColor(MaxMesh.vertCol[Face.t[2]]));
		}
	}

	// UVs
	TMap<uint32, int32> HashToChannel;
	bool bIsFirstUVChannelValid = true;

	for (int32 i = 1; i <= MaxMesh.getNumMaps(); ++i)
	{
		if (MaxMesh.mapSupport(i) == BOOL(true) && MaxMesh.getNumMapVerts(i) > 0)
		{
			DatasmithMesh.AddUVChannel();
			const int32 UVChannelIndex = DatasmithMesh.GetUVChannelsCount() - 1;
			const int32 UVsCount = MaxMesh.getNumMapVerts(i);

			DatasmithMesh.SetUVCount(UVChannelIndex, UVsCount);

			UVVert* Vertex = MaxMesh.mapVerts(i);

			for (int32 j = 0; j < UVsCount; ++j)
			{
				const UVVert& MaxUV = Vertex[j];
				DatasmithMesh.SetUV(UVChannelIndex, j, MaxUV.x, 1.f - MaxUV.y);
			}

			TVFace* Faces = MaxMesh.mapFaces(i);
			for (int32 j = 0; j < MaxMesh.getNumFaces(); ++j)
			{
				DatasmithMesh.SetFaceUV(j, UVChannelIndex, Faces[j].t[0], Faces[j].t[1], Faces[j].t[2]);
			}

			if (UVChannelIndex == 0)
			{
				//Verifying that the UVs are properly unfolded, which is required to calculate the tangent in unreal.
				bIsFirstUVChannelValid = FDatasmithMeshUtils::IsUVChannelValid(DatasmithMesh, UVChannelIndex);
			}

			uint32 Hash = DatasmithMesh.GetHashForUVChannel(UVChannelIndex);
			int32* PointerToChannel = HashToChannel.Find(Hash);

			if (PointerToChannel)
			{
				// Remove the channel because there is another one that is identical
				DatasmithMesh.RemoveUVChannel();

				// Map the user-specified UV Channel (in 3dsmax) to the actual UV channel that will be exported to Unreal
				UVChannelsMap.Add(i - 1, *PointerToChannel);
			}
			else
			{
				// Map the user-specified UV Channel (in 3dsmax) to the actual UV channel that will be exported to Unreal
				UVChannelsMap.Add(i - 1, UVChannelIndex);
				HashToChannel.Add(Hash, UVChannelIndex);
			}
		}
	}

	if (!bIsFirstUVChannelValid)
	{
		LogWarning(FString::Printf(TEXT("%s's UV channel #1 contains degenerated triangles, this can cause issues in Unreal. It is recommended to properly unfold and flatten exported UV data.")
			, static_cast<const TCHAR*>(ExportedNode->GetName())));
	}

	// Faces
	for (int i = 0; i < NumFaces; i++)
	{
		// Create polygons. Assign texture and texture UV indices.
		// all faces of the cube have the same texture

		Face& MaxFace = MaxMesh.faces[i];
		int MaterialId = bForceSingleMat ? 0 : MaxFace.getMatID();

		SupportedChannels.Add(MaterialId);

		//Max's channel UI is not zero-based, so we register an incremented ChannelID for better visual consistency after importing in Unreal.
		DatasmithMesh.SetFace(i, MaxFace.getVert(0), MaxFace.getVert(1), MaxFace.getVert(2), MaterialId + 1);
		DatasmithMesh.SetFaceSmoothingMask(i, (uint32)MaxFace.getSmGroup());
	}

	//Normals

	MaxMesh.SpecifyNormals();
	MeshNormalSpec* Normal = MaxMesh.GetSpecifiedNormals();
	Normal->MakeNormalsExplicit(false);
	Normal->CheckNormals();

	Matrix3 RotationMatrix;
	RotationMatrix.IdentityMatrix();
	Quat ObjectOffsetRotation = ExportedNode->GetObjOffsetRot();
	RotateMatrix(RotationMatrix, ObjectOffsetRotation);

	Point3 Point;

	for (int i = 0; i < NumFaces; i++)
	{
		Point = Normal->GetNormal(i, 0).Normalize() * RotationMatrix;
		FVector NormalVector = Converter.toDatasmithVector(Point);
		DatasmithMesh.SetNormal(i * 3, NormalVector.X, NormalVector.Y, NormalVector.Z);

		Point = Normal->GetNormal(i, 1).Normalize() * RotationMatrix;
		NormalVector = Converter.toDatasmithVector(Point);
		DatasmithMesh.SetNormal(i * 3 + 1, NormalVector.X, NormalVector.Y, NormalVector.Z);

		Point = Normal->GetNormal(i, 2).Normalize() * RotationMatrix;
		NormalVector = Converter.toDatasmithVector(Point);
		DatasmithMesh.SetNormal(i * 3 + 2, NormalVector.X, NormalVector.Y, NormalVector.Z);
	}
}

void SetNormalForAFace(FDatasmithMesh& DatasmithMesh, int32 Index, const FVector& NormalVector)
{
	DatasmithMesh.SetNormal(Index * 3, NormalVector.X, NormalVector.Y, NormalVector.Z);
	DatasmithMesh.SetNormal(Index * 3 + 1, NormalVector.X, NormalVector.Y, NormalVector.Z);
	DatasmithMesh.SetNormal(Index * 3 + 2, NormalVector.X, NormalVector.Y, NormalVector.Z);
}

bool FillDatasmithMeshFromBoundingBox(TimeValue CurrentTime, FDatasmithMesh& DatasmithMesh, const GeomUtils::FRenderMeshForConversion& MaxMesh)
{
	if (!MaxMesh.GetNode())
	{
		return false;
	}
	FDatasmithConverter Converter;

	if (!MaxMesh.IsValid())
	{
		return false;
	}

	if (MaxMesh.GetMesh()->getNumFaces() == 0)
	{
		return false;
	}

	if (MaxMesh.GetMesh()->getBoundingBox().IsEmpty())
	{
		MaxMesh.GetMesh()->buildBoundingBox();
	}

	Box3 BoundingBox = MaxMesh.GetMesh()->getBoundingBox();

	DatasmithMesh.SetVerticesCount(8);
	DatasmithMesh.SetFacesCount(12);

	for (int32 i = 0; i < 8; i++)
	{
		FVector Vertex = Converter.toDatasmithVector(BoundingBox[i]);
		Vertex = MaxMesh.GetPivot().TransformPosition(Vertex);
		DatasmithMesh.SetVertex(i, Vertex.X, Vertex.Y, Vertex.Z);
	}

	// Make a cube from the vertices

	Matrix3 RotationMatrix;
	RotationMatrix.IdentityMatrix();
	Quat ObjectOffsetRotation = MaxMesh.GetNode()->GetObjOffsetRot();
	RotateMatrix(RotationMatrix, ObjectOffsetRotation);
	Point3 Point;
	FVector NormalVector;

	// Points toward negative z (without a transform and in max coordinates)
	DatasmithMesh.SetFace(0, 1, 0, 2);
	DatasmithMesh.SetFace(1, 1, 2, 3);
	Point = Point3(0.f, 0.f, -1.f) * RotationMatrix;
	NormalVector = Converter.toDatasmithNormal(Point);
	SetNormalForAFace(DatasmithMesh, 0, NormalVector);
	SetNormalForAFace(DatasmithMesh, 1, NormalVector);

	// Points toward positive y (without a transform and in max coordinates)
	DatasmithMesh.SetFace(2, 3, 2, 7);
	DatasmithMesh.SetFace(3, 2, 6, 7);
	Point = Point3(0.f, 1.f, 0.f) * RotationMatrix;
	NormalVector = Converter.toDatasmithNormal(Point);
	SetNormalForAFace(DatasmithMesh, 2, NormalVector);
	SetNormalForAFace(DatasmithMesh, 3, NormalVector);
	
	// Points toward positive x (without a transform and in max coordinates)
	DatasmithMesh.SetFace(4, 1, 3, 7);
	DatasmithMesh.SetFace(5, 5, 1, 7);
	Point = Point3(1.f, 0.f, 0.f) * RotationMatrix;
	NormalVector = Converter.toDatasmithNormal(Point);
	SetNormalForAFace(DatasmithMesh, 4, NormalVector);
	SetNormalForAFace(DatasmithMesh, 5, NormalVector);

	// Points toward negative x (without a transform and in max coordinates)
	DatasmithMesh.SetFace(6, 2, 0, 6);
	DatasmithMesh.SetFace(7, 0, 4, 6);
	Point = Point3(-1.f, 0.f, 0.f) * RotationMatrix;
	NormalVector = Converter.toDatasmithNormal(Point);
	SetNormalForAFace(DatasmithMesh, 6, NormalVector);
	SetNormalForAFace(DatasmithMesh, 7, NormalVector);

	// Points toward negative y (without a transform and in max coordinates)
	DatasmithMesh.SetFace(8, 0, 1, 5);
	DatasmithMesh.SetFace(9, 4, 0, 5);
	Point = Point3(0.f, -1.f, 0.f) * RotationMatrix;
	NormalVector = Converter.toDatasmithNormal(Point);
	SetNormalForAFace(DatasmithMesh, 8, NormalVector);
	SetNormalForAFace(DatasmithMesh, 9, NormalVector);

	// Points toward positive z (without a transform and in max coordinates)
	DatasmithMesh.SetFace(10, 4, 5, 6);
	DatasmithMesh.SetFace(11, 6, 5, 7);
	Point = Point3(0.f, 0.f, 1.f) * RotationMatrix;
	NormalVector = Converter.toDatasmithNormal(Point);
	SetNormalForAFace(DatasmithMesh, 10, NormalVector);
	SetNormalForAFace(DatasmithMesh, 11, NormalVector);

	return true;
}


bool CreateDatasmithMeshFromMaxMesh(FDatasmithMesh& DatasmithMesh, const MeshConversionParams& Params, TSet<uint16>& SupportedChannels, TMap<int32, int32>& UVChannelsMap)
{
	bool bResult = false;
	if (Params.RenderMesh.GetMesh()->getNumFaces())
	{
		// Copy mesh to clean it before filling Datasmith mesh from it
		Mesh CachedMesh;
		CachedMesh.DeepCopy(Params.RenderMesh.GetMesh(), TOPO_CHANNEL | GEOM_CHANNEL | TEXMAP_CHANNEL | VERTCOLOR_CHANNEL);

		CachedMesh.DeleteIsoVerts();
		CachedMesh.RemoveDegenerateFaces();
		CachedMesh.RemoveIllegalFaces();

		// Need to invalidate/rebuild strips/edges after topology change(removing bad verts/faces)
		CachedMesh.InvalidateStrips();
		CachedMesh.BuildStripsAndEdges();

		if (CachedMesh.getNumFaces() > 0)
		{
			FillDatasmithMeshFromMaxMesh(DatasmithMesh, CachedMesh, Params.Node, Params.bConsolidateMaterialIds, SupportedChannels, UVChannelsMap, Params.RenderMesh.GetPivot());

			bResult = true; // Set to true, don't care what ExportToUObject does here - we need to move it to a thread anyway
		}
		CachedMesh.FreeAll();
	}
	return bResult;
}

Object* GetBaseObject(INode* Node, TimeValue Time)
{
	ObjectState ObjState = Node->EvalWorldState(Time);
	Object* Obj = ObjState.obj;

	if (Obj)
	{
		SClass_ID SuperClassID;
		SuperClassID = Obj->SuperClassID();
		while (SuperClassID == GEN_DERIVOB_CLASS_ID)
		{
			Obj = ((IDerivedObject*)Obj)->GetObjRef();
			SuperClassID = Obj->SuperClassID();
		}
	}

	return Obj;
}

int SetObjectParamValue(TimeValue CurrentTime, Object *Obj, const FString& ParamName, int DesiredValue)
{
	bool bFoundDisplayValue = false;
	int PrevDisplayValue = DesiredValue; // Display value to see mesh
	int NumParamBlocks = Obj->NumParamBlocks();
	for (short BlockIndex = 0; BlockIndex < NumParamBlocks && !bFoundDisplayValue; ++BlockIndex)
	{
		IParamBlock2* ParamBlock2 = Obj->GetParamBlockByID(BlockIndex);
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		for (int ParamIndex = 0; ParamIndex < ParamBlockDesc->count; ++ParamIndex)
		{
			ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[ParamIndex];
			if (FCString::Stricmp(ParamDefinition.int_name, *ParamName) == 0)
			{
				PrevDisplayValue = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime);
				if (PrevDisplayValue != DesiredValue)
				{
					ParamBlock2->SetValue(ParamDefinition.ID, CurrentTime, DesiredValue);
				}
				bFoundDisplayValue = true;
				break;
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	return PrevDisplayValue;
}

}
}

namespace DatasmithMaxDirectLink
{

template<typename T>
static void UpdateMD5SimpleType(FMD5& MD5, const T& Value)
{
	static_assert(TIsPODType<T>::Value, "Simple type required");
	MD5.Update(reinterpret_cast<const uint8*>(&Value), sizeof(Value));
}

template<typename T>
static void UpdateMD5Array(FMD5& MD5, TArray<T> Value)
{
	static_assert(TIsPODType<T>::Value, "This function requires POD array");
	UpdateMD5SimpleType(MD5, Value.Num());
	if (!Value.IsEmpty())
	{
		MD5.Update(reinterpret_cast<const uint8*>(Value.GetData()), Value.GetTypeSize()*Value.Num());
	}
}

static void UpdateMD5(FMD5& MD5, const FDatasmithMesh& DatasmithMesh)
{
	FMD5Hash MeshHash = DatasmithMesh.CalculateHash();
	MD5.Update(MeshHash.GetBytes(), MeshHash.GetSize());
}

FMD5Hash FDatasmithMeshConverter::ComputeHash()
{
	FMD5 MD5;

	UpdateMD5(MD5, RenderMesh);

	TArray<uint16> SupportedChannelsArray = SupportedChannels.Array();
	SupportedChannelsArray.Sort();
	UpdateMD5Array(MD5, SupportedChannelsArray);

	UpdateMD5SimpleType(MD5, UVChannelsMap.Num());

	if (!UVChannelsMap.IsEmpty())
	{
		TArray<TPair<int32, int32>> UVChannelsMapArray = UVChannelsMap.Array();
		UVChannelsMapArray.Sort();
		for (TPair<int32, int32> KVP : UVChannelsMapArray)
		{
			UpdateMD5SimpleType(MD5, KVP.Key);
			UpdateMD5SimpleType(MD5, KVP.Value);
		}
	}

	UpdateMD5SimpleType(MD5, SelectedLightmapUVChannel);

	UpdateMD5SimpleType(MD5, bHasCollision);
	if (bHasCollision)
	{
		UpdateMD5(MD5, CollisionMesh);
	}
	FMD5Hash Hash;
	Hash.Set(MD5);
	return Hash;
}


// todo: paralelize calls to ExportToUObject 
bool ConvertMaxMeshToDatasmith(TimeValue CurrentTime, FMeshConverterSource& MeshSource, FDatasmithMeshConverter& DatasmithMeshConverter)
{
	TOptional<FDatasmithMaxStaticMeshAttributes> DatasmithAttributes = FDatasmithMaxStaticMeshAttributes::ExtractStaticMeshAttributes(MeshSource.Node);

	if (DatasmithAttributes && DatasmithAttributes->GetExportMode() == EStaticMeshExportMode::BoundingBox)
	{
		if (GeomUtils::FillDatasmithMeshFromBoundingBox(CurrentTime, DatasmithMeshConverter.RenderMesh, MeshSource.RenderMesh))
		{
			return true;
		}
		else
		{
			LogWarning(FString(TEXT("Invalid object: ")) + MeshSource.Node->GetName());
			return false;
		}
	}

	MeshConversionParams RenderMeshParams = {
		MeshSource.RenderMesh.GetNode(),
		MeshSource.RenderMesh,
		MeshSource.bConsolidateMaterialIds
	};

	if (!GeomUtils::CreateDatasmithMeshFromMaxMesh(DatasmithMeshConverter.RenderMesh, RenderMeshParams, DatasmithMeshConverter.SupportedChannels, DatasmithMeshConverter.UVChannelsMap))
	{
		LogWarning(FString(TEXT("Invalid object: ")) + MeshSource.Node->GetName());
		return false;
	}

	// Mapping between the 3ds max channel and the exported mesh channel
	if (DatasmithAttributes)
	{
		DatasmithMeshConverter.SelectedLightmapUVChannel = DatasmithAttributes->GetLightmapUVChannel();
	}

	if (MeshSource.CollisionMesh.IsValid())
	{
		TSet<uint16> SupportedChannelsDummy; // ignore map channels for collision mesh
		TMap<int32, int32> UVChannelsMapDummy;

		MeshConversionParams CollisionParams = {
			MeshSource.CollisionMesh.GetNode(),
			MeshSource.CollisionMesh,
			true // Consolidate material ids into single mesh for collision
		};

		if (GeomUtils::CreateDatasmithMeshFromMaxMesh(DatasmithMeshConverter.CollisionMesh, CollisionParams, SupportedChannelsDummy, UVChannelsMapDummy))
		{
			DatasmithMeshConverter.bHasCollision = true;
		}
	}

	return true;
}

}


#include "Windows/HideWindowsPlatformTypes.h"

#endif // NEW_DIRECTLINK_PLUGIN
