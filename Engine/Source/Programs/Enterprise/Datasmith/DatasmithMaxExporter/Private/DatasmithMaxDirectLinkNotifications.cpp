// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef NEW_DIRECTLINK_PLUGIN

#include "DatasmithMaxDirectLink.h"


#include "Logging/LogMacros.h"


#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "max.h"

	#include "notify.h"

	#include "ISceneEventManager.h"
MAX_INCLUDES_END

namespace DatasmithMaxDirectLink
{

// This is used to handle some of change events
class FNodeEventCallback : public INodeEventCallback
{
	ISceneTracker& SceneTracker;

public:
	FNodeEventCallback(ISceneTracker& InSceneTracker) : SceneTracker(InSceneTracker)
	{
	}

	virtual BOOL VerboseDeleted() { return TRUE; }

	virtual void NameChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"NameChanged", nodes);

		for (int NodeIndex = 0; NodeIndex < nodes.Count(); ++NodeIndex)
		{
			SceneTracker.NodeNameChanged(nodes[NodeIndex]);
		}
	}

	virtual void LinkChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"LinkChanged", nodes);
		for (int NodeIndex = 0; NodeIndex < nodes.Count(); ++NodeIndex)
		{
			SceneTracker.NodeLinkChanged(nodes[NodeIndex]);
		}
	}

#if MAX_PRODUCT_YEAR_NUMBER < 2020  // Starting 2020 using ReferenceMaker to handle this notification
	virtual void UserPropertiesChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"UserPropertiesChanged", nodes);
		for (int NodeIndex = 0; NodeIndex < nodes.Count(); ++NodeIndex)
		{
			SceneTracker.NodePropertiesChanged(NodeEventNamespace::GetNodeByKey(nodes[NodeIndex]));
		}
	}
#endif

	// Not used:

	virtual void LayerChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"LayerChanged", nodes);
	}

	virtual void HideChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"HideChanged", nodes);
	}

	virtual void RenderPropertiesChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"RenderPropertiesChanged", nodes);
	}

	virtual void GeometryChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"GeometryChanged", nodes);
	}

	virtual void TopologyChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"TopologyChanged", nodes);
	}


	virtual void MappingChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"MappingChanged", nodes);
	}

	// Fired when node transform changes
	virtual void ControllerOtherEvent(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"ControllerOtherEvent", nodes);
	}

	// Tracks material assignment on node
	virtual void MaterialStructured(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"MaterialStructured", nodes);
	}

	// Tracks node's material parameter change(even if it's a submaterial of multimat that is assigned)
	virtual void MaterialOtherEvent(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"MaterialOtherEvent", nodes);
	}

	virtual void Added(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"Added", nodes);
	}

	virtual void Deleted(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"Deleted", nodes);
	}

	virtual void GroupChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"GroupChanged", nodes);
	}

	virtual void HierarchyOtherEvent(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"HierarchyOtherEvent", nodes);
	}

	virtual void ModelStructured(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"ModelStructured", nodes);
	}

	virtual void ExtentionChannelChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"ExtentionChannelChanged", nodes);
	}

	virtual void ModelOtherEvent(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"ModelOtherEvent", nodes);
	}

	virtual void ControllerStructured(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"ControllerStructured", nodes);
	}

	virtual void WireColorChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"WireColorChanged", nodes);
	}

	virtual void DisplayPropertiesChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"DisplayPropertiesChanged", nodes);
	}

	virtual void PropertiesOtherEvent(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"PropertiesOtherEvent", nodes);
	}

	virtual void SubobjectSelectionChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"SubobjectSelectionChanged", nodes);
	}

	virtual void SelectionChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"SelectionChanged", nodes);
	}

	virtual void FreezeChanged(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"FreezeChanged", nodes);
	}

	virtual void DisplayOtherEvent(NodeKeyTab& nodes) override
	{
		LogNodeEvent(L"DisplayOtherEvent", nodes);
	}

	virtual void CallbackBegin() override
	{

		LOG_DEBUG_HEAVY(L"NodeEventCallback: CallbackBegin\n");
	}

	virtual void CallbackEnd() override
	{
		LOG_DEBUG_HEAVY(L"NodeEventCallback: CallbackEnd\n");
	}

	ISceneEventManager::CallbackKey CallbackKey;
};


// Watches all notifications sent to a single ReferenceTarget(INode, Mtl etc)
template<typename ItemNotificationsHandler>
class TBaseItemNotifications: public ReferenceMaker
{
public:

	explicit TBaseItemNotifications(ItemNotificationsHandler& InHandler): Handler(InHandler)
	{
		LogDebug(FString::Printf(TEXT("TBaseItemNotifications(%s - %p)::TBaseItemNotifications"), *GetName(), Handler.GetNotificationTargetHandle()));

		// Don't record setting reference for Undo/Redo  - we control this ourselves(e.g. on ADDED_NODE event for addition)
		HoldSuspend SuspendUndoRedoGuard; 
		ReplaceReference(0, Handler.GetNotificationTargetHandle());
	}

	void Delete()
	{
		LogDebug(FString::Printf(TEXT("TBaseItemNotifications(%p)::Delete"), Handler.GetNotificationTargetHandle()));
		
		HoldSuspend SuspendUndoRedoGuard; 
		DeleteMe(); //Instead of calling DeleteAllRefs() in destructor, calling DeleteMe under HoldSuspend to disable undo for this object
	}

	// Same pointer could be added to the scene - handle this in case notifier for this pointer still kept
	void ReAdded()
	{
		ReplaceReference(0, Handler.GetNotificationTargetHandle());
	}

protected:

	virtual RefResult NotifyRefChanged(const Interval& ChangeInterval, RefTargetHandle TargetHandle, PartID& PartId, RefMessage Message, BOOL propagate) override
	{
		if (!bReferenceSet)
		{
			LOG_DEBUG_HEAVY(FString::Printf(TEXT("TBaseItemNotifications(UNSET - %p)::NotifyRefChanged:  Target - %p, Message - %x"), Handler.GetNotificationTargetHandle(), TargetHandle, Message));
			return REF_DONTCARE;
		}
		
		LOG_DEBUG_HEAVY(FString::Printf(TEXT("TBaseItemNotifications(%s - %p)::NotifyRefChanged:  Target - %p, Message - %x"), *GetName(), Handler.GetNotificationTargetHandle(), TargetHandle, Message));

		return Handler.HandleEvent(ChangeInterval, TargetHandle, PartId, Message);
	}

	virtual int NumRefs() override
	{
		return 1;
	}

	FString GetName()
	{
		return Handler.GetName();
	}

	virtual RefTargetHandle GetReference(int ReferenceIndex) override
	{
		if (bReferenceSet)
		{
			LOG_DEBUG_HEAVY(FString::Printf(TEXT("TBaseItemNotifications(%s - %p)::GetReference: %d"), *GetName(), Handler.GetNotificationTargetHandle(), ReferenceIndex));
			return Handler.GetNotificationTargetHandle();
		}
		else
		{
			LOG_DEBUG_HEAVY(FString::Printf(TEXT("TBaseItemNotifications(UNSET - %p)::GetReference: %d"), Handler.GetNotificationTargetHandle(), ReferenceIndex));
			return nullptr;
		}
	}

	virtual void SetReference(int ReferenceIndex, RefTargetHandle Handle) override
	{
		LOG_DEBUG_HEAVY(FString::Printf(TEXT("TBaseItemNotifications(%s - %p)::SetReference: %d, %p"), *GetName(), Handler.GetNotificationTargetHandle(), ReferenceIndex, Handle));

		// Either clears reference or passes the same handler
		check((Handle == nullptr ) || (Handle == Handler.GetNotificationTargetHandle()));

		bool bReferenceSetNew = Handle != nullptr;

		if (bReferenceSet && !bReferenceSetNew)
		{
			Handler.HandleDeleted();
		}

		bReferenceSet = bReferenceSetNew;
	}

private:
	ItemNotificationsHandler& Handler; // Handler for specific item type
	bool bReferenceSet = false;

protected:
	virtual ~TBaseItemNotifications() override {} // Should call Delete

};

class FMaterialNotifications: public FNoncopyable
{
public:

	FMaterialNotifications(ISceneTracker& InSceneTracker, Mtl& Material)
		: SceneTracker(InSceneTracker)
		, Material(Material)
		, ItemNotifications(new TBaseItemNotifications{*this})
	{
	}

	~FMaterialNotifications()
	{
		ItemNotifications->Delete();
	}

	FString GetName()
	{
		return Material.GetName().data();
	}

	RefTargetHandle GetNotificationTargetHandle()
	{
		return &Material;
	}

	RefResult HandleEvent(const Interval& ChangeInterval, RefTargetHandle TargetHandle, PartID& PartId, RefMessage Message)
	{
		if (SceneTracker.IsUpdateInProgress())
		{
			LogDebug(TEXT("     Event Skipped during Update"));
			// Don't propagate ReferenceMaker events when Update is in progress - these events are result of accessing render geometry on some objects, including
			// Smooth/TurboSmooth modifiers, VRay proxy and similar. Plugin retrieves render mesh from them and those objects fire geometry change events
			// Plugin reverts settings back to previously set(i.e. viewport for smooth modifiers or whatever vray proxy had for display)
			return REF_DONTCARE;
		}

		switch (Message)
		{
		case REFMSG_REF_DELETED:
			// 
		break;
	    case REFMSG_CHANGE:
			LOG_DEBUG_HEAVY(FString::Printf(TEXT("   REFMSG_CHANGE.PartId:  %llx"), PartId));

			SceneTracker.MaterialGraphModified(&Material);
		break;

		case REFMSG_SUBANIM_STRUCTURE_CHANGED:
			SceneTracker.MaterialGraphModified(&Material);
		break;

		default: 
			break;
		};

		return REF_SUCCEED;
	}

	void HandleDeleted()
	{
	}

	void ReAdded()
	{
		ItemNotifications->ReAdded();
	}

private:
	ISceneTracker& SceneTracker;
	Mtl& Material;
	TBaseItemNotifications<FMaterialNotifications>* ItemNotifications;
};


class FMaterialObserver
{
public:
	ISceneTracker& SceneTracker;

	FMaterialObserver(ISceneTracker& InSceneTracker) : SceneTracker(InSceneTracker)
	{
	}

	~FMaterialObserver()
	{
	}

	void AddItem(Mtl* Material)
	{
		if (TUniquePtr<FMaterialNotifications>* NotifierPtr = Items.Find(Material))
		{
			(*NotifierPtr)->ReAdded();
		}
		else
		{
			Items.Emplace(Material, new FMaterialNotifications(SceneTracker, *Material));
		}
	}

	void Reset()
	{
		Items.Reset();
	}

private:
	
	TMap<Mtl*, TUniquePtr<FMaterialNotifications>> Items;
};

class FNodeNotifications: public FNoncopyable
{
public:

	FNodeNotifications(ISceneTracker& InSceneTracker, INode& Node)
		: SceneTracker(InSceneTracker)
		, Node(Node)
		, ItemNotifications(new TBaseItemNotifications{*this})
	{
	}

	~FNodeNotifications()
	{
		ItemNotifications->Delete();
	}

	RefResult HandleEvent(const Interval& ChangeInterval, RefTargetHandle TargetHandle, PartID& PartId, RefMessage Message)
	{
		LogDebugNode(TEXT("     Node:"), &Node);

		if (SceneTracker.IsUpdateInProgress())
		{
			LogDebug(TEXT("     Event Skipped during Update"));
			// Don't propagate ReferenceMaker events when Update is in progress - these events are result of accessing render geometry on some objects, including
			// Smooth/TurboSmooth modifiers, VRay proxy and similar. Plugin retrieves render mesh from them and those objects fire geometry change events
			// Plugin reverts settings back to previously set(i.e. viewport for smooth modifiers or whatever vray proxy had for display)
			return REF_DONTCARE;
		}

		switch (Message)
		{
		case REFMSG_REF_DELETED:
			// 
		break;
	    case REFMSG_CHANGE:
			LOG_DEBUG_HEAVY(FString::Printf(TEXT("   REFMSG_CHANGE.PartId:  %llx"), PartId));

            if (PartId & (PART_TOPO|PART_GEOM|PART_TEXMAP))
            {
				SceneTracker.NodeGeometryChanged(&Node);
            }

            if (PartId & PART_TM)
            {
				SceneTracker.NodeTransformChanged(&Node);
            }

		break;

		case REFMSG_BEGIN_EDIT: 
			// Workaround for missing events for some modifiers
			// E.g. Body Cutter won't sent a CHANGE event an object is converted to it
			SceneTracker.NodeGeometryChanged(&Node);
		break;

		case REFMSG_NODE_MATERIAL_CHANGED:
			SceneTracker.NodeMaterialAssignmentChanged(&Node);
		break;

		case REFMSG_DISPLAY_MATERIAL_CHANGE:
			SceneTracker.NodeMaterialGraphModified(&Node);
		break;

		// Sadly, this event only sent for node when it's being hidden by using Display/Hide Byt Category. Unhiding it using that option doesn't send meaningful message per node
		// At least NOTIFY_BY_CATEGORY_DISPLAY_FILTER_CHANGED is sent(handled elsewhere)
		// case REFMSG_NODE_DISPLAY_PROP_CHANGED: break;

		case REFMSG_SUBANIM_STRUCTURE_CHANGED:
			if (Node.GetMtl() == TargetHandle)
			{
				SceneTracker.NodeMaterialGraphModified(&Node);
			}
		break;

		case REFMSG_NODE_RENDERING_PROP_CHANGED:
			// Handle Renderable flag change. mxs: box.setRenderable
			SceneTracker.NodePropertiesChanged(&Node);
		break;

#if MAX_PRODUCT_YEAR_NUMBER >= 2020
		case REFMSG_NODE_USER_PROPERTY_CHANGED:
			SceneTracker.NodePropertiesChanged(&Node);
		break;
#endif

		// SceneTracker.NodeGeometryChanged(nodes[NodeIndex]);
		default: 
			break;
		};

		return REF_SUCCEED;
	}

	FString GetName()
	{
		return Node.GetName();
	}

	RefTargetHandle GetNotificationTargetHandle()
	{
		return &Node;
	}

	void HandleDeleted()
	{
		SceneTracker.NodeDeleted(&Node);
	}

	void ReAdded()
	{
		ItemNotifications->ReAdded();
	}

private:
	ISceneTracker& SceneTracker;
	INode& Node;
	TBaseItemNotifications<FNodeNotifications>* ItemNotifications;
};

class FNodeObserver
{
public:
	ISceneTracker& SceneTracker;

	FNodeObserver(ISceneTracker& InSceneTracker) : SceneTracker(InSceneTracker)
	{
	}

	~FNodeObserver()
	{
	}

	void AddItem(INode* Node)
	{
		if (TUniquePtr<FNodeNotifications>* NotifierPtr = Items.Find(Node))
		{
			(*NotifierPtr)->ReAdded();
		}
		else
		{
			Items.Emplace(Node, new FNodeNotifications(SceneTracker, *Node));
		}
	}

	void Reset()
	{
		Items.Reset();
	}

private:
	
	TMap<INode*, TUniquePtr<FNodeNotifications>> Items;
};

FNotifications::FNotifications(IExporter& InExporter)
	: Exporter(InExporter)
	, NodeObserver(MakeUnique<FNodeObserver>(InExporter.GetSceneTracker()))
	, MaterialObserver(MakeUnique<FMaterialObserver>(InExporter.GetSceneTracker()))
{
	RegisterForSystemNotifications(); // Always follow system events like file reset, new, open and others. But some system notifications also track changes within scene.
}

FNotifications::~FNotifications()
{
	for (int Code: NotificationCodesRegistered)
	{
		UnRegisterNotification(On3dsMaxNotification, this, Code);
	}
	NotificationCodesRegistered.Reset();

	StopSceneChangeTracking();
}

void FNotifications::AddNode(INode* Node)
{
	NodeObserver->AddItem(Node);
}

void FNotifications::AddMaterial(Mtl* Node)
{
	MaterialObserver->AddItem(Node);
}

void FNotifications::RegisterForSystemNotifications()
{
	// Build todo: remove strings, for debug/logging
#pragma warning(push)
#pragma warning(disable:4995) // disable error on deprecated events, we just assign handlers not firing them
	int Codes[] = { NOTIFY_UNITS_CHANGE, NOTIFY_TIMEUNITS_CHANGE, NOTIFY_VIEWPORT_CHANGE, NOTIFY_SPACEMODE_CHANGE, NOTIFY_SYSTEM_PRE_RESET, NOTIFY_SYSTEM_POST_RESET, NOTIFY_SYSTEM_PRE_NEW, NOTIFY_SYSTEM_POST_NEW, NOTIFY_FILE_PRE_OPEN, NOTIFY_FILE_POST_OPEN, NOTIFY_FILE_PRE_MERGE, NOTIFY_FILE_PRE_SAVE, NOTIFY_FILE_POST_SAVE, NOTIFY_FILE_OPEN_FAILED, NOTIFY_FILE_PRE_SAVE_OLD, NOTIFY_FILE_POST_SAVE_OLD, NOTIFY_SELECTIONSET_CHANGED, NOTIFY_BITMAP_CHANGED, NOTIFY_PRE_RENDER, NOTIFY_POST_RENDER, NOTIFY_PRE_RENDERFRAME, NOTIFY_POST_RENDERFRAME, NOTIFY_PRE_IMPORT, NOTIFY_POST_IMPORT, NOTIFY_IMPORT_FAILED, NOTIFY_PRE_EXPORT, NOTIFY_POST_EXPORT, NOTIFY_EXPORT_FAILED, NOTIFY_NODE_RENAMED, NOTIFY_PRE_PROGRESS, NOTIFY_POST_PROGRESS, NOTIFY_MODPANEL_SEL_CHANGED, NOTIFY_RENDPARAM_CHANGED, NOTIFY_MATLIB_PRE_OPEN, NOTIFY_MATLIB_POST_OPEN, NOTIFY_MATLIB_PRE_SAVE, NOTIFY_MATLIB_POST_SAVE, NOTIFY_MATLIB_PRE_MERGE, NOTIFY_MATLIB_POST_MERGE, NOTIFY_FILELINK_BIND_FAILED, NOTIFY_FILELINK_DETACH_FAILED, NOTIFY_FILELINK_RELOAD_FAILED, NOTIFY_FILELINK_ATTACH_FAILED, NOTIFY_FILELINK_PRE_BIND, NOTIFY_FILELINK_POST_BIND, NOTIFY_FILELINK_PRE_DETACH, NOTIFY_FILELINK_POST_DETACH, NOTIFY_FILELINK_PRE_RELOAD, NOTIFY_FILELINK_POST_RELOAD, NOTIFY_FILELINK_PRE_ATTACH, NOTIFY_FILELINK_POST_ATTACH, NOTIFY_RENDER_PREEVAL, NOTIFY_NODE_CREATED, NOTIFY_NODE_LINKED, NOTIFY_NODE_UNLINKED, NOTIFY_NODE_HIDE, NOTIFY_NODE_UNHIDE, NOTIFY_NODE_FREEZE, NOTIFY_NODE_UNFREEZE, NOTIFY_NODE_PRE_MTL, NOTIFY_NODE_POST_MTL, NOTIFY_SCENE_ADDED_NODE, NOTIFY_SCENE_PRE_DELETED_NODE, NOTIFY_SCENE_POST_DELETED_NODE, NOTIFY_SEL_NODES_PRE_DELETE, NOTIFY_SEL_NODES_POST_DELETE, NOTIFY_WM_ENABLE, NOTIFY_SYSTEM_SHUTDOWN, NOTIFY_SYSTEM_STARTUP, NOTIFY_PLUGIN_LOADED, NOTIFY_SYSTEM_SHUTDOWN2, NOTIFY_ANIMATE_ON, NOTIFY_ANIMATE_OFF, NOTIFY_COLOR_CHANGE, NOTIFY_PRE_EDIT_OBJ_CHANGE, NOTIFY_POST_EDIT_OBJ_CHANGE, NOTIFY_RADIOSITYPROCESS_STARTED, NOTIFY_RADIOSITYPROCESS_STOPPED, NOTIFY_RADIOSITYPROCESS_RESET, NOTIFY_RADIOSITYPROCESS_DONE, NOTIFY_LIGHTING_UNIT_DISPLAY_SYSTEM_CHANGE, NOTIFY_BEGIN_RENDERING_REFLECT_REFRACT_MAP, NOTIFY_BEGIN_RENDERING_ACTUAL_FRAME, NOTIFY_BEGIN_RENDERING_TONEMAPPING_IMAGE, NOTIFY_RADIOSITY_PLUGIN_CHANGED, NOTIFY_SCENE_UNDO, NOTIFY_SCENE_REDO, NOTIFY_MANIPULATE_MODE_OFF, NOTIFY_MANIPULATE_MODE_ON, NOTIFY_SCENE_XREF_PRE_MERGE, NOTIFY_SCENE_XREF_POST_MERGE, NOTIFY_OBJECT_XREF_PRE_MERGE, NOTIFY_OBJECT_XREF_POST_MERGE, NOTIFY_PRE_MIRROR_NODES, NOTIFY_POST_MIRROR_NODES, NOTIFY_NODE_CLONED, NOTIFY_PRE_NOTIFYDEPENDENTS, NOTIFY_POST_NOTIFYDEPENDENTS, NOTIFY_MTL_REFDELETED, NOTIFY_TIMERANGE_CHANGE, NOTIFY_PRE_MODIFIER_ADDED, NOTIFY_POST_MODIFIER_ADDED, NOTIFY_PRE_MODIFIER_DELETED, NOTIFY_POST_MODIFIER_DELETED, NOTIFY_FILELINK_POST_RELOAD_PRE_PRUNE, NOTIFY_PRE_NODES_CLONED, NOTIFY_POST_NODES_CLONED, NOTIFY_SYSTEM_PRE_DIR_CHANGE, NOTIFY_SYSTEM_POST_DIR_CHANGE, NOTIFY_SV_SELECTIONSET_CHANGED, NOTIFY_SV_DOUBLECLICK_GRAPHNODE, NOTIFY_PRE_RENDERER_CHANGE, NOTIFY_POST_RENDERER_CHANGE, NOTIFY_SV_PRE_LAYOUT_CHANGE, NOTIFY_SV_POST_LAYOUT_CHANGE, NOTIFY_BY_CATEGORY_DISPLAY_FILTER_CHANGED, NOTIFY_CUSTOM_DISPLAY_FILTER_CHANGED, NOTIFY_LAYER_CREATED, NOTIFY_LAYER_DELETED, NOTIFY_NODE_LAYER_CHANGED, NOTIFY_TABBED_DIALOG_CREATED, NOTIFY_TABBED_DIALOG_DELETED, NOTIFY_NODE_NAME_SET, NOTIFY_HW_TEXTURE_CHANGED, NOTIFY_MXS_STARTUP, NOTIFY_MXS_POST_STARTUP, NOTIFY_ACTION_ITEM_HOTKEY_PRE_EXEC, NOTIFY_ACTION_ITEM_HOTKEY_POST_EXEC, NOTIFY_SCENESTATE_PRE_SAVE, NOTIFY_SCENESTATE_POST_SAVE, NOTIFY_SCENESTATE_PRE_RESTORE, NOTIFY_SCENESTATE_POST_RESTORE, NOTIFY_SCENESTATE_DELETE, NOTIFY_SCENESTATE_RENAME, NOTIFY_SCENE_PRE_UNDO, NOTIFY_SCENE_PRE_REDO, NOTIFY_SCENE_POST_UNDO, NOTIFY_SCENE_POST_REDO, NOTIFY_MXS_SHUTDOWN, NOTIFY_D3D_PRE_DEVICE_RESET, NOTIFY_D3D_POST_DEVICE_RESET, NOTIFY_TOOLPALETTE_MTL_SUSPEND, NOTIFY_TOOLPALETTE_MTL_RESUME, NOTIFY_CLASSDESC_REPLACED, NOTIFY_FILE_PRE_OPEN_PROCESS, NOTIFY_FILE_POST_OPEN_PROCESS, NOTIFY_FILE_PRE_SAVE_PROCESS, NOTIFY_FILE_POST_SAVE_PROCESS, NOTIFY_CLASSDESC_LOADED, NOTIFY_TOOLBARS_PRE_LOAD, NOTIFY_TOOLBARS_POST_LOAD, NOTIFY_ATS_PRE_REPATH_PHASE, NOTIFY_ATS_POST_REPATH_PHASE, NOTIFY_PROXY_TEMPORARY_DISABLE_START, NOTIFY_PROXY_TEMPORARY_DISABLE_END, NOTIFY_FILE_CHECK_STATUS, NOTIFY_NAMED_SEL_SET_CREATED, NOTIFY_NAMED_SEL_SET_DELETED, NOTIFY_NAMED_SEL_SET_RENAMED, NOTIFY_NAMED_SEL_SET_PRE_MODIFY, NOTIFY_NAMED_SEL_SET_POST_MODIFY, NOTIFY_MODPANEL_SUBOBJECTLEVEL_CHANGED, NOTIFY_FAILED_DIRECTX_MATERIAL_TEXTURE_LOAD, NOTIFY_POST_SCENE_RESET, NOTIFY_ANIM_LAYERS_ENABLED, NOTIFY_ANIM_LAYERS_DISABLED, NOTIFY_ACTION_ITEM_PRE_START_OVERRIDE, NOTIFY_ACTION_ITEM_POST_START_OVERRIDE, NOTIFY_ACTION_ITEM_PRE_END_OVERRIDE, NOTIFY_ACTION_ITEM_POST_END_OVERRIDE, NOTIFY_PRE_NODE_GENERAL_PROP_CHANGED, NOTIFY_POST_NODE_GENERAL_PROP_CHANGED, NOTIFY_PRE_NODE_GI_PROP_CHANGED, NOTIFY_POST_NODE_GI_PROP_CHANGED, NOTIFY_PRE_NODE_MENTALRAY_PROP_CHANGED, NOTIFY_POST_NODE_MENTALRAY_PROP_CHANGED, NOTIFY_PRE_NODE_BONE_PROP_CHANGED, NOTIFY_POST_NODE_BONE_PROP_CHANGED, NOTIFY_PRE_NODE_USER_PROP_CHANGED, NOTIFY_POST_NODE_USER_PROP_CHANGED, NOTIFY_PRE_NODE_RENDER_PROP_CHANGED, NOTIFY_POST_NODE_RENDER_PROP_CHANGED, NOTIFY_PRE_NODE_DISPLAY_PROP_CHANGED, NOTIFY_POST_NODE_DISPLAY_PROP_CHANGED, NOTIFY_PRE_NODE_BASIC_PROP_CHANGED, NOTIFY_POST_NODE_BASIC_PROP_CHANGED, NOTIFY_SELECTION_LOCK, NOTIFY_SELECTION_UNLOCK, NOTIFY_PRE_IMAGE_VIEWER_DISPLAY, NOTIFY_POST_IMAGE_VIEWER_DISPLAY, NOTIFY_IMAGE_VIEWER_UPDATE, NOTIFY_CUSTOM_ATTRIBUTES_ADDED, NOTIFY_CUSTOM_ATTRIBUTES_REMOVED, NOTIFY_OS_THEME_CHANGED, NOTIFY_ACTIVE_VIEWPORT_CHANGED, NOTIFY_PRE_MAXMAINWINDOW_SHOW, NOTIFY_POST_MAXMAINWINDOW_SHOW, NOTIFY_CLASSDESC_ADDED, NOTIFY_OBJECT_DEFINITION_CHANGE_BEGIN, NOTIFY_OBJECT_DEFINITION_CHANGE_END, NOTIFY_MTLBASE_PARAMDLG_PRE_OPEN, NOTIFY_MTLBASE_PARAMDLG_POST_CLOSE, NOTIFY_PRE_APP_FRAME_THEME_CHANGED, NOTIFY_APP_FRAME_THEME_CHANGED, NOTIFY_PRE_VIEWPORT_DELETE, NOTIFY_PRE_WORKSPACE_CHANGE, NOTIFY_POST_WORKSPACE_CHANGE, NOTIFY_PRE_WORKSPACE_COLLECTION_CHANGE, NOTIFY_POST_WORKSPACE_COLLECTION_CHANGE, NOTIFY_KEYBOARD_SETTING_CHANGED, NOTIFY_MOUSE_SETTING_CHANGED, NOTIFY_TOOLBARS_PRE_SAVE, NOTIFY_TOOLBARS_POST_SAVE, NOTIFY_APP_ACTIVATED, NOTIFY_APP_DEACTIVATED, NOTIFY_VIEWPORT_SAFEFRAME_TOGGLE, NOTIFY_PLUGINS_PRE_SHUTDOWN, NOTIFY_PLUGINS_PRE_UNLOAD, NOTIFY_CUI_MENUS_POST_LOAD, NOTIFY_LAYER_PARENT_CHANGED, NOTIFY_ACTION_ITEM_EXECUTION_STARTED, NOTIFY_ACTION_ITEM_EXECUTION_ENDED, NOTIFY_INTERACTIVE_PLUGIN_INSTANCE_CREATION_STARTED, NOTIFY_INTERACTIVE_PLUGIN_INSTANCE_CREATION_ENDED, NOTIFY_POST_NODE_SELECT_OPERATION

#if MAX_PRODUCT_YEAR_NUMBER >= 2018
		, NOTIFY_PRE_VIEWPORT_TOOLTIP, NOTIFY_WELCOMESCREEN_DONE, NOTIFY_PLAYBACK_START, NOTIFY_PLAYBACK_END, NOTIFY_SCENE_EXPLORER_NEEDS_UPDATE, NOTIFY_FILE_POST_OPEN_PROCESS_FINALIZED, NOTIFY_FILE_POST_MERGE_PROCESS_FINALIZED
#endif

#if MAX_PRODUCT_YEAR_NUMBER >= 2022
		, NOTIFY_PRE_PROJECT_FOLDER_CHANGE, NOTIFY_POST_PROJECT_FOLDER_CHANGE, NOTIFY_PRE_MXS_STARTUP_SCRIPT_LOAD, NOTIFY_SYSTEM_SHUTDOWN_CHECK, NOTIFY_SYSTEM_SHUTDOWN_CHECK_FAILED, NOTIFY_SYSTEM_SHUTDOWN_CHECK_PASSED, NOTIFY_FILE_POST_MERGE3, NOTIFY_ACTIVESHADE_IN_FRAMEBUFFER_TOGGLED, NOTIFY_PRE_ACTIVESHADE_IN_VIEWPORT_TOGGLED, NOTIFY_POST_ACTIVESHADE_IN_VIEWPORT_TOGGLED
#endif
#if MAX_PRODUCT_YEAR_NUMBER <= 2025 // Deprecated in 2025, removed in 2026
		, NOTIFY_CUI_MENUS_UPDATED, NOTIFY_CUI_MENUS_PRE_SAVE, NOTIFY_CUI_MENUS_POST_SAVE, NOTIFY_RENDER_PREEVAL_FRAMEINFO
#endif
		, NOTIFY_INTERNAL_USE_START };
	FString Strings[] = { TEXT("NOTIFY_UNITS_CHANGE"), TEXT("NOTIFY_TIMEUNITS_CHANGE"), TEXT("NOTIFY_VIEWPORT_CHANGE"), TEXT("NOTIFY_SPACEMODE_CHANGE"), TEXT("NOTIFY_SYSTEM_PRE_RESET"), TEXT("NOTIFY_SYSTEM_POST_RESET"), TEXT("NOTIFY_SYSTEM_PRE_NEW"), TEXT("NOTIFY_SYSTEM_POST_NEW"), TEXT("NOTIFY_FILE_PRE_OPEN"), TEXT("NOTIFY_FILE_POST_OPEN"), TEXT("NOTIFY_FILE_PRE_MERGE"), TEXT("NOTIFY_FILE_PRE_SAVE"), TEXT("NOTIFY_FILE_POST_SAVE"), TEXT("NOTIFY_FILE_OPEN_FAILED"), TEXT("NOTIFY_FILE_PRE_SAVE_OLD"), TEXT("NOTIFY_FILE_POST_SAVE_OLD"), TEXT("NOTIFY_SELECTIONSET_CHANGED"), TEXT("NOTIFY_BITMAP_CHANGED"), TEXT("NOTIFY_PRE_RENDER"), TEXT("NOTIFY_POST_RENDER"), TEXT("NOTIFY_PRE_RENDERFRAME"), TEXT("NOTIFY_POST_RENDERFRAME"), TEXT("NOTIFY_PRE_IMPORT"), TEXT("NOTIFY_POST_IMPORT"), TEXT("NOTIFY_IMPORT_FAILED"), TEXT("NOTIFY_PRE_EXPORT"), TEXT("NOTIFY_POST_EXPORT"), TEXT("NOTIFY_EXPORT_FAILED"), TEXT("NOTIFY_NODE_RENAMED"), TEXT("NOTIFY_PRE_PROGRESS"), TEXT("NOTIFY_POST_PROGRESS"), TEXT("NOTIFY_MODPANEL_SEL_CHANGED"), TEXT("NOTIFY_RENDPARAM_CHANGED"), TEXT("NOTIFY_MATLIB_PRE_OPEN"), TEXT("NOTIFY_MATLIB_POST_OPEN"), TEXT("NOTIFY_MATLIB_PRE_SAVE"), TEXT("NOTIFY_MATLIB_POST_SAVE"), TEXT("NOTIFY_MATLIB_PRE_MERGE"), TEXT("NOTIFY_MATLIB_POST_MERGE"), TEXT("NOTIFY_FILELINK_BIND_FAILED"), TEXT("NOTIFY_FILELINK_DETACH_FAILED"), TEXT("NOTIFY_FILELINK_RELOAD_FAILED"), TEXT("NOTIFY_FILELINK_ATTACH_FAILED"), TEXT("NOTIFY_FILELINK_PRE_BIND"), TEXT("NOTIFY_FILELINK_POST_BIND"), TEXT("NOTIFY_FILELINK_PRE_DETACH"), TEXT("NOTIFY_FILELINK_POST_DETACH"), TEXT("NOTIFY_FILELINK_PRE_RELOAD"), TEXT("NOTIFY_FILELINK_POST_RELOAD"), TEXT("NOTIFY_FILELINK_PRE_ATTACH"), TEXT("NOTIFY_FILELINK_POST_ATTACH"), TEXT("NOTIFY_RENDER_PREEVAL"), TEXT("NOTIFY_NODE_CREATED"), TEXT("NOTIFY_NODE_LINKED"), TEXT("NOTIFY_NODE_UNLINKED"), TEXT("NOTIFY_NODE_HIDE"), TEXT("NOTIFY_NODE_UNHIDE"), TEXT("NOTIFY_NODE_FREEZE"), TEXT("NOTIFY_NODE_UNFREEZE"), TEXT("NOTIFY_NODE_PRE_MTL"), TEXT("NOTIFY_NODE_POST_MTL"), TEXT("NOTIFY_SCENE_ADDED_NODE"), TEXT("NOTIFY_SCENE_PRE_DELETED_NODE"), TEXT("NOTIFY_SCENE_POST_DELETED_NODE"), TEXT("NOTIFY_SEL_NODES_PRE_DELETE"), TEXT("NOTIFY_SEL_NODES_POST_DELETE"), TEXT("NOTIFY_WM_ENABLE"), TEXT("NOTIFY_SYSTEM_SHUTDOWN"), TEXT("NOTIFY_SYSTEM_STARTUP"), TEXT("NOTIFY_PLUGIN_LOADED"), TEXT("NOTIFY_SYSTEM_SHUTDOWN2"), TEXT("NOTIFY_ANIMATE_ON"), TEXT("NOTIFY_ANIMATE_OFF"), TEXT("NOTIFY_COLOR_CHANGE"), TEXT("NOTIFY_PRE_EDIT_OBJ_CHANGE"), TEXT("NOTIFY_POST_EDIT_OBJ_CHANGE"), TEXT("NOTIFY_RADIOSITYPROCESS_STARTED"), TEXT("NOTIFY_RADIOSITYPROCESS_STOPPED"), TEXT("NOTIFY_RADIOSITYPROCESS_RESET"), TEXT("NOTIFY_RADIOSITYPROCESS_DONE"), TEXT("NOTIFY_LIGHTING_UNIT_DISPLAY_SYSTEM_CHANGE"), TEXT("NOTIFY_BEGIN_RENDERING_REFLECT_REFRACT_MAP"), TEXT("NOTIFY_BEGIN_RENDERING_ACTUAL_FRAME"), TEXT("NOTIFY_BEGIN_RENDERING_TONEMAPPING_IMAGE"), TEXT("NOTIFY_RADIOSITY_PLUGIN_CHANGED"), TEXT("NOTIFY_SCENE_UNDO"), TEXT("NOTIFY_SCENE_REDO"), TEXT("NOTIFY_MANIPULATE_MODE_OFF"), TEXT("NOTIFY_MANIPULATE_MODE_ON"), TEXT("NOTIFY_SCENE_XREF_PRE_MERGE"), TEXT("NOTIFY_SCENE_XREF_POST_MERGE"), TEXT("NOTIFY_OBJECT_XREF_PRE_MERGE"), TEXT("NOTIFY_OBJECT_XREF_POST_MERGE"), TEXT("NOTIFY_PRE_MIRROR_NODES"), TEXT("NOTIFY_POST_MIRROR_NODES"), TEXT("NOTIFY_NODE_CLONED"), TEXT("NOTIFY_PRE_NOTIFYDEPENDENTS"), TEXT("NOTIFY_POST_NOTIFYDEPENDENTS"), TEXT("NOTIFY_MTL_REFDELETED"), TEXT("NOTIFY_TIMERANGE_CHANGE"), TEXT("NOTIFY_PRE_MODIFIER_ADDED"), TEXT("NOTIFY_POST_MODIFIER_ADDED"), TEXT("NOTIFY_PRE_MODIFIER_DELETED"), TEXT("NOTIFY_POST_MODIFIER_DELETED"), TEXT("NOTIFY_FILELINK_POST_RELOAD_PRE_PRUNE"), TEXT("NOTIFY_PRE_NODES_CLONED"), TEXT("NOTIFY_POST_NODES_CLONED"), TEXT("NOTIFY_SYSTEM_PRE_DIR_CHANGE"), TEXT("NOTIFY_SYSTEM_POST_DIR_CHANGE"), TEXT("NOTIFY_SV_SELECTIONSET_CHANGED"), TEXT("NOTIFY_SV_DOUBLECLICK_GRAPHNODE"), TEXT("NOTIFY_PRE_RENDERER_CHANGE"), TEXT("NOTIFY_POST_RENDERER_CHANGE"), TEXT("NOTIFY_SV_PRE_LAYOUT_CHANGE"), TEXT("NOTIFY_SV_POST_LAYOUT_CHANGE"), TEXT("NOTIFY_BY_CATEGORY_DISPLAY_FILTER_CHANGED"), TEXT("NOTIFY_CUSTOM_DISPLAY_FILTER_CHANGED"), TEXT("NOTIFY_LAYER_CREATED"), TEXT("NOTIFY_LAYER_DELETED"), TEXT("NOTIFY_NODE_LAYER_CHANGED"), TEXT("NOTIFY_TABBED_DIALOG_CREATED"), TEXT("NOTIFY_TABBED_DIALOG_DELETED"), TEXT("NOTIFY_NODE_NAME_SET"), TEXT("NOTIFY_HW_TEXTURE_CHANGED"), TEXT("NOTIFY_MXS_STARTUP"), TEXT("NOTIFY_MXS_POST_STARTUP"), TEXT("NOTIFY_ACTION_ITEM_HOTKEY_PRE_EXEC"), TEXT("NOTIFY_ACTION_ITEM_HOTKEY_POST_EXEC"), TEXT("NOTIFY_SCENESTATE_PRE_SAVE"), TEXT("NOTIFY_SCENESTATE_POST_SAVE"), TEXT("NOTIFY_SCENESTATE_PRE_RESTORE"), TEXT("NOTIFY_SCENESTATE_POST_RESTORE"), TEXT("NOTIFY_SCENESTATE_DELETE"), TEXT("NOTIFY_SCENESTATE_RENAME"), TEXT("NOTIFY_SCENE_PRE_UNDO"), TEXT("NOTIFY_SCENE_PRE_REDO"), TEXT("NOTIFY_SCENE_POST_UNDO"), TEXT("NOTIFY_SCENE_POST_REDO"), TEXT("NOTIFY_MXS_SHUTDOWN"), TEXT("NOTIFY_D3D_PRE_DEVICE_RESET"), TEXT("NOTIFY_D3D_POST_DEVICE_RESET"), TEXT("NOTIFY_TOOLPALETTE_MTL_SUSPEND"), TEXT("NOTIFY_TOOLPALETTE_MTL_RESUME"), TEXT("NOTIFY_CLASSDESC_REPLACED"), TEXT("NOTIFY_FILE_PRE_OPEN_PROCESS"), TEXT("NOTIFY_FILE_POST_OPEN_PROCESS"), TEXT("NOTIFY_FILE_PRE_SAVE_PROCESS"), TEXT("NOTIFY_FILE_POST_SAVE_PROCESS"), TEXT("NOTIFY_CLASSDESC_LOADED"), TEXT("NOTIFY_TOOLBARS_PRE_LOAD"), TEXT("NOTIFY_TOOLBARS_POST_LOAD"), TEXT("NOTIFY_ATS_PRE_REPATH_PHASE"), TEXT("NOTIFY_ATS_POST_REPATH_PHASE"), TEXT("NOTIFY_PROXY_TEMPORARY_DISABLE_START"), TEXT("NOTIFY_PROXY_TEMPORARY_DISABLE_END"), TEXT("NOTIFY_FILE_CHECK_STATUS"), TEXT("NOTIFY_NAMED_SEL_SET_CREATED"), TEXT("NOTIFY_NAMED_SEL_SET_DELETED"), TEXT("NOTIFY_NAMED_SEL_SET_RENAMED"), TEXT("NOTIFY_NAMED_SEL_SET_PRE_MODIFY"), TEXT("NOTIFY_NAMED_SEL_SET_POST_MODIFY"), TEXT("NOTIFY_MODPANEL_SUBOBJECTLEVEL_CHANGED"), TEXT("NOTIFY_FAILED_DIRECTX_MATERIAL_TEXTURE_LOAD"), TEXT("NOTIFY_RENDER_PREEVAL_FRAMEINFO"), TEXT("NOTIFY_POST_SCENE_RESET"), TEXT("NOTIFY_ANIM_LAYERS_ENABLED"), TEXT("NOTIFY_ANIM_LAYERS_DISABLED"), TEXT("NOTIFY_ACTION_ITEM_PRE_START_OVERRIDE"), TEXT("NOTIFY_ACTION_ITEM_POST_START_OVERRIDE"), TEXT("NOTIFY_ACTION_ITEM_PRE_END_OVERRIDE"), TEXT("NOTIFY_ACTION_ITEM_POST_END_OVERRIDE"), TEXT("NOTIFY_PRE_NODE_GENERAL_PROP_CHANGED"), TEXT("NOTIFY_POST_NODE_GENERAL_PROP_CHANGED"), TEXT("NOTIFY_PRE_NODE_GI_PROP_CHANGED"), TEXT("NOTIFY_POST_NODE_GI_PROP_CHANGED"), TEXT("NOTIFY_PRE_NODE_MENTALRAY_PROP_CHANGED"), TEXT("NOTIFY_POST_NODE_MENTALRAY_PROP_CHANGED"), TEXT("NOTIFY_PRE_NODE_BONE_PROP_CHANGED"), TEXT("NOTIFY_POST_NODE_BONE_PROP_CHANGED"), TEXT("NOTIFY_PRE_NODE_USER_PROP_CHANGED"), TEXT("NOTIFY_POST_NODE_USER_PROP_CHANGED"), TEXT("NOTIFY_PRE_NODE_RENDER_PROP_CHANGED"), TEXT("NOTIFY_POST_NODE_RENDER_PROP_CHANGED"), TEXT("NOTIFY_PRE_NODE_DISPLAY_PROP_CHANGED"), TEXT("NOTIFY_POST_NODE_DISPLAY_PROP_CHANGED"), TEXT("NOTIFY_PRE_NODE_BASIC_PROP_CHANGED"), TEXT("NOTIFY_POST_NODE_BASIC_PROP_CHANGED"), TEXT("NOTIFY_SELECTION_LOCK"), TEXT("NOTIFY_SELECTION_UNLOCK"), TEXT("NOTIFY_PRE_IMAGE_VIEWER_DISPLAY"), TEXT("NOTIFY_POST_IMAGE_VIEWER_DISPLAY"), TEXT("NOTIFY_IMAGE_VIEWER_UPDATE"), TEXT("NOTIFY_CUSTOM_ATTRIBUTES_ADDED"), TEXT("NOTIFY_CUSTOM_ATTRIBUTES_REMOVED"), TEXT("NOTIFY_OS_THEME_CHANGED"), TEXT("NOTIFY_ACTIVE_VIEWPORT_CHANGED"), TEXT("NOTIFY_PRE_MAXMAINWINDOW_SHOW"), TEXT("NOTIFY_POST_MAXMAINWINDOW_SHOW"), TEXT("NOTIFY_CLASSDESC_ADDED"), TEXT("NOTIFY_OBJECT_DEFINITION_CHANGE_BEGIN"), TEXT("NOTIFY_OBJECT_DEFINITION_CHANGE_END"), TEXT("NOTIFY_MTLBASE_PARAMDLG_PRE_OPEN"), TEXT("NOTIFY_MTLBASE_PARAMDLG_POST_CLOSE"), TEXT("NOTIFY_PRE_APP_FRAME_THEME_CHANGED"), TEXT("NOTIFY_APP_FRAME_THEME_CHANGED"), TEXT("NOTIFY_PRE_VIEWPORT_DELETE"), TEXT("NOTIFY_PRE_WORKSPACE_CHANGE"), TEXT("NOTIFY_POST_WORKSPACE_CHANGE"), TEXT("NOTIFY_PRE_WORKSPACE_COLLECTION_CHANGE"), TEXT("NOTIFY_POST_WORKSPACE_COLLECTION_CHANGE"), TEXT("NOTIFY_KEYBOARD_SETTING_CHANGED"), TEXT("NOTIFY_MOUSE_SETTING_CHANGED"), TEXT("NOTIFY_TOOLBARS_PRE_SAVE"), TEXT("NOTIFY_TOOLBARS_POST_SAVE"), TEXT("NOTIFY_APP_ACTIVATED"), TEXT("NOTIFY_APP_DEACTIVATED"), TEXT("NOTIFY_CUI_MENUS_UPDATED"), TEXT("NOTIFY_CUI_MENUS_PRE_SAVE"), TEXT("NOTIFY_CUI_MENUS_POST_SAVE"), TEXT("NOTIFY_VIEWPORT_SAFEFRAME_TOGGLE"), TEXT("NOTIFY_PLUGINS_PRE_SHUTDOWN"), TEXT("NOTIFY_PLUGINS_PRE_UNLOAD"), TEXT("NOTIFY_CUI_MENUS_POST_LOAD"), TEXT("NOTIFY_LAYER_PARENT_CHANGED"), TEXT("NOTIFY_ACTION_ITEM_EXECUTION_STARTED"), TEXT("NOTIFY_ACTION_ITEM_EXECUTION_ENDED"), TEXT("NOTIFY_INTERACTIVE_PLUGIN_INSTANCE_CREATION_STARTED"), TEXT("NOTIFY_INTERACTIVE_PLUGIN_INSTANCE_CREATION_ENDED"), TEXT("NOTIFY_POST_NODE_SELECT_OPERATION")

#if MAX_PRODUCT_YEAR_NUMBER >= 2018
		, TEXT("NOTIFY_PRE_VIEWPORT_TOOLTIP"), TEXT("NOTIFY_WELCOMESCREEN_DONE"), TEXT("NOTIFY_PLAYBACK_START"), TEXT("NOTIFY_PLAYBACK_END"), TEXT("NOTIFY_SCENE_EXPLORER_NEEDS_UPDATE"), TEXT("NOTIFY_FILE_POST_OPEN_PROCESS_FINALIZED") , TEXT("NOTIFY_FILE_POST_MERGE_PROCESS_FINALIZED")
#endif

#if MAX_PRODUCT_YEAR_NUMBER >= 2022
		, TEXT("NOTIFY_PRE_PROJECT_FOLDER_CHANGE"), TEXT("NOTIFY_POST_PROJECT_FOLDER_CHANGE"), TEXT("NOTIFY_PRE_MXS_STARTUP_SCRIPT_LOAD"), TEXT("NOTIFY_SYSTEM_SHUTDOWN_CHECK"), TEXT("NOTIFY_SYSTEM_SHUTDOWN_CHECK_FAILED"), TEXT("NOTIFY_SYSTEM_SHUTDOWN_CHECK_PASSED"), TEXT("NOTIFY_FILE_POST_MERGE3"), TEXT("NOTIFY_ACTIVESHADE_IN_FRAMEBUFFER_TOGGLED"), TEXT("NOTIFY_PRE_ACTIVESHADE_IN_VIEWPORT_TOGGLED"), TEXT("NOTIFY_POST_ACTIVESHADE_IN_VIEWPORT_TOGGLED")
#endif

		, TEXT("NOTIFY_INTERNAL_USE_START") };
#pragma warning(pop)

	int i = 0;
	for (int Code : Codes)
	{
		RegisterNotification(On3dsMaxNotification, this, Code);
		NotificationCodetoString.Add(Code, Strings[i]);
		NotificationCodesRegistered.Add(Code);
		++i;
	}
}

void FNotifications::StartSceneChangeTracking()
{
	if (bSceneChangeTracking)
	{
		return;
	}
	bSceneChangeTracking = true;

	NodeEventCallback = MakeUnique<FNodeEventCallback>(Exporter.GetSceneTracker());
	// Setup Node Event System callback
	// https://help.autodesk.com/view/3DSMAX/2018/ENU/?guid=__files_GUID_7C91D285_5683_4606_9F7C_B8D3A7CA508B_htm
	NodeEventCallback->CallbackKey = GetISceneEventManager()->RegisterCallback(NodeEventCallback.Get());
}

void FNotifications::StopSceneChangeTracking()
{
	if (!bSceneChangeTracking)
	{
		return;
	}
	bSceneChangeTracking = false;

	GetISceneEventManager()->UnRegisterCallback(NodeEventCallback->CallbackKey);
	NodeEventCallback.Reset();

	NodeObserver->Reset();
	MaterialObserver->Reset();
}

FString FNotifications::ConvertNotificationCodeToString(int code)
{
	FString* Str = NotificationCodetoString.Find(code);
	return Str ? *Str : TEXT("<unknown>");
}

void FNotifications::PrepareForUpdate()
{
	if (NodeEventCallback)
	{
		GetISceneEventManager()->TriggerMessages(NodeEventCallback->CallbackKey);
	}
}

void FNotifications::On3dsMaxNotification(void* param, NotifyInfo* info)
{
	FNotifications& NotificationsHandler = *static_cast<FNotifications*>(param);
	IExporter* Exporter = &NotificationsHandler.Exporter;

	switch (info->intcode)
	{
	// Skip some events to display(spamming tests)
	case NOTIFY_VIEWPORT_CHANGE:
	case NOTIFY_PRE_RENDERER_CHANGE:
	case NOTIFY_POST_RENDERER_CHANGE:
	case NOTIFY_CUSTOM_ATTRIBUTES_ADDED:
	case NOTIFY_CUSTOM_ATTRIBUTES_REMOVED:
	case NOTIFY_MTL_REFDELETED:
		break;

	case NOTIFY_PLUGINS_PRE_SHUTDOWN: // This one crashes when calling LogInfo
		return;
	// case NOTIFY_NODE_LAYER_CHANGED:
	// 	break;
	case NOTIFY_NODE_UNLINKED:
	case NOTIFY_NODE_CREATED:
	case NOTIFY_NODE_LINKED:
		{
#if MAX_PRODUCT_YEAR_NUMBER >= 2027
		INode* Node = GetNotifyParam<NOTIFY_NODE_UNLINKED, NOTIFY_NODE_CREATED, NOTIFY_NODE_LINKED>(info);
#else
		INode* Node = reinterpret_cast<INode*>(info->callParam);
#endif
			LogDebugNode(NotificationsHandler.ConvertNotificationCodeToString(info->intcode), Node);
		}
		break;
	default:
		LOG_DEBUG_HEAVY(FString(TEXT("Notify: ")) + NotificationsHandler.ConvertNotificationCodeToString(info->intcode));
	};

	if (NotificationsHandler.bSceneChangeTracking)
	{
		ISceneTracker& SceneTracker = Exporter->GetSceneTracker();
		switch (info->intcode)
		{
		case NOTIFY_BY_CATEGORY_DISPLAY_FILTER_CHANGED:
			SceneTracker.HideByCategoryChanged();
			break;

		case NOTIFY_NODE_POST_MTL:
			// Event - node got a new material
			break;

		case NOTIFY_SCENE_ADDED_NODE:
		{
			// note: INodeEventCallback::Added/Deleted is not used because there's a test case when it fails:
			//   When a box is being created(dragging corners using mouse interface) and then cancelled during creation(RMB pressed)
			//   INodeEventCallback::Deleted event is not fired by Max, although Added was called(along with other change events during creation)
#if MAX_PRODUCT_YEAR_NUMBER >= 2027
			INode* Node = GetNotifyParam<NOTIFY_SCENE_ADDED_NODE>(info);
#else
			INode* Node = reinterpret_cast<INode*>(info->callParam);
#endif

			LogDebugNode(NotificationsHandler.ConvertNotificationCodeToString(info->intcode), Node);
			SceneTracker.NodeAdded(Node);

			break;
		}

		case NOTIFY_SCENE_PRE_DELETED_NODE:
		{
			// note: INodeEventCallback::Deleted is not called when object creation was cancelled in the process

#if MAX_PRODUCT_YEAR_NUMBER >= 2027
			INode* Node = GetNotifyParam<NOTIFY_SCENE_PRE_DELETED_NODE>(info);
#else
			INode* Node = reinterpret_cast<INode*>(info->callParam);
#endif
			LogDebugNode(NotificationsHandler.ConvertNotificationCodeToString(info->intcode), Node);

			SceneTracker.NodeDeleted(Node);
			break;
		}
		case NOTIFY_NODE_UNLINKED:
		{
#if MAX_PRODUCT_YEAR_NUMBER >= 2027
			INode* Node = GetNotifyParam<NOTIFY_NODE_UNLINKED>(info);
#else
			INode* Node = reinterpret_cast<INode*>(info->callParam);
#endif
			LogDebugNode(NotificationsHandler.ConvertNotificationCodeToString(NOTIFY_NODE_UNLINKED), Node);
			break;
		}

		case NOTIFY_NODE_HIDE:
		case NOTIFY_NODE_UNHIDE:
		{
#if MAX_PRODUCT_YEAR_NUMBER >= 2027
			INode* Node = GetNotifyParam<NOTIFY_NODE_HIDE, NOTIFY_NODE_UNHIDE>(info);
#else
			INode* Node = reinterpret_cast<INode*>(info->callParam);
#endif
			SceneTracker.NodeHideChanged(Node);
			break;
		}

		case NOTIFY_SCENE_XREF_POST_MERGE:
		{
#if MAX_PRODUCT_YEAR_NUMBER >= 2027
			INode* Node = GetNotifyParam<NOTIFY_SCENE_XREF_POST_MERGE>(info);
#else
			INode* Node = reinterpret_cast<INode*>(info->callParam);
#endif
			LogDebugNode(NotificationsHandler.ConvertNotificationCodeToString(info->intcode), Node);

			SceneTracker.NodeXRefMerged(Node);
			break;
		}
		}
	}

	// Handle events for scene load/new/reset
	// Note - different UI functions which destroy current scene send different notifications
	// Reset - NOTIFY_SYSTEM_PRE_RESET NOTIFY_POST_SCENE_RESET
	// Open -  NOTIFY_FILE_PRE_OPEN_PROCESS NOTIFY_FILE_PRE_OPEN NOTIFY_POST_SCENE_RESET NOTIFY_FILE_POST_OPEN NOTIFY_FILE_POST_OPEN_PROCESS NOTIFY_FILE_POST_OPEN_PROCESS_FINALIZED
	// File>New>New All -  NOTIFY_SYSTEM_PRE_NEW NOTIFY_POST_SCENE_RESET NOTIFY_SYSTEM_POST_NEW
	// File>New>New From Template -  NOTIFY_SYSTEM_PRE_RESET NOTIFY_POST_SCENE_RESET NOTIFY_SYSTEM_POST_RESET
	switch (info->intcode)
	{
	// Handle New/Reset events sent before scene reset - stop tracking immediately when "Pre" events are received - after this point all nodes are invalid, don't wait for "Post" event
	case NOTIFY_SYSTEM_PRE_NEW:  // Sent when File>New>New All is selected 
	case NOTIFY_SYSTEM_PRE_RESET:  // Sent when Reset OR File>New>New From Template is selected
	case NOTIFY_FILE_PRE_OPEN:  // Sent when File>Open is used
		Exporter->ResetSceneTracking();
		break;

	case NOTIFY_SYSTEM_POST_NEW:
	case NOTIFY_SYSTEM_POST_RESET:
	case NOTIFY_FILE_POST_OPEN:
		Exporter->InitializeDirectLinkForScene();
		break;

	}
}

}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // NEW_DIRECTLINK_PLUGIN