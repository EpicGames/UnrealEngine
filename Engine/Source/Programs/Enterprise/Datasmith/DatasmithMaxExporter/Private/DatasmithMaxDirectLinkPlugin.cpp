// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef NEW_DIRECTLINK_PLUGIN

#include "DatasmithMaxDirectLink.h"

#include "CoreMinimal.h"

#include "DatasmithMaxExporterDefines.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "Max.h"
MAX_INCLUDES_END


THIRD_PARTY_INCLUDES_START
#include <locale.h>
THIRD_PARTY_INCLUDES_END

static FString OriginalLocale( _wsetlocale(LC_NUMERIC, nullptr) ); // Cache LC_NUMERIC locale before initialization of UE4
static FString NewLocale = _wsetlocale(LC_NUMERIC, TEXT("C"));

HINSTANCE HInstanceMax;

__declspec( dllexport ) bool LibInitialize(void)
{
	// Restore LC_NUMERIC locale after initialization of UE4
	_wsetlocale(LC_NUMERIC, *OriginalLocale);

	UE_SET_LOG_VERBOSITY(LogDatasmithMaxExporter, Verbose);
	return true;
}

__declspec( dllexport ) int LibShutdown()
{
	DatasmithMaxDirectLink::ShutdownExporter();

	// Set GIsRequestingExit flag so that static dtors don't crash
	if (!IsEngineExitRequested())
	{
		RequestEngineExit(TEXT("LibShutdown received"));
	}

	return TRUE;
}


__declspec(dllexport) const TCHAR* LibDescription()
{
	return TEXT("Unreal Datasmith Exporter With DirectLink Support");
}

// Return version so can detect obsolete DLLs
__declspec(dllexport) ULONG LibVersion()
{

	return VERSION_3DSMAX;
}

__declspec(dllexport) int LibNumberClasses()
{
	return 0;
}

__declspec(dllexport) ClassDesc* LibClassDesc(int i)
{
	return nullptr;
}

/** public functions **/
BOOL WINAPI DllMain(HINSTANCE hinstDLL, ULONG FdwReason, LPVOID LpvReserved)
{
	switch(FdwReason) {
	case DLL_PROCESS_ATTACH:{
		MaxSDK::Util::UseLanguagePackLocale();
		HInstanceMax = hinstDLL;
		DisableThreadLibraryCalls(HInstanceMax);
		break;
	}
	case DLL_PROCESS_DETACH:
	{
		break;
	}
	};
	return (TRUE);
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // NEW_DIRECTLINK_PLUGIN
