// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithMaxExporterUtils.h"

#include "IDatasmithSceneElements.h"

namespace DatasmithMaxExporterUtils
{
	void ExportMaxTagsForDatasmithActor(const TSharedPtr<IDatasmithActorElement>& ActorElement, INode* Node, INode* ParentNode, TMap<TPair<uint32, TPair<uint32, uint32>>, MAXClass*>& KnownMaxClass, TMap<uint32, MAXSuperClass*>& KnownMaxSuperClass)
	{
		if (Node && ActorElement.IsValid())
		{
			const ObjectState& objState = Node->EvalWorldState(GetCOREInterface()->GetTime());

			if (Object* Obj = objState.obj)
			{
				// Extract the super class name
				SClass_ID SuperClassID = Obj->SuperClassID();
				MAXSuperClass* SuperClass = nullptr;
				MAXSuperClass** PtrToSuperClass = KnownMaxSuperClass.Find((uint32)SuperClassID);

				if (PtrToSuperClass)
				{
					SuperClass = *PtrToSuperClass;
				}
				else
				{
					SuperClass = lookup_MAXSuperClass(SuperClassID);
					KnownMaxSuperClass.Add((uint32)SuperClassID, SuperClass);
				}

				if (SuperClass)
				{
					Value* NameAsValue = SuperClass->name;
					if (NameAsValue)
					{
						const MCHAR* Name = NameAsValue->to_string();
						if (Name)
						{
							ActorElement->AddTag(*FString::Printf(TEXT("Max.superclassof: %s"), (const TCHAR*)Name));
						}
					}
				}

				// Extract the class name without localization
				Class_ID ClassID = Obj->ClassID();

				TPair<uint32, uint32> ClassIDAsPair = TPair<uint32, uint32>((uint32)ClassID.PartA(), (uint32)ClassID.PartB());
				TPair<uint32, TPair<uint32, uint32>> ClassKey = TPair<uint32, TPair<uint32, uint32>>((uint32)SuperClassID, ClassIDAsPair);

				MAXClass* Class = nullptr;
				MAXClass** PtrToClass = KnownMaxClass.Find( ClassKey );

				if (PtrToClass)
				{
					Class = *PtrToClass;
				}
				else
				{
					Class = lookup_MAXClass(&ClassID, SuperClassID);
					KnownMaxClass.Add(ClassKey, Class);
				}

				if (Class)
				{
					Value* NameAsValue = Class->name;
					if (NameAsValue)
					{
						const MCHAR* Name = NameAsValue->to_string();
						if (Name)
						{
							ActorElement->AddTag(*FString::Printf(TEXT("Max.classof: %s"), (const TCHAR*)Name));
						}
					}
				}
			}

			// Rest of the extracted information for the basic 3ds max tags
			ActorElement->AddTag(*FString::Printf(TEXT("Max.handle: %lu"), Node->GetHandle()));
			ActorElement->AddTag(*FString::Printf(TEXT("Max.isGroupHead: %s"), Node->IsGroupHead() ? TEXT("true") : TEXT("false")));
			ActorElement->AddTag(*FString::Printf(TEXT("Max.isGroupMember: %s"), Node->IsGroupMember() ? TEXT("true") : TEXT("false")));

			if (ParentNode)
			{
				ActorElement->AddTag(*FString::Printf(TEXT("Max.parent.handle: %lu"), ParentNode->GetHandle()));
			}
			if (INode* Target = Node->GetTarget())
			{
				ActorElement->AddTag(*FString::Printf(TEXT("Max.Target.handle: %lu"), Target->GetHandle()));
			}
			if (INode* LookAt = Node->GetLookatNode())
			{
				ActorElement->AddTag(*FString::Printf(TEXT("Max.LookAt.handle: %lu"), LookAt->GetHandle()));
			}
		}
	}
}

