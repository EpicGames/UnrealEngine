// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef NEW_DIRECTLINK_PLUGIN

#include "DatasmithMaxDirectLink.h"

#include "DatasmithMaxSceneExporter.h"
#include "DatasmithMaxHelper.h"
#include "DatasmithMaxWriter.h"
#include "DatasmithMaxClassIDs.h"

#include "DatasmithMaxLogger.h"
#include "DatasmithMaxSceneHelper.h"
#include "DatasmithMaxCameraExporter.h"
#include "DatasmithMaxAttributes.h"
#include "DatasmithMaxProgressManager.h"
#include "DatasmithMaxMeshExporter.h"
#include "DatasmithMaxExporterUtils.h"

#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"

#include "DatasmithExporterManager.h"
#include "DatasmithExportOptions.h"
#include "DatasmithSceneExporter.h"

#include "DatasmithMesh.h"
#include "DatasmithMeshExporter.h"

#include "IDatasmithExporterUIModule.h"
#include "IDirectLinkUI.h"



#include "DatasmithSceneFactory.h"

#include "DatasmithDirectLink.h"

#include "HAL/PlatformTime.h"
#include "Logging/LogMacros.h"

#include "Async/Async.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "Max.h"
	#include "bitmap.h"
	#include "gamma.h"

	#include "notify.h"

	#include "ilayer.h"
	#include "ilayermanager.h"

	#include "ISceneEventManager.h"

	#include "IFileResolutionManager.h" // for GetActualPath

	#include "xref/iXrefObj.h"

	#include "maxscript/maxscript.h"
MAX_INCLUDES_END

// Make FMD5Hash usable in TMap as a key
inline uint32 GetTypeHash(const FMD5Hash& Hash)
{
	uint32* HashAsInt32 = (uint32*)Hash.GetBytes();
	return HashAsInt32[0] ^ HashAsInt32[1] ^ HashAsInt32[2] ^ HashAsInt32[3];
}

namespace DatasmithMaxDirectLink
{

typedef Texmap* FTexmapKey;

class FDatasmith3dsMaxScene
{
public:
	FDatasmith3dsMaxScene()
	{
		ResetScene();
	}

	void ResetScene()
	{
		DatasmithSceneRef.Reset();
		SceneExporterRef.Reset();
	}

	void SetupScene()
	{
		DatasmithSceneRef = FDatasmithSceneFactory::CreateScene(TEXT(""));
		SceneExporterRef = MakeShared<FDatasmithSceneExporter>();

		MSTR Renderer;
		FString Host;
		Host = TEXT("Autodesk 3dsmax ") + FString::FromInt(MAX_VERSION_MAJOR) + TEXT(".") + FString::FromInt(MAX_VERSION_MINOR) + TEXT(".") + FString::FromInt(MAX_VERSION_POINT);
		GetCOREInterface()->GetCurrentRenderer()->GetClassName(Renderer);

		DatasmithSceneRef->SetProductName(TEXT("3dsmax"));
		DatasmithSceneRef->SetHost( *( Host + Renderer ) );

		// Set the vendor name of the application used to build the scene.
		DatasmithSceneRef->SetVendor(TEXT("Autodesk"));

		FString Version = FString::FromInt(MAX_VERSION_MAJOR) + TEXT(".") + FString::FromInt(MAX_VERSION_MINOR) + TEXT(".") + FString::FromInt(MAX_VERSION_POINT);
		DatasmithSceneRef->SetProductVersion(*Version);
	}

	TSharedPtr<IDatasmithScene> GetDatasmithScene()
	{
		return DatasmithSceneRef;
	}

	FDatasmithSceneExporter& GetSceneExporter()
	{
		return *SceneExporterRef;
	}

	void SetName(const TCHAR* InName)
	{
		SceneExporterRef->SetName(InName);
		DatasmithSceneRef->SetName(InName);
		DatasmithSceneRef->SetLabel(InName);
	}

	void SetOutputPath(const TCHAR* InOutputPath)
	{
		// Set the output folder where this scene will be exported.
		SceneExporterRef->SetOutputPath(InOutputPath);
		DatasmithSceneRef->SetResourcePath(SceneExporterRef->GetOutputPath());
	}

	void PreExport()
	{
		// Start measuring the time taken to export the scene.
		SceneExporterRef->PreExport();
	}

	TSharedPtr<IDatasmithScene> DatasmithSceneRef;
	TSharedPtr<FDatasmithSceneExporter> SceneExporterRef;
};

class FNodeTrackerHandle
{
public:
	explicit FNodeTrackerHandle(FNodeKey InNodeKey, INode* InNode) : Impl(MakeShared<FNodeTracker>(InNodeKey, InNode)) {}

	FNodeTracker* GetNodeTracker() const
	{
		return Impl.Get();
	}

private:
	TSharedPtr<FNodeTracker> Impl;
};

// Every node which is resolved to the same object is considered an instance
// This class holds all the nodes which resolve to the same object
struct FInstances: FNoncopyable
{
	AnimHandle Handle; // Handle of anim that is instanced

	Object* EvaluatedObj = nullptr;
	Mtl* Material = nullptr; // Material assigned to Datasmith StaticMesh, used to check if a particular instance needs to override it

	TSet<class FNodeTracker*> NodeTrackers;

	// Datasmith mesh conversion results
	FMeshConverted Converted;
	Interval ValidityInterval;
	bool bShouldBakePivot; // Pivot(when it's common for all instances) should be baked into vertices in order to avoid creating extra actors
	FTransform BakePivot; // The pivot transform to bake vertices with

	bool HasMesh()
	{
		return Converted.GetDatasmithMeshElement().IsValid();
	}

	const TCHAR* GetStaticMeshPathName()
	{
		return Converted.GetDatasmithMeshElement()->GetName();
	}

	// Record wich material is assigned to static mesh
	void AssignMaterialToStaticMesh(Mtl* InMaterial)
	{
		Material = InMaterial;
	}
};

// todo: rename Manager
//   Groups all geometry nodes by their prototype object(geom they resolve to)
class FInstancesManager
{
public:
	void Reset()
	{
		InstancesForAnimHandle.Reset();
	}

	FInstances& AddNodeTracker(FNodeTracker& NodeTracker, FMeshNodeConverter& Converter, Object* Obj)
	{
		TUniquePtr<FInstances>& Instances = InstancesForAnimHandle.FindOrAdd(Converter.InstanceHandle);

		if (!Instances)
		{
			Instances = MakeUnique<FInstances>();
			Instances->Handle = Converter.InstanceHandle;
			Instances->EvaluatedObj = Obj;
		}

		// need to invalidate mesh assignment to node that wasn't the first to add to instances(so if instances weren't invalidated - this node needs mesh anyway)
		Instances->NodeTrackers.Add(&NodeTracker);
		return *Instances;
	}

	FInstances* RemoveNodeTracker(FNodeTracker& NodeTracker)
	{
		if (FInstances* Instances = GetInstancesForNodeTracker(NodeTracker))
		{
			// need to invalidate mesh assignment to node that wasn't the first to add to instances(so if instances weren't invalidated - this node needs mesh anyway)
			Instances->NodeTrackers.Remove(&NodeTracker);
			return Instances;
		}

		return nullptr;
	}

	FInstances* GetInstancesForNodeTracker(FNodeTracker& NodeTracker)
	{
		if (!ensure(NodeTracker.GetConverter().ConverterType == FNodeConverter::MeshNode))
		{
			return nullptr;
		}
		if (TUniquePtr<FInstances>* InstancesPtr = InstancesForAnimHandle.Find(static_cast<FMeshNodeConverter&>(NodeTracker.GetConverter()).InstanceHandle))
		{
			return InstancesPtr->Get();
		}
		return nullptr;
	}

	void RemoveInstances(const FInstances& Instances)
	{
		ensure(Instances.NodeTrackers.IsEmpty()); // Supposed to remove only unused Instances
		InstancesForAnimHandle.Remove(Instances.Handle);
	}

private:
	TMap<AnimHandle, TUniquePtr<FInstances>> InstancesForAnimHandle; // set of instanced nodes for each AnimHandle
};

class FLayerTracker
{
public:
	FLayerTracker(const FString& InName, bool bInIsHidden): Name(InName), bIsHidden(bInIsHidden)
	{
	}

	void SetName(const FString& InName)
	{
		if (Name == InName)
		{
			return;
		}
		bIsInvalidated = true;
		Name = InName;
	}

	void SetIsHidden(bool bInIsHidden)
	{
		if (bIsHidden == bInIsHidden)
		{
			return;
		}
		bIsInvalidated = true;
		bIsHidden = bInIsHidden;
	}

	FString Name;
	bool bIsHidden = false;

	bool bIsInvalidated = true;
};

class FUpdateProgress
{
public:
	class FStage: FNoncopyable
	{
	public:
		FUpdateProgress& UpdateProgress;

		FStage(FUpdateProgress& InUpdateProgress, const TCHAR* InName, int32 InStageCount) : UpdateProgress(InUpdateProgress), Name(InName), StageCount(InStageCount)
		{
			TimeStart =  FDateTime::UtcNow();
		}

		void Finished()
		{
			TimeFinish =  FDateTime::UtcNow();
		}

		FStage& ProgressStage(const TCHAR* SubstageName, int32 InStageCount)
		{
			LogDebug(SubstageName);
			if (UpdateProgress.ProgressManager)
			{
				StageIndex++;
				UpdateProgress.ProgressManager->SetMainMessage(*FString::Printf(TEXT("%s (%d of %d)"), SubstageName, StageIndex, StageCount));
				UpdateProgress.ProgressManager->ProgressEvent(0, TEXT(""));
			}
			return Stages.Emplace_GetRef(UpdateProgress, SubstageName, InStageCount);
		}

		void ProgressEvent(float Progress, const TCHAR* Message)
		{
			LogDebug(FString::Printf(TEXT("%f %s"), Progress, Message));
			if (UpdateProgress.ProgressManager)
			{
				UpdateProgress.ProgressManager->ProgressEvent(Progress, Message);
			}
		}

		void SetResult(const FString& Text)
		{
			Result = Text;
		}

		FString Name;
		int32 StageCount;
		int32 StageIndex = 0;

		FDateTime TimeStart;
		FDateTime TimeFinish;

		FString Result;

		TArray<FStage> Stages;
	};

	FUpdateProgress(bool bShowProgressBar, int32 InStageCount) : MainStage(*this, TEXT("Total"), InStageCount)
	{
		if (bShowProgressBar)
		{
			ProgressManager = MakeUnique<FDatasmithMaxProgressManager>();
		}
	}

	void PrintStatisticss()
	{
		PrintStage(MainStage);
	}

	void PrintStage(FStage& Stage, FString Indent=TEXT(""))
	{
		LogCompletion(Indent + FString::Printf(TEXT("    %s - %s"), *Stage.Name, *(Stage.TimeFinish-Stage.TimeStart).ToString()));
		if (!Stage.Result.IsEmpty())
		{
			LogInfo(Indent + TEXT("      #") + Stage.Result);
		}
		for(FStage& ChildStage: Stage.Stages)
		{
			PrintStage(ChildStage, Indent + TEXT("  "));
		}
	}

	void Finished()
	{
		MainStage.Finished();
	}

	TUniquePtr<FDatasmithMaxProgressManager> ProgressManager;

	FStage MainStage;
};

class FProgressCounter
{
public:

	FProgressCounter(FUpdateProgress::FStage& InProgressStage, int32 InCount)
		: ProgressStage(InProgressStage)
		, Count(InCount)
		, SecondsOfLastUpdate(FPlatformTime::Seconds())
	{
	}

	void Next()
	{
		double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime - SecondsOfLastUpdate > UpdateIntervalMin) // Don't spam progress bar
		{
			ProgressStage.ProgressEvent(float(Index) / Count, *FString::Printf(TEXT("%d of %d"), Index, Count) );
			SecondsOfLastUpdate = CurrentTime;
		}
		Index++;
	}
private:
	FUpdateProgress::FStage& ProgressStage;
	int32 Count;
	int32 Index = 0;
	const double UpdateIntervalMin = 0.05; // Don't update progress it last update was just recently
	double SecondsOfLastUpdate;
};

class FProgressStageGuard
{
public:
	FUpdateProgress::FStage& Stage;
	FProgressStageGuard(FUpdateProgress::FStage& ParentStage, const TCHAR* InName, int32 Count=0) : Stage(ParentStage.ProgressStage(InName, Count))
	{
	}

	~FProgressStageGuard()
	{
		Stage.Finished();
		if (ComputeResultDeferred)
		{
			Stage.Result = ComputeResultDeferred();
		}
	}

	TFunction<FString()> ComputeResultDeferred;
};

#define PROGRESS_STAGE(Name) FProgressStageGuard ProgressStage(MainStage, TEXT(Name));
#define PROGRESS_STAGE_COUNTER(Count) FProgressCounter ProgressCounter(ProgressStage.Stage, Count);
#define PROGRESS_STAGE_RESULT(Text) ProgressStage.Stage.SetResult(Text);
// Simplily creation of Stage result (called in Guard dtor)
#define PROGRESS_STAGE_RESULT_DEFERRED ProgressStage.ComputeResultDeferred = [&]()

// Convert various node data to Datasmith tags
class FTagsConverter
{
public:
	void ConvertNodeTags(FNodeTracker& NodeTracker)
	{
		INode* Node = NodeTracker.Node;
		INode* ParentNode = Node->GetParentNode();
		DatasmithMaxExporterUtils::ExportMaxTagsForDatasmithActor( NodeTracker.GetConverted().DatasmithActorElement, Node, ParentNode, KnownMaxDesc, KnownMaxSuperClass );
	}

private:
	// We don't know how the 3ds max lookup_MaxClass is implemented so we use this map to skip it when we can
	TMap<TPair<uint32, TPair<uint32, uint32>>, MAXClass*> KnownMaxDesc;
	// Same for the lookup_MAXSuperClass.
	TMap<uint32, MAXSuperClass*> KnownMaxSuperClass;

};


// In order to retrieve Render geometry rather than Viewport geometry
// RenderBegin need to be called for all RefMakers to be exported (and RenderEnd afterwards)
// e.g. When using Optimize modifier on a geometry it has separate LODs for Render and Viewport and
// GetRenderMesh would return Viewport lod if called without RenderBegin first. Consequently
// without RenderEnd it would display Render LOD in viewport.
class FNodesPreparer
{
public:

	class FBeginRefEnumProc : public RefEnumProc
	{
	public:
		void SetTime(TimeValue StartTime)
		{
			Time = StartTime;
		}

		virtual int proc(ReferenceMaker* RefMaker) override
		{
			RefMaker->RenderBegin(Time);
			return REF_ENUM_CONTINUE;
		}

	private:
		TimeValue Time;
	};

	class FEndRefEnumProc : public RefEnumProc
	{
	public:
		void SetTime(TimeValue EndTime)
		{
			Time = EndTime;
		}

		virtual int32 proc(ReferenceMaker* RefMaker) override
		{
			RefMaker->RenderEnd(Time);
			return REF_ENUM_CONTINUE;
		}

	private:
		TimeValue Time;
	};

	void Start(TimeValue Time, bool bInRenderQuality)
	{
		bRenderQuality = bInRenderQuality;

		BeginProc.SetTime(Time);
		EndProc.SetTime(Time);

		if (bRenderQuality)
		{
			BeginProc.BeginEnumeration();
		}
	}

	void Finish()
	{
		if (bRenderQuality)
		{
			BeginProc.EndEnumeration();

			// Call RenderEnd on every node that had RenderBegin called
			EndProc.BeginEnumeration();
			for(INode* Node: NodesPrepared)
			{
				Node->EnumRefHierarchy(EndProc);
			}
			EndProc.EndEnumeration();
			NodesPrepared.Reset();
		}
	}

	void PrepareNode(INode* Node)
	{
		if (bRenderQuality)
		{
			// Skip if node was already Prepared
			bool bIsAlreadyPrepared;
			NodesPrepared.FindOrAdd(Node, &bIsAlreadyPrepared);
			if (bIsAlreadyPrepared)
			{
				return;
			}

			Node->EnumRefHierarchy(BeginProc);
		}
	}

	bool bRenderQuality = false; // If need to call RenderBegin on all nodes to make them return Render-quality mesh

	FBeginRefEnumProc BeginProc;
	FEndRefEnumProc EndProc;

	TSet<INode*> NodesPrepared;
};

struct FExportOptions
{
	bool bSelectedOnly = false;
	bool bAnimatedTransforms = false;

	bool bStatSync = false;
	int32 TextureResolution = 4;

	static const int32 TextureResolutionMax = 6; //

	bool bXRefScenes = true;
};

// Global export options, stored in preferences
class FPersistentExportOptions: public IPersistentExportOptions
{
public:
	void Load()
	{
		if (bLoaded)
		{
			return;
		}
		GetBool(TEXT("AnimatedTransforms"), Options.bAnimatedTransforms);
		GetInt(TEXT("TextureResolution"), Options.TextureResolution);
		GetBool(TEXT("XRefScenes"), Options.bXRefScenes);
		bLoaded = true;
	}

	void GetBool(const TCHAR* Name, bool& bValue)
	{
		if (!GConfig)
		{
			return;
		}
		FString ConfigPath = GetConfigPath();
		GConfig->GetBool(TEXT("Export"), Name, bValue, ConfigPath);
	}

	void SetBool(const TCHAR* Name, bool bValue)
	{
		if (!GConfig)
		{
			return;
		}
		FString ConfigPath = GetConfigPath();
		GConfig->SetBool(TEXT("Export"), Name, bValue, ConfigPath);
		GConfig->Flush(false, ConfigPath);
	}

	void GetInt(const TCHAR* Name, int32& bValue)
	{
		if (!GConfig)
		{
			return;
		}
		FString ConfigPath = GetConfigPath();
		GConfig->GetInt(TEXT("Export"), Name, bValue, ConfigPath);
	}

	void SetInt(const TCHAR* Name, int32 bValue)
	{
		if (!GConfig)
		{
			return;
		}
		FString ConfigPath = GetConfigPath();
		GConfig->SetInt(TEXT("Export"), Name, bValue, ConfigPath);
		GConfig->Flush(false, ConfigPath);
	}

	FString GetConfigPath()
	{
#if MAX_PRODUCT_YEAR_NUMBER >= 2025
		FString PlugCfgPath = (TCHAR*)GetCOREInterface()->GetDir(APP_PLUGCFG_DIR).data();
#else
		FString PlugCfgPath = GetCOREInterface()->GetDir(APP_PLUGCFG_DIR);
#endif
		return FPaths::Combine(PlugCfgPath, TEXT("UnrealDatasmithMax.ini"));
	}

	virtual void SetAnimatedTransforms(bool bValue) override
	{
		Options.bAnimatedTransforms = bValue;
		SetBool(TEXT("AnimatedTransforms"), Options.bAnimatedTransforms);
	}

	virtual bool GetAnimatedTransforms() override
	{
		return Options.bAnimatedTransforms;
	}

	virtual void SetXRefScenes(bool bValue) override
	{
		Options.bXRefScenes = bValue;
		SetBool(TEXT("XRefScenes"), Options.bXRefScenes);
	}

	virtual bool GetXRefScenes() override
	{
		return Options.bXRefScenes;
	}

	virtual void SetStatSync(bool bValue) override
	{
		Options.bStatSync = bValue;
		SetBool(TEXT("StatExport"), Options.bStatSync);
	}

	virtual bool GetStatSync() override
	{
		return Options.bStatSync;
	}

	virtual void SetTextureResolution(int32 Value) override
	{
		
		Options.TextureResolution = FMath::Clamp(Value, 0, FExportOptions::TextureResolutionMax);
		SetInt(TEXT("TextureResolution"), Options.TextureResolution);
	}

	virtual int32 GetTextureResolution() override
	{
		return Options.TextureResolution;
	}

	FExportOptions Options;
	bool bLoaded = false;
};

class FIncludeXRefGuard
{
public:


	FIncludeXRefGuard(bool bInIncludeXRefWhileParsing): bIncludeXRefWhileParsing(bInIncludeXRefWhileParsing)
	{
		if (bIncludeXRefWhileParsing)
		{
			bIncludeXRefsInHierarchyStored = GetCOREInterface()->GetIncludeXRefsInHierarchy();
			GetCOREInterface()->SetIncludeXRefsInHierarchy(true);
		}
	}
	~FIncludeXRefGuard()
	{
		if (bIncludeXRefWhileParsing)
		{
			GetCOREInterface()->SetIncludeXRefsInHierarchy(bIncludeXRefsInHierarchyStored);
		}
		
	}

	bool bIncludeXRefWhileParsing;
	BOOL bIncludeXRefsInHierarchyStored;
};

class FInvalidatedNodeTrackers: FNoncopyable
{
public:

	void Add(FNodeTracker& NodeTracker)
	{
		check(!IteratorUsageCount);
		InvalidatedNodeTrackers.Add(&NodeTracker);
	}

	int32 Num()
	{
		return InvalidatedNodeTrackers.Num();
	}

	void Append(const TSet<FNodeTracker*>& NodeTrackers)
	{
		check(!IteratorUsageCount);
		InvalidatedNodeTrackers.Append(NodeTrackers);
	}

	// Called when update is finished and all changes are processed and recorded
	void Finish()
	{
		check(!IteratorUsageCount);
		InvalidatedNodeTrackers.Reset();
	}

	// Scene is reset so invalidation is reset too
	void Reset()
	{
		check(!IteratorUsageCount);
		InvalidatedNodeTrackers.Reset();
	}

	bool HasInvalidated()
	{
		return !InvalidatedNodeTrackers.IsEmpty();
	}

	void RemoveFromInvalidated(FNodeTracker& NodeTracker)
	{
		check(!IteratorUsageCount);
		InvalidatedNodeTrackers.Remove(&NodeTracker);
	}

	/**
	 * @return if anything was deleted
	 */
	bool PurgeDeletedNodeTrackers(class FSceneTracker& Scene);

	TArray<FNodeTracker*> Copy()
	{
		return InvalidatedNodeTrackers.Array();
	}

	// Iterate invalidated nodes checking that set of invalidated nodes is not changed during iteration
	class TBaseIterator: private FNoncopyable
	{
	private:

		typedef FNodeTracker ItElementType;

	public:

		using ElementItType = TSet<FNodeTracker*>::TRangedForIterator;

		FInvalidatedNodeTrackers& InvalidatedNodeTrackers;

		FORCEINLINE TBaseIterator(FInvalidatedNodeTrackers& InInvalidatedNodeTrackers, const ElementItType& InElementIt)
			: InvalidatedNodeTrackers(InInvalidatedNodeTrackers), ElementIt(InElementIt)
		{
			InvalidatedNodeTrackers.IteratorUsageCount++;
		}

		FORCEINLINE ~TBaseIterator()
		{
			InvalidatedNodeTrackers.IteratorUsageCount--;
		}

		FORCEINLINE TBaseIterator& operator++()
		{
			++Index;
			++ElementIt;
			return *this;
		}

		FORCEINLINE ItElementType* operator->() const
		{
			return *ElementIt;
		}
		FORCEINLINE ItElementType& operator*() const
		{
			return **ElementIt;
		}

		FORCEINLINE friend bool operator==(const TBaseIterator& Lhs, const TBaseIterator& Rhs) { return Lhs.ElementIt == Rhs.ElementIt; }
		FORCEINLINE friend bool operator!=(const TBaseIterator& Lhs, const TBaseIterator& Rhs) { return Lhs.ElementIt != Rhs.ElementIt; }

		ElementItType ElementIt;
		int32 Index = 0; // For simpler debugging
	};

	using TRangedForIterator=TBaseIterator;

	FORCEINLINE TRangedForIterator begin() { return TRangedForIterator(*this, InvalidatedNodeTrackers.begin()); }
	FORCEINLINE TRangedForIterator end() { return TRangedForIterator(*this, InvalidatedNodeTrackers.end()); }

private:
	TSet<FNodeTracker*> InvalidatedNodeTrackers;

	int32 IteratorUsageCount = 0;
};

class FNodeTrackersNames: FNoncopyable
{
public:
	TMap<FString, TSet<FNodeTracker*>> NodesForName;  // Each name can be used by a set of nodes
	void Reset()
	{
		NodesForName.Reset();
	}

	const FString& GetNodeName(FNodeTracker& NodeTracker)
	{
		return NodeTracker.Name;
	}

	void Update(FNodeTracker& NodeTracker)
	{
		FString Name = NodeTracker.Node->GetName();
		if (Name != NodeTracker.Name)
		{
			NodesForName[NodeTracker.Name].Remove(&NodeTracker);

			NodeTracker.Name = Name;
			NodesForName.FindOrAdd(NodeTracker.Name).Add(&NodeTracker);
		}
	}

	void Add(FNodeTracker& NodeTracker)
	{
		FString Name = NodeTracker.Node->GetName();

		// todo: dupclicated with Update
		NodeTracker.Name = Name;
		NodesForName.FindOrAdd(NodeTracker.Name).Add(&NodeTracker);
	}

	void Remove(const FNodeTracker& NodeTracker)
	{
		if (TSet<FNodeTracker*>* NodeTrackersPtr = NodesForName.Find(NodeTracker.Name))
		{
			NodeTrackersPtr->Remove(&NodeTracker);
		}
	}

	template <typename Func>
	void EnumerateForName(const FString& Name, Func Callable)
	{
		if (TSet<FNodeTracker*>* NodeTrackersPtr = NodesForName.Find(Name))
		{
			for (FNodeTracker* NodeTracker: *NodeTrackersPtr)
			{
				Callable(*NodeTracker);
			}
		}
	}
};

#ifdef CANCEL_DEBUG_ENABLE

	// Cancel handling implementation that allows cancelling from Max Script (e.g. to trigger it faster than human can)
	class FCancel
	{
	public:

		void Reset()
		{
			bUseGetCancelMxsCallback = true;
		}

		bool GetCancelMxsCallback(bool& bSuccess)
		{
			MCHAR* ScriptCode = _T("Datasmith_GetCancel()");
			FPValue Result;
	#if MAX_PRODUCT_YEAR_NUMBER >= 2022
			bSuccess = ExecuteMAXScriptScript(ScriptCode, MAXScript::ScriptSource::NonEmbedded, FALSE, &Result) != FALSE;
	#else
			bSuccess = ExecuteMAXScriptScript(ScriptCode, FALSE, &Result) != FALSE;
	#endif
			if (bSuccess)
			{
				check (Result.type == TYPE_BOOL);
				return Result.b != FALSE;
			}
			return false;
		}

		bool GetCancel()
		{
			if (bUseGetCancelMxsCallback)
			{
				return GetCancelMxsCallback(bUseGetCancelMxsCallback); // Stop using callback if it fails(i.e. no mxs function defined or another error)
			}

			return GetCOREInterface()->GetCancel() != FALSE;
		}

	private:
		bool bUseGetCancelMxsCallback = false;
	};
#else
	class FCancel
	{
	public:

		void Reset()
		{
		}

		bool GetCancel()
		{
			return GetCOREInterface()->GetCancel() != FALSE;
		}
	};
#endif

class FIesTexturesCollection: FNoncopyable
{
public:

	struct FTextureIesConverted
	{
		FTextureIesConverted(const TSharedPtr<IDatasmithTextureElement>& InTextureElement)
			: TextureElement(InTextureElement)
			, UsageCount(0)
		{
		}

		TSharedPtr<IDatasmithTextureElement> TextureElement;
		int32 UsageCount;
	};

	FIesTexturesCollection(ISceneTracker& InSceneTracker): SceneTracker(InSceneTracker)
	{
	}

	// Cache Ies textures by their path's base name
	FString GetIesTextureKey(const FString& IesFilePath)
	{
		return FDatasmithUtils::SanitizeObjectName(FPaths::GetBaseFilename(IesFilePath));
	}

	const TCHAR* AcquireIesTexture(const FString& IesFilePath)
	{
		if (IesFilePath.IsEmpty())
		{
			return nullptr;
		}

		FTextureIesConverted& Texture = GetOrCreateIesTexture(IesFilePath);
		Texture.UsageCount++;
		return Texture.TextureElement->GetName();
	}

	FTextureIesConverted& GetOrCreateIesTexture(const FString& IesFilePath)
	{
		FString Name = GetIesTextureKey(IesFilePath);

		if (FTextureIesConverted* TexturePtr = TexturesIes.Find(Name))
		{
			return *TexturePtr;
		}

		TSharedRef<IDatasmithTextureElement> TextureElement = FDatasmithSceneFactory::CreateTexture(*(Name + FString("_Ies")));

		TextureElement->SetTextureMode(EDatasmithTextureMode::Ies);
		TextureElement->SetLabel(*Name);
		TextureElement->SetFile(*IesFilePath);

		SceneTracker.GetDatasmithSceneRef()->AddTexture(TextureElement);

		return TexturesIes.Emplace(Name, TextureElement);
	}

	void ReleaseIesTexture(const FString& IesFilePath)
	{
		FString BaseName = GetIesTextureKey(IesFilePath);

		if(!TexturesIes.Contains(BaseName))
		{
			return;
		}

		FTextureIesConverted& Texture = TexturesIes[BaseName];
		Texture.UsageCount--;
		if (!Texture.UsageCount)
		{
			SceneTracker.RemoveTexture(Texture.TextureElement);
			TexturesIes.Remove(BaseName);
		}
	}

	void Reset()
	{
		TexturesIes.Reset();
	}

	ISceneTracker& SceneTracker;

	TMap<FString, FTextureIesConverted> TexturesIes;
};


// Holds states of entities for syncronization and handles change events
class FSceneTracker: public ISceneTracker
{
public:
	FSceneTracker(const FExportOptions& InOptions, FDatasmith3dsMaxScene& InExportedScene, FNotifications* InNotificationsHandler)
		: Options(InOptions)
		, ExportedScene(InExportedScene)
		, NotificationsHandler(InNotificationsHandler)
		, MaterialsCollectionTracker(*this)
		, IesTextures(*this)
	{}

	virtual TSharedRef<IDatasmithScene> GetDatasmithSceneRef() override
	{
		return ExportedScene.GetDatasmithScene().ToSharedRef();
	}

	virtual const TCHAR* GetAssetsOutputPath() override
	{
		return ExportedScene.GetSceneExporter().GetAssetsOutputPath();
	}

	bool bParseXRefScenes = true;
	bool bIncludeXRefWhileParsing = false; // note - this should not be set when bParseXRefScenes if disabled

	virtual bool IsUpdateInProgress() override
	{
		return bUpdateInProgress;
	}

	// Parse scene or XRef scene(in this case attach to parent datasmith actor)
	bool ParseScene(INode* SceneRootNode, FXRefScene XRefScene=FXRefScene())
	{
		LogDebugNode(TEXT("ParseScene"), SceneRootNode);
		// todo: do we need Root Datasmith node of scene/XRefScene in the hierarchy?
		// is there anything we need to handle for main file root node?
		// for XRefScene? Maybe addition/removal? Do we need one node to consolidate XRefScene under?

		// nodes comming from XRef Scenes/Objects could be null
		if (!SceneRootNode)
		{
			return true;
		}

		if (XRefScene && !bParseXRefScenes)
		{
			return true; 
		}

		if (!bIncludeXRefWhileParsing)
		{
			// Parse XRefScenes
			for (int XRefChild = 0; XRefChild < SceneRootNode->GetXRefFileCount(); ++XRefChild)
			{
				DWORD XRefFlags = SceneRootNode->GetXRefFlags(XRefChild);

				SCENE_UPDATE_STAT_INC(ParseScene, XRefFileEncountered);

				// XRef is disabled - not shown in viewport/render. Not loaded.
				if (XRefFlags & XREF_DISABLED)
				{
					SCENE_UPDATE_STAT_INC(ParseScene, XRefFileDisabled);
					// todo: baseline - doesn't check this - exports even disabled and XREF_HIDDEN scenes
					continue;
				}

				FString Path = FDatasmithMaxSceneExporter::GetActualPath(SceneRootNode->GetXRefFile(XRefChild).GetFileName());
				if (FPaths::FileExists(Path) == false)
				{
					SCENE_UPDATE_STAT_INC(ParseScene, XRefFileMissing);
					LogWarning(FString::Printf(TEXT("XRefScene file \"%s\" cannot be found"), *FPaths::GetCleanFilename(*Path)));
				}
				else
				{
					SCENE_UPDATE_STAT_INC(ParseScene, XRefFileToParse);
					if (!ParseScene(SceneRootNode->GetXRefTree(XRefChild), FXRefScene{SceneRootNode, XRefChild}))
					{
						return false;
					}
				}
			}
		}

		int32 ChildNum = SceneRootNode->NumberOfChildren();
		for (int32 ChildIndex = 0; ChildIndex < ChildNum; ++ChildIndex)
		{
			if (FNodeTracker* NodeTracker = ParseNode(SceneRootNode->GetChildNode(ChildIndex)))
			{
				// Record XRef this child node is at the root of
				// It's needed restore hierarchy when converting
				NodeTracker->SetXRefIndex(XRefScene);
			}
			else
			{
				return false;
			}
		}
		return true;
	}

	FNodeTracker* ParseNode(INode* Node)
	{
		LogDebugNode(TEXT("ParseNode"), Node);
		SCENE_UPDATE_STAT_INC(ParseNode, NodesEncountered);

		FNodeTracker* NodeTracker =  GetNodeTracker(Node);

		if (GetCancel())
		{
			return nullptr;
		}

		if (NodeTracker)
		{
			InvalidateNode(*NodeTracker, false);
		}
		else
		{
			NodeTracker = AddNode(Node);
		}

		// Parse children
		int32 ChildNum = Node->NumberOfChildren();
		for (int32 ChildIndex = 0; ChildIndex < ChildNum; ++ChildIndex)
		{
			if(!ParseNode(Node->GetChildNode(ChildIndex)))
			{
				return nullptr;
			}
		}
		return NodeTracker;
	}

	void ParseNode(FNodeTracker& NodeTracker)
	{
		if (NodeTracker.Node->IsRootNode())
		{
			ParseScene(NodeTracker.Node);
		}
		else
		{
			ParseNode(NodeTracker.Node);
		}
	}

	// Check every layer and if it's modified invalidate nodes assigned to it
	// 3ds Max doesn't have events for all Layer changes(e.g. Name seems to be just an UI thing and has no notifications) so
	// we need to go through all layers every update to see what's changed
	bool UpdateLayers()
	{
		bool bChangeEncountered = false;

		ILayerManager* LayerManager = GetCOREInterface13()->GetLayerManager();
		int LayerCount = LayerManager->GetLayerCount();

		for (int LayerIndex = 0; LayerIndex < LayerCount; ++LayerIndex)
		{
			ILayer* Layer = LayerManager->GetLayer(LayerIndex);

			AnimHandle Handle = Animatable::GetHandleByAnim(Layer);

			TUniquePtr<FLayerTracker>& LayerTracker = LayersForAnimHandle.FindOrAdd(Handle);

			bool bIsHidden = Layer->IsHidden(TRUE);
			FString Name = Layer->GetName().data();

			if (!LayerTracker)
			{
				LayerTracker = MakeUnique<FLayerTracker>(Name, bIsHidden);
			}

			LayerTracker->SetName(Name);
			LayerTracker->SetIsHidden(bIsHidden);

			if (LayerTracker->bIsInvalidated)
			{
				bChangeEncountered = true;
				if (TSet<FNodeTracker*>* NodeTrackersPtr = NodesPerLayer.Find(LayerTracker.Get()))
				{
					for(FNodeTracker* NodeTracker: *NodeTrackersPtr)
					{
						InvalidateNode(*NodeTracker, false);
					}
				}
				LayerTracker->bIsInvalidated = false;
			}
		}
		return bChangeEncountered;
	}

	// Applies all recorded changes to Datasmith scene
	bool Update(FUpdateProgress::FStage& MainStage, bool bRenderQuality)
	{
		// Disable Undo, editing, redraw, messages during export/sync so that nothing changes the scene
		GetCOREInterface()->EnableUndo(false);
		GetCOREInterface()->DisableSceneRedraw();
		SuspendAll UberSuspend(TRUE, TRUE, TRUE, TRUE, TRUE, TRUE);

		// Flush all updates for SceneEventManager - so they are not received in mid of Update
		// When ProgressBar is updated it calls internal max event loop which can send unprocessed events to the callback
		if (NotificationsHandler)
		{
			NotificationsHandler->PrepareForUpdate();
		}

		DatasmithMaxLogger::Get().Purge();

		bUpdateInProgress = true;
		NodesPreparer.Start(GetCOREInterface()->GetTime(), bRenderQuality);

		bool bResult = false;

		{
			const int32 StageCount = 12;
			FProgressStageGuard ProgressStage(MainStage, TEXT("Update"), StageCount);
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				if (TSharedPtr<IDatasmithScene> ScenePtr = ExportedScene.GetDatasmithScene())
				{
					IDatasmithScene& Scene = *ScenePtr;
					return FString::Printf(TEXT("Actors: %d; Meshes: %d, Materials: %d Textures: %d"), 
					                       Scene.GetActorsCount(),
					                       Scene.GetMeshesCount(),
					                       Scene.GetMaterialsCount(),
					                       Scene.GetTexturesCount()
					);
				}
				return FString(TEXT("<no scene>"));
			};

			bResult = UpdateInternalSafe(ProgressStage.Stage);

		}

		NodesPreparer.Finish();

		// Revert Update flag after RenderEnd was called for all nodes. It fires change events too when
		// switching from Render to Viewport geometry so we need to know to ignore those events.
		bUpdateInProgress = false; 

		UberSuspend.Resume();
		GetCOREInterface()->EnableSceneRedraw();
		GetCOREInterface()->EnableUndo(true);

		return bResult;
	}

	FCancel Cancel;

	bool GetCancel()
	{
		if (Cancel.GetCancel())
		{
			LogWarning("Update Cancelled");
			return true;
		}
		return false;
	}

	bool UpdateInternalSafe(FUpdateProgress::FStage& MainStage)
	{
		bool bResult = false;
		__try
		{
			bResult = UpdateInternal(MainStage);
		}
		__except(EXCEPTION_EXECUTE_HANDLER)
		{
			LogWarning(TEXT("Update finished with exception"));
		}
		return bResult;
	}

	void RemapUVChannels(const TSharedPtr<IDatasmithUEPbrMaterialElement>& MaterialElement, const FDatasmithMaxMeshExporter::FUVChannelsMap& UVChannels)
	{
		// Remap UV channels of all composite textures of each shaders in the material
		for (int32 i = 0; i < MaterialElement->GetExpressionsCount(); ++i)
		{
			IDatasmithMaterialExpression* MaterialExpression = MaterialElement->GetExpression(i);

			if ( MaterialExpression && MaterialExpression->IsSubType( EDatasmithMaterialExpressionType::TextureCoordinate ) )
			{
				IDatasmithMaterialExpressionTextureCoordinate& TextureCoordinate = static_cast< IDatasmithMaterialExpressionTextureCoordinate& >( *MaterialExpression );

				const int32* UVChannel = UVChannels.Find( TextureCoordinate.GetCoordinateIndex() );
				if ( UVChannel != nullptr )
				{
					TextureCoordinate.SetCoordinateIndex( *UVChannel );
				}
			}
		}
	}

	void UpdateHideByCategory(bool& bChangeEncountered)
	{
		DWORD HideByCategoryFlagsNew = GetCOREInterface()->GetHideByCategoryFlags();
		if (bHideByCategoryChanged)
		{
			DWORD HideByCategoryFlagsChanged = HideByCategoryFlags ^ HideByCategoryFlagsNew;

			for (TPair<FNodeKey, FNodeTrackerHandle> NodeKeyAndNodeTracker: NodeTrackers)
			{
				// Excluded ProgressCounter/Cancel handling from this loop - the code is simple, running quickly even for lots of nodes.

				bool bNeedInvalidate = false;
				FNodeTracker* NodeTracker = NodeKeyAndNodeTracker.Value.GetNodeTracker();
				switch (NodeTracker->GetConverterType())
				{
				case FNodeConverter::Unknown:
					bNeedInvalidate = true;
					break;
				case FNodeConverter::MeshNode:
					bNeedInvalidate = (HideByCategoryFlagsChanged & (HIDE_OBJECTS|HIDE_SHAPES)) != 0;
					break;
				case FNodeConverter::HismNode: 
					bNeedInvalidate = (HideByCategoryFlagsChanged & HIDE_OBJECTS) != 0;
					break;
				case FNodeConverter::LightNode:
					bNeedInvalidate = (HideByCategoryFlagsChanged & HIDE_LIGHTS) != 0;
					break;
				case FNodeConverter::CameraNode: 
					bNeedInvalidate = (HideByCategoryFlagsChanged & HIDE_CAMERAS) != 0;
					break;
				case FNodeConverter::HelperNode: 
					bNeedInvalidate = (HideByCategoryFlagsChanged & HIDE_HELPERS) != 0;
					break;
				default: ;
				}
				if (bNeedInvalidate)
				{
					InvalidateNode(*NodeTracker, false);
					bChangeEncountered = true;
				}
			}
			
			bHideByCategoryChanged = false;
		}
		HideByCategoryFlags = HideByCategoryFlagsNew;
	}

	 // returns if any change in scene was encountered and scene update completed(i.e. DirectLink Sync can be run)
	bool UpdateInternal(FUpdateProgress::FStage& MainStage)
	{
		CurrentSyncPoint.Time = GetCOREInterface()->GetTime();

		// Changes present only when there are modified layers(changes checked manually), nodes(notified by Max) or materials(notified by Max with all changes in dependencies)
		bool bChangeEncountered = false;

		Stats.Reset();


		if (bParseXRefScenes != Options.bXRefScenes)
		{
			bParseXRefScenes = Options.bXRefScenes;

			// Clear all tracked nodes when XRef scenes export option changes
			// note: when including xref into hierarchy it might be impossible to invalidate/remove only xref nodes as they are indistinquishable
			//  But when parsing XRef scenes explicitly it seems to be possible but need to track XRefTree Node
			TArray<FNodeKey> NodeTrackerKeys;
			NodeTrackers.GetKeys(NodeTrackerKeys);
			for (FNodeKey NodeTrackerKey: NodeTrackerKeys)
			{
				RemoveNodeTracker(*GetNodeTracker(NodeTrackerKey));
			}
		}

		{
			PROGRESS_STAGE("Refresh layers")
			bChangeEncountered |= UpdateLayers();
		}

		Cancel.Reset();
		if(GetCancel())
		{
			return false;	
		}


		{
			PROGRESS_STAGE("Remove deleted nodes")
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsRemoveDeletedNodes();
			};

			bChangeEncountered |= InvalidatedNodeTrackers.PurgeDeletedNodeTrackers(*this);
		}

		{
			PROGRESS_STAGE("Check Hide By Category")
			UpdateHideByCategory(bChangeEncountered);
		}

		{
			PROGRESS_STAGE("Check Nodes Validity")
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsCheckTimeSliderValidity();
			};

			FIncludeXRefGuard IncludeXRefGuard(bIncludeXRefWhileParsing);

			FNodeTracker* NodeTracker = GetNodeTracker(GetCOREInterface()->GetRootNode());
			if (NodeTracker)
			{
				InvalidateOutdatedNodeTracker(*NodeTracker);
			}
			else
			{
				// Add root node if it wasn't before(first Update)
				AddNode(GetCOREInterface()->GetRootNode());
			}
		}

		{
			PROGRESS_STAGE("Parse Invalidated Node Hierachy")
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsParseScene();
			};

			FIncludeXRefGuard IncludeXRefGuard(bIncludeXRefWhileParsing);

			PROGRESS_STAGE_COUNTER(InvalidatedNodeTrackers.Num());
			for (FNodeTracker* NodeTracker: InvalidatedNodeTrackers.Copy())
			{
				ProgressCounter.Next();
				if(GetCancel())
				{
					return false;	
				}

				ParseNode(*NodeTracker);
			}
		}

		{
			PROGRESS_STAGE("Refresh collisions") // Update set of nodes used for collision
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsRefreshCollisions();
			};

			PROGRESS_STAGE_COUNTER(InvalidatedNodeTrackers.Num());
			TSet<FNodeTracker*> NodesWithChangedCollisionStatus; // Need to invalidate these nodes to make them renderable or to keep them from renderable depending on collision status

			for (FNodeTracker& NodeTracker: InvalidatedNodeTrackers)
			{
				ProgressCounter.Next();
				if(GetCancel())
				{
					return false;	
				}

				UpdateCollisionStatus(NodeTracker, NodesWithChangedCollisionStatus);
			}

			// Rebuild all nodes that has changed them being colliders
			for (FNodeTracker* NodeTracker : NodesWithChangedCollisionStatus)
			{
				InvalidateNode(*NodeTracker, false);
			}

			SCENE_UPDATE_STAT_SET(RefreshCollisions, ChangedNodes, NodesWithChangedCollisionStatus.Num());

		}

		{
			PROGRESS_STAGE("Process invalidated nodes")
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsProcessInvalidatedNodes();
			};

			PROGRESS_STAGE_COUNTER(InvalidatedNodeTrackers.Num());

			for (FNodeTracker& NodeTracker: InvalidatedNodeTrackers)
			{
				ProgressCounter.Next();
				if(GetCancel())
				{
					return false;	
				}

				UpdateNode(NodeTracker);
			}
		}

		{
			PROGRESS_STAGE("Process geometry")
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsProcessInvalidatedInstances();
			};
			PROGRESS_STAGE_COUNTER(InvalidatedInstances.Num());
			for (FInstances* Instances : InvalidatedInstances)
			{
				ProgressCounter.Next();
				if(GetCancel())
				{
					return false;	
				}

				UpdateInstances(*Instances);

				// Need to re-convert and reattach all instances of an updated node
				InvalidatedNodeTrackers.Append(Instances->NodeTrackers);
			}

			InvalidatedInstances.Reset();
		}

		{
			PROGRESS_STAGE("Convert nodes to datasmith")
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsConvertNodesToDatasmith();
			};

			PROGRESS_STAGE_COUNTER(InvalidatedNodeTrackers.Num());
			for (FNodeTracker& NodeTracker: InvalidatedNodeTrackers)
			{
				ProgressCounter.Next();
				if(GetCancel())
				{
					return false;	
				}

				if (NodeTracker.HasConverter())
				{
					SCENE_UPDATE_STAT_INC(ConvertNodes, Converted)
					NodeTracker.GetConverter().ConvertToDatasmith(*this, NodeTracker);
				}
			}
		}

		{
			PROGRESS_STAGE("Reparent Datasmith Actors");
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsReparentDatasmithActors();
			};
			
			for (FNodeTracker& NodeTracker: InvalidatedNodeTrackers)
			{
				// if(GetCancel())
				// {
				// 	return false;	
				// }
				AttachNodeToDatasmithScene(NodeTracker);
			}
		}

		{
			PROGRESS_STAGE("Mark nodes validated")
			// Finally mark all nodetrackers as valid(we didn't interrupt update as it went to this point)
			// And calculate subtree validity for each subtree of each updated node

			bChangeEncountered |= InvalidatedNodeTrackers.HasInvalidated(); // Right before resetting invalidated nodes, record that anything was invalidated

			// Recalculate subtree validity using updated nodes validity
			UpdateSubtreeValidity(*GetNodeTracker(GetCOREInterface()->GetRootNode()));

			for (FNodeTracker& NodeTracker: InvalidatedNodeTrackers)
			{
				NodeTracker.SubtreeValidity.SetValid();
				NodeTracker.Validity.SetValid();
			}

			InvalidatedNodeTrackers.Finish();
		}

		// 0 -> 64; 4 -> 1024; 6 -> 4096
		int32 MaxBakedTextureSize = 1 << (Options.TextureResolution + 6);

		// When requested max texture size changed  invalidate all materials(material update emits textures to bake)
		if (MaxBakedTextureSize != FDatasmithExportOptions::MaxTextureSize)
		{
			for (TTuple<MtlBase*, FMaterialTrackerHandle> Material : MaterialsCollectionTracker.MaterialTrackers)
			{
				MaterialsCollectionTracker.InvalidateMaterial(Material.Value.GetMaterialTracker()->Material);
			}
			FDatasmithExportOptions::MaxTextureSize = MaxBakedTextureSize;
		}

		bChangeEncountered |= !MaterialsCollectionTracker.GetInvalidatedMaterials().IsEmpty();

		// Each tracked(i.e. assigned to a visible node) Max material can result in multiple actual materials
		// e.g. an assigned MultiSubObj material is aclually a set of material not directly assigned to a node
		TSet<Mtl*> ActualMaterialToUpdate; 
		{
			PROGRESS_STAGE("Process invalidated materials");
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsProcessInvalidatedMaterials();
			};

			PROGRESS_STAGE_COUNTER(MaterialsCollectionTracker.GetInvalidatedMaterials().Num());
			for (FMaterialTracker* MaterialTracker : MaterialsCollectionTracker.GetInvalidatedMaterials())
			{
				ProgressCounter.Next();
				if(GetCancel())
				{
					return false;	
				}

				SCENE_UPDATE_STAT_INC(ProcessInvalidatedMaterials, Invalidated);

				MaterialsCollectionTracker.UpdateMaterial(MaterialTracker);

				for (Mtl* ActualMaterial : MaterialTracker->GetActualMaterials())
				{
					SCENE_UPDATE_STAT_INC(ProcessInvalidatedMaterials, ActualToUpdate);
					ActualMaterialToUpdate.Add(ActualMaterial);
				}
				MaterialTracker->bInvalidated = false;
			}

			MaterialsCollectionTracker.ResetInvalidatedMaterials();
		}

		TSet<Texmap*> ActualTexmapsToUpdate;
		{
			PROGRESS_STAGE("Update materials")
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsUpdateMaterials();
			};
			PROGRESS_STAGE_COUNTER(ActualMaterialToUpdate.Num());

			MaterialsCollectionTracker.TexmapConverters.Reset();

			for (Mtl* ActualMaterial: ActualMaterialToUpdate)
			{
				ProgressCounter.Next();
				if(GetCancel())
				{
					break;	
				}

				MaterialsCollectionTracker.ConvertMaterial(ActualMaterial, ExportedScene.GetDatasmithScene().ToSharedRef(), ExportedScene.GetSceneExporter().GetAssetsOutputPath(), ActualTexmapsToUpdate);
			}

			if(GetCancel())
			{
				return false;	
			}

		}

		{
			PROGRESS_STAGE("Update textures")
			PROGRESS_STAGE_RESULT_DEFERRED
			{
				return FormatStatsUpdateTextures();
			};
			PROGRESS_STAGE_COUNTER(ActualTexmapsToUpdate.Num());

			for (Texmap* Texture : ActualTexmapsToUpdate)
			{
				ProgressCounter.Next();
				if(GetCancel())
				{
					return false;	
				}
				SCENE_UPDATE_STAT_INC(UpdateTextures, Total);

				MaterialsCollectionTracker.UpdateTexmap(Texture);
			}
		}

		// todo: removes textures that were added again(materials were updated). Need to fix this by identifying exactly which textures are being updated and removing ahead
		//TMap<FString, TSharedPtr<IDatasmithTextureElement>> TexturesAdded;
		//TArray<TSharedPtr<IDatasmithTextureElement>> TexturesToRemove;
		//for(int32 TextureIndex = 0; TextureIndex < ExportedScene.GetDatasmithScene()->GetTexturesCount(); ++TextureIndex )
		//{
		//	TSharedPtr<IDatasmithTextureElement> TextureElement = ExportedScene.GetDatasmithScene()->GetTexture(TextureIndex);
		//	FString Name = TextureElement->GetName();
		//	if (TexturesAdded.Contains(Name))
		//	{
		//		TexturesToRemove.Add(TexturesAdded[Name]);
		//		TexturesAdded[Name] = TextureElement;
		//	}
		//	else
		//	{
		//		TexturesAdded.Add(Name, TextureElement);
		//	}
		//}

		//for(TSharedPtr<IDatasmithTextureElement> Texture: TexturesToRemove)
		//{
		//	ExportedScene.GetDatasmithScene()->RemoveTexture(Texture);
		//}
		return bChangeEncountered;
	}

	FString FormatStatsParseScene()
	{
		return FString::Printf(TEXT("Nodes: parsed %d"), SCENE_UPDATE_STAT_GET(ParseNode, NodesEncountered));
	}

	FString FormatStatsRemoveDeletedNodes()
	{
		return FString::Printf(TEXT("Nodes: deleted %d"), SCENE_UPDATE_STAT_GET(RemoveDeletedNodes, Nodes));
	}

	FString FormatStatsUpdateNodeNames()
	{
		return FString::Printf(TEXT("Nodes: updated %d of total %d"), InvalidatedNodeTrackers.Num(), NodeTrackers.Num());
	}

	FString FormatStatsRefreshCollisions()
	{
		return FString::Printf(TEXT("Nodes: added %d to invalidated %d"), SCENE_UPDATE_STAT_GET(RefreshCollisions, ChangedNodes), InvalidatedNodeTrackers.Num());
	}

	FString FormatStatsCheckTimeSliderValidity()
	{
		return FString::Printf(TEXT("Check TimeSlider: checked %d, invalidated %d, skipped  - already invalidated %d, subtree valid %d"),
			SCENE_UPDATE_STAT_GET(CheckTimeSlider, TotalChecks),
			SCENE_UPDATE_STAT_GET(CheckTimeSlider, Invalidated),
			SCENE_UPDATE_STAT_GET(CheckTimeSlider, SkippedAsAlreadyInvalidated),
			SCENE_UPDATE_STAT_GET(CheckTimeSlider, SkippedAsSubtreeValid)
			);
	}

	FString FormatStatsProcessInvalidatedNodes()
	{
		return FString::Printf(TEXT("Nodes: %d updated, %d skipped unselected, %d skipped hidden"), SCENE_UPDATE_STAT_GET(UpdateNode, NodesUpdated), SCENE_UPDATE_STAT_GET(UpdateNode, SkippedAsUnselected), SCENE_UPDATE_STAT_GET(UpdateNode, SkippedAsHiddenNode));
	}

	FString FormatStatsConvertNodesToDatasmith()
	{
		return FString::Printf(TEXT("Nodes: %d converted"), SCENE_UPDATE_STAT_GET(ConvertNodes, Converted));
	}

	FString FormatStatsProcessInvalidatedInstances()
	{
		return FString::Printf(TEXT("Unique meshes: %ld updated"), SCENE_UPDATE_STAT_GET(UpdateInstances, GeometryUpdated));
	}

	FString FormatStatsReparentDatasmithActors()
	{
		return FString::Printf(TEXT("Nodes: %d attached, to root %d, skipped %d"), SCENE_UPDATE_STAT_GET(ReparentActors, Attached), SCENE_UPDATE_STAT_GET(ReparentActors, AttachedToRoot), SCENE_UPDATE_STAT_GET(ReparentActors, SkippedWithoutDatasmithActor));
	}

	FString FormatStatsProcessInvalidatedMaterials()
	{
		return FString::Printf(TEXT("Materials: %d reparsed, found %d actual to update"), SCENE_UPDATE_STAT_GET(ProcessInvalidatedMaterials, Invalidated), SCENE_UPDATE_STAT_GET(ProcessInvalidatedMaterials, ActualToUpdate));
	}

	FString FormatStatsUpdateMaterials()
	{
		return FString::Printf(TEXT("Materials: %d updated, %d converted, %d skipped as already converted"), SCENE_UPDATE_STAT_GET(UpdateMaterials, Total), SCENE_UPDATE_STAT_GET(UpdateMaterials, Converted), SCENE_UPDATE_STAT_GET(UpdateMaterials, SkippedAsAlreadyConverted));
	}

	FString FormatPixelCountApproximately(int64 Value)
	{
		if (Value < 2*1024 )
		{
			return FString::Printf(TEXT("%lld pixels"), Value);
		}

		Value /= 1024; // Kilo

		if (Value  < 2*1024 )
		{
			return FString::Printf(TEXT("%lldK pixels"), Value);
		}

		Value /= 1024; // Mega

		if (Value  < 2*1024 )
		{
			return FString::Printf(TEXT("%lld Megapixels"), Value);
		}

		Value /= 1024; // Giga

		return FString::Printf(TEXT("%lld Gigapixels"), Value);
	}

	FString FormatStatsUpdateTextures()
	{
		if (SCENE_UPDATE_STAT_GET(UpdateTextures, Baked) > 0)
		{
			return FString::Printf(TEXT("Texmaps: %d updated, baked %d, %s baked total(%s per textmap on average)"), 
				SCENE_UPDATE_STAT_GET(UpdateTextures, Total),
				SCENE_UPDATE_STAT_GET(UpdateTextures, Baked),
				*FormatPixelCountApproximately(SCENE_UPDATE_STAT_GET(UpdateTextures, BakedPixels)),
				*FormatPixelCountApproximately(SCENE_UPDATE_STAT_GET(UpdateTextures, BakedPixels) / SCENE_UPDATE_STAT_GET(UpdateTextures, Baked)
				));
		}
		else
		{
			return FString::Printf(TEXT("Texmaps: %d updated"), SCENE_UPDATE_STAT_GET(UpdateTextures, Total));
		}
	}

	bool ExportAnimations(FUpdateProgress::FStage& MainStage)
	{
		PROGRESS_STAGE("Export Animations");

		FDatasmithConverter Converter;
		// Use the same name for the unique level sequence as the scene name
		TSharedRef<IDatasmithLevelSequenceElement> LevelSequence = FDatasmithSceneFactory::CreateLevelSequence(GetDatasmithScene().GetName());
		LevelSequence->SetFrameRate(GetFrameRate());

		PROGRESS_STAGE_COUNTER(NodeTrackers.Num());
		for (TPair<FNodeKey, FNodeTrackerHandle> NodeKeyAndNodeTracker: NodeTrackers)
		{
			ProgressCounter.Next();
			if (GetCancel())
			{
				return false;
			}

			FNodeTracker* NodeTracker = NodeKeyAndNodeTracker.Value.GetNodeTracker();

			if (NodeTracker->HasConverted())
			{
				// Not all nodes are converted to Datasmith actors(e.g. hidden nodes are omitted from Datasmith scene)
				// Transform animation is exported relative to parent actor's so we need node with actual datasmith actor to compute relative transform.
				FNodeTracker* ParentNodeTracker = GetAncestorNodeTrackerWithDatasmithActor(*NodeTracker);
				INode* ParentNode = ParentNodeTracker ? ParentNodeTracker->Node : nullptr;

				if (NodeTracker->GetConverterType() == FNodeConverter::LightNode)
				{
					const TSharedPtr<IDatasmithLightActorElement> LightElement = StaticCastSharedPtr< IDatasmithLightActorElement >(NodeTracker->GetConverted().DatasmithActorElement);
					const FMaxLightCoordinateConversionParams LightParams(NodeTracker->Node,
						LightElement->IsA(EDatasmithElementType::AreaLight) ? StaticCastSharedPtr<IDatasmithAreaLightElement>(LightElement)->GetLightShape() : EDatasmithLightShape::None);
					FDatasmithMaxSceneExporter::ExportAnimation(LevelSequence, ParentNode, NodeTracker->Node, NodeTracker->GetConverted().DatasmithActorElement->GetName(), Converter.UnitToCentimeter, LightParams);
				}
				else
				{
					FDatasmithMaxSceneExporter::ExportAnimation(LevelSequence, ParentNode, NodeTracker->Node, NodeTracker->GetConverted().DatasmithActorElement->GetName(), Converter.UnitToCentimeter);
				}
			}
		}
		if (LevelSequence->GetAnimationsCount() > 0)
		{
			GetDatasmithScene().AddLevelSequence(LevelSequence);
		}
		return true;
	}

	virtual void RemoveMaterial(const TSharedPtr<IDatasmithBaseMaterialElement>& DatasmithMaterial) override
	{
		ExportedScene.DatasmithSceneRef->RemoveMaterial(DatasmithMaterial);
	}

	virtual void RemoveTexture(const TSharedPtr<IDatasmithTextureElement>& DatasmithTextureElement) override
	{
		ExportedScene.DatasmithSceneRef->RemoveTexture(DatasmithTextureElement);		
	}

	FNodeTracker* GetNodeTracker(FNodeKey NodeKey)
	{
		if (FNodeTrackerHandle* NodeTrackerHandle = NodeTrackers.Find(NodeKey))
		{
			return NodeTrackerHandle->GetNodeTracker();
		}
		return nullptr;
	}

	FNodeTracker* GetNodeTracker(INode* ParentNode)
	{
		return GetNodeTracker(NodeEventNamespace::GetKeyByNode(ParentNode));
	}

	// Promote validity up the ancestor chain
	// Each Subtree Validity interval represents intersection of all invervals of descendants
	// Used to quickly determine if some whole subtree needs to be updated when Time Slider changes
	void PromoteValidity(FNodeTracker& NodeTracker, const FValidity& Validity)
	{
		if (Validity.Overlaps(NodeTracker.SubtreeValidity))
		{
			// Subtree validity is already fully within new validity - don't need to compute intersection and promote it further
			return;
		}

		NodeTracker.SubtreeValidity.NarrowValidityToInterval(Validity);

		// Promote recalculated SubtreeValidity to parent
		if (FNodeTracker* ParentNodeTracker = GetNodeTracker(NodeTracker.Node->GetParentNode()))
		{
			PromoteValidity(*ParentNodeTracker, NodeTracker.SubtreeValidity);
		}
	}

	// Node invalidated:
	//   - on initial parsing
	//   - change event received for node itself
	//   - node up hierarchy invalidated - previous or current
	//   - node's validity interval is invalid for current time

	void InvalidateNode(FNodeTracker& NodeTracker, bool bCheckCalledInProgress = true)
	{
		if (bCheckCalledInProgress)
		{
			// We don't expect node changes while Update inprogress(unless Invalidate called explicitly in certain places)
			if (!ensure(!bUpdateInProgress))
			{
				return;
			}
		}

		if (NodeTracker.bDeleted)
		{
			// Change events sometimes received for nodes that are already deleted
			// skipping processing of node subtree because INode pointer may already be invalid
			// Test case: create container, add node to it. Close it, open it, close again, then sync
			// Change event will be received for NodeKey that returns NULL from NodeEventNamespace::GetNodeByKey
			if (bIncludeXRefWhileParsing)
			{
				check(NodeEventNamespace::GetNodeByKey(NodeTracker.NodeKey));
				NodeTracker.bDeleted = false;
			}
			else
			{
				return;
			}
		}

		if (NodeTracker.Validity.IsInvalidated())
		{
			// Don't do work twice - already invalidated node would have its subhierarchy invalidated
			//   in case subhierarchy was changed for already invalidated node - the changed nodes would invalidate too(responding to reparent event)
			return;
		}

		NodeTracker.Validity.Invalidate();
		InvalidatedNodeTrackers.Add(NodeTracker);
		PromoteValidity(NodeTracker, NodeTracker.Validity);
	}

	// todo: make fine invalidates - full only something like geometry change, but finer for transform, name change and more
	FNodeTracker* InvalidateNode(FNodeKey NodeKey, bool bCheckCalledInProgress = true)
	{
		INode* Node = NodeEventNamespace::GetNodeByKey(NodeKey);

		LogDebugNode(TEXT("InvalidateNode"), Node);
		// We don't expect node chances while Update inprogress(unless Invalidate called explicitly)
		if (bCheckCalledInProgress)
		{
			ensure(!bUpdateInProgress);
		}

		if (FNodeTracker* NodeTracker =  GetNodeTracker(NodeKey))
		{
			if (Node)
			{
				InvalidateNode(*NodeTracker, bCheckCalledInProgress);
				return NodeTracker;
			}
			else
			{
				// Sometimes node update received without node Delete event
				// Test case: create container, add node to it. Close it, open it, close again, then sync
				InvalidatedNodeTrackers.Add(*NodeTracker);
				NodeTracker->bDeleted = true;
				PromoteValidity(*NodeTracker, NodeTracker->Validity);
			}
		}
		else
		{
			if (Node)
			{
				return AddNode(NodeKey, Node);
			}
			// See comment above. When Containers are used and no XRef Scenes export is enabled event may be received for node which doesn't exist
		}
		return nullptr;
	}

	FNodeTracker* AddNode(FNodeKey NodeKey, INode* Node)
	{
		LogDebugNode(TEXT("AddNode"), Node);
		check(!NodeTrackers.Contains(NodeKey));

		FNodeTracker* NodeTracker = NodeTrackers.Emplace(NodeKey, FNodeTrackerHandle(NodeKey, Node)).GetNodeTracker();

		NodeTrackersNames.Add(*NodeTracker);
		InvalidatedNodeTrackers.Add(*NodeTracker);
		PromoteValidity(*NodeTracker, NodeTracker->Validity);

		if (NotificationsHandler)
		{
			NotificationsHandler->AddNode(Node);
		}

		return NodeTracker;
	}

	FNodeTracker* AddNode(INode* Node)
	{
		return AddNode(NodeEventNamespace::GetKeyByNode(Node), Node);
	}

	void InvalidateOutdatedNodeTracker(FNodeTracker& NodeTracker)
	{
		SCENE_UPDATE_STAT_INC(CheckTimeSlider, TotalChecks);
		// Skip node that was already invalidated(and so its whole subtree)
		if (NodeTracker.Validity.IsInvalidated())
		{
			SCENE_UPDATE_STAT_INC(CheckTimeSlider, SkippedAsAlreadyInvalidated);
			return;
		}

		// Skip check in case whole subtree valid for current time
		if (NodeTracker.SubtreeValidity.IsValidForSyncPoint(CurrentSyncPoint))
		{
			SCENE_UPDATE_STAT_INC(CheckTimeSlider, SkippedAsSubtreeValid);
			return;
		}

		// If node is invalid - reevaliate whole subtree
		// todo: it it possible to optimize reevaluation of whole subtree?
		//   certain types of invalidation need not to propagate down to descendants(e.g. geometry change)
		//   but some need to, like transform change
		if (!NodeTracker.Validity.IsValidForSyncPoint(CurrentSyncPoint))
		{
			SCENE_UPDATE_STAT_INC(CheckTimeSlider, Invalidated);
			InvalidateNode(NodeTracker, false);
		}
		else
		{
			FIncludeXRefGuard IncludeXRefGuard(bIncludeXRefWhileParsing);
			int32 ChildNum = NodeTracker.Node->NumberOfChildren();
			for (int32 ChildIndex = 0; ChildIndex < ChildNum; ++ChildIndex)
			{
				if (FNodeTracker* ChildNodeTracker = GetNodeTracker(NodeTracker.Node->GetChildNode(ChildIndex)))
				{
					InvalidateOutdatedNodeTracker(*ChildNodeTracker);
				}
			}
		}
	}

	bool UpdateSubtreeValidity(FNodeTracker& NodeTracker)
	{
		// FIncludeXRefGuard IncludeXRefGuard(bIncludeXRefWhileParsing);

		// Skip check in case whole subtree was valid for current time
		if (NodeTracker.SubtreeValidity.IsValidForSyncPoint(CurrentSyncPoint))
		{
			return true;
		}

		NodeTracker.SubtreeValidity.Invalidate();
		NodeTracker.SubtreeValidity.ResetValidityInterval();
		NodeTracker.SubtreeValidity.NarrowValidityToInterval(NodeTracker.Validity);

		int32 ChildNum = NodeTracker.Node->NumberOfChildren();
		for (int32 ChildIndex = 0; ChildIndex < ChildNum; ++ChildIndex)
		{
			if (FNodeTracker* ChildNodeTracker = GetNodeTracker(NodeTracker.Node->GetChildNode(ChildIndex)))
			{
				if (UpdateSubtreeValidity(*ChildNodeTracker))
				{
					NodeTracker.SubtreeValidity.NarrowValidityToInterval(ChildNodeTracker->SubtreeValidity);
				}
				else
				{
					NodeTracker.SubtreeValidity.Invalidate();
					return false;
				}
			}
		}
		return true;
	}
	
	void ClearNodeFromDatasmithScene(FNodeTracker& NodeTracker)
	{
		ReleaseNodeTrackerFromDatasmithMetadata(NodeTracker);

		// remove from hierarchy
		if (NodeTracker.HasConverted())
		{
			FNodeConverted& Converted = NodeTracker.GetConverted();
			TSharedPtr<IDatasmithMeshActorElement> DatasmithMeshActor = Converted.DatasmithMeshActor;

			// Remove mesh actor before removing its parent Actor in case there a separate MeshActor
			if (Converted.DatasmithMeshActor)
			{
				if (Converted.DatasmithActorElement != Converted.DatasmithMeshActor)
				{
					Converted.DatasmithActorElement->RemoveChild(Converted.DatasmithMeshActor);
				}

				MaterialsCollectionTracker.UnSetMaterialsForMeshActor(Converted.DatasmithMeshActor);


				Converted.DatasmithMeshActor.Reset();
				// todo: consider pool of MeshActors
			}

			if (TSharedPtr<IDatasmithActorElement> ParentActor = Converted.DatasmithActorElement->GetParentActor())
			{
				ParentActor->RemoveChild(Converted.DatasmithActorElement);
			}
			else
			{
				// Detach all children(so they won't be reattached automatically to root when actor is detached from parent below)
				// Children reattachment will happen later in Update
				int32 ChildCount = Converted.DatasmithActorElement->GetChildrenCount();
				// Remove last child each time to optimize array elements relocation
				for(int32 ChildIndex = ChildCount-1; ChildIndex >= 0; --ChildIndex)
				{
					Converted.DatasmithActorElement->RemoveChild(Converted.DatasmithActorElement->GetChild(ChildIndex));
				}
				ExportedScene.DatasmithSceneRef->RemoveActor(Converted.DatasmithActorElement, EDatasmithActorRemovalRule::RemoveChildren);
			}
			Converted.DatasmithActorElement.Reset();

			NodeTracker.ReleaseConverted();
		}
		
	}

	// Called when mesh element is not needed anymore and should be removed from the scene
	virtual void ReleaseMeshElement(FMeshConverted& Converted) override
	{
		if (Meshes.Release(Converted))
		{
			MaterialsCollectionTracker.UnSetMaterialsForMeshElement(Converted.DatasmithMeshElement);
			GetDatasmithScene().RemoveMesh(Converted.DatasmithMeshElement);

			Meshes.Remove(Converted);
		}
	}

	void ReleaseNodeTrackerFromLayer(FNodeTracker& NodeTracker)
	{
		if (NodeTracker.Layer)
		{
			if (TSet<FNodeTracker*>* NodeTrackerPtr = NodesPerLayer.Find(NodeTracker.Layer))
			{
				NodeTrackerPtr->Remove(&NodeTracker);
			}
			NodeTracker.Layer = nullptr;
		}
	}

	void ReleaseNodeTrackerFromDatasmithMetadata(FNodeTracker& NodeTracker)
	{
		TSharedPtr<IDatasmithMetaDataElement> DatasmithMetadata;
		if (NodeDatasmithMetadata.RemoveAndCopyValue(&NodeTracker, DatasmithMetadata))
		{
			GetDatasmithScene().RemoveMetaData(DatasmithMetadata);
		}
	}

	// Release node from any connection to other tracked objects
	// When node is removed or to clear converter that is configured for invalidated node in Update
	// BUT excluding node name and collision status - these two are updated before UpdateNode is called
	void RemoveFromTracked(FNodeTracker& NodeTracker)
	{
		// todo: record previous converter Node type to speed up cleanup. Or just add 'unconverted' flag to speed up this for nodes that weren't converted yet

		ReleaseNodeTrackerFromLayer(NodeTracker);

		if (NodeTracker.HasConverter())
		{
			NodeTracker.GetConverter().RemoveFromTracked(*this, NodeTracker);
			NodeTracker.ReleaseConverter();
		}
	}

	void UpdateCollisionStatus(FNodeTracker& NodeTracker, TSet<FNodeTracker*>& NodesWithChangedCollisionStatus)
	{
		// Check if collision assigned for node changed
		{
			TOptional<FDatasmithMaxStaticMeshAttributes> DatasmithAttributes = FDatasmithMaxStaticMeshAttributes::ExtractStaticMeshAttributes(NodeTracker.Node);

			bool bOutFromDatasmithAttributes;
			INode* CollisionNode = GeomUtils::GetCollisionNode(*this, NodeTracker.Node, DatasmithAttributes ? &DatasmithAttributes.GetValue() : nullptr, bOutFromDatasmithAttributes);

			FNodeTracker* CollisionNodeTracker = GetNodeTracker(CollisionNode); // This node should be tracked

			if (NodeTracker.Collision != CollisionNodeTracker)
			{
				// Update usage counters for collision nodes

				// Remove previous
				if (TSet<FNodeTracker*>* CollisionUsersPtr = CollisionNodes.Find(NodeTracker.Collision))
				{
					TSet<FNodeTracker*>& CollisionUsers = *CollisionUsersPtr;
					CollisionUsers.Remove(&NodeTracker);

					if (CollisionUsers.IsEmpty())
					{
						CollisionNodes.Remove(NodeTracker.Collision);
						NodesWithChangedCollisionStatus.Add(NodeTracker.Collision);
					}
				}

				// Add new
				if (CollisionNodeTracker)
				{
					if (TSet<FNodeTracker*>* CollisionUsersPtr = CollisionNodes.Find(CollisionNodeTracker))
					{
						TSet<FNodeTracker*>& CollisionUsers = *CollisionUsersPtr;
						CollisionUsers.Add(&NodeTracker);
					}
					else
					{
						TSet<FNodeTracker*>& CollisionUsers = CollisionNodes.Add(CollisionNodeTracker);
						CollisionUsers.Add(&NodeTracker);
						NodesWithChangedCollisionStatus.Add(CollisionNodeTracker);
					}
				}
				NodeTracker.Collision = CollisionNodeTracker;
			}
		}

		// Check if node changed its being assigned as collision
		{
			if (FDatasmithMaxSceneHelper::HasCollisionName(NodeTracker.Node))
			{
				CollisionNodes.Add(&NodeTracker); // Always view node with 'collision' name as a collision node(i.e. no render)

				//Check named collision assignment(e.g. 'UCP_<other nothe name>')
				// Split collision prefix and find node that might use this node as collision mesh
				FString NodeName = NodeTrackersNames.GetNodeName(NodeTracker);
				FString LeftString, RightString;
				NodeName.Split(TEXT("_"), &LeftString, &RightString);

				NodeTrackersNames.EnumerateForName(RightString, [&](FNodeTracker& CollisionUserNodeTracker)
				{
					if (CollisionUserNodeTracker.Collision != &NodeTracker)
					{
						NodesWithChangedCollisionStatus.Add(&CollisionUserNodeTracker); // Invalidate each node that has collision changed
					}
				});
			}
			else
			{
				// Remove from registered collision nodes if there's not other users(i.e. using Datasmith attributes reference)
				if (TSet<FNodeTracker*>* CollisionUsersPtr = CollisionNodes.Find(&NodeTracker))
				{
					if (CollisionUsersPtr->IsEmpty())
					{
						CollisionNodes.Remove(&NodeTracker);
					}
				}
			}
		}
	}

	void RemoveNodeTracker(FNodeTracker& NodeTracker)
	{
		SCENE_UPDATE_STAT_INC(RemoveDeletedNodes, Nodes);

		InvalidatedNodeTrackers.RemoveFromInvalidated(NodeTracker);

		ClearNodeFromDatasmithScene(NodeTracker);
		RemoveFromTracked(NodeTracker);

		NodeTrackersNames.Remove(NodeTracker);

		if (TSet<FNodeTracker*>* CollisionUsersPtr = CollisionNodes.Find(NodeTracker.Collision))
		{
			TSet<FNodeTracker*>& CollisionUsers = *CollisionUsersPtr;
			CollisionUsers.Remove(&NodeTracker);

			if (CollisionUsers.IsEmpty())
			{
				CollisionNodes.Remove(NodeTracker.Collision);
			}
		}

		NodeTrackers.Remove(NodeTracker.NodeKey);
	}

	void UpdateNode(FNodeTracker& NodeTracker)
	{
		SCENE_UPDATE_STAT_INC(UpdateNode, NodesUpdated);
		// Forget anything that this node was before update: place in datasmith hierarchy, datasmith objects, instances connection. Updating may change anything
		ClearNodeFromDatasmithScene(NodeTracker);
		RemoveFromTracked(NodeTracker); // todo: might keep it tracked by complicating conversions later(e.g. to avoid extra work if object stays the same)

		NodeTracker.Validity.ResetValidityInterval(); // Make infinite validity to narrow it during update

		ConvertNodeObject(NodeTracker);
	}

	void ConvertNodeObject(FNodeTracker& NodeTracker)
	{
		// Update layer connection
		ILayer* Layer = (ILayer*)NodeTracker.Node->GetReference(NODE_LAYER_REF);
		if (Layer)
		{
			if (TUniquePtr<FLayerTracker>* LayerPtr = LayersForAnimHandle.Find(Animatable::GetHandleByAnim(Layer)))
			{
				FLayerTracker* LayerTracker =  LayerPtr->Get();
				NodeTracker.Layer = LayerTracker;
				NodesPerLayer.FindOrAdd(LayerTracker).Add(&NodeTracker);
			}
		}

		if (CollisionNodes.Contains(&NodeTracker))
		{
			SCENE_UPDATE_STAT_INC(UpdateNode, SkippedAsCollisionNode);
			return;
		}

		if (NodeTracker.Node->IsNodeHidden(TRUE) || !NodeTracker.Node->Renderable())
		{
			SCENE_UPDATE_STAT_INC(UpdateNode, SkippedAsHiddenNode);
			return;
		}

		if (Options.bSelectedOnly && !NodeTracker.Node->Selected())
		{
			SCENE_UPDATE_STAT_INC(UpdateNode, SkippedAsUnselected);
			return;
		}

		ObjectState ObjState = NodeTracker.Node->EvalWorldState(CurrentSyncPoint.Time);
		Object* Obj = ObjState.obj;
		if (!Obj)
		{
			return;
		}

		SClass_ID SuperClassID = Obj->SuperClassID();
		switch (SuperClassID)
		{
		case HELPER_CLASS_ID:
			SCENE_UPDATE_STAT_INC(UpdateNode, HelpersEncontered);
			NodeTracker.CreateConverter<FHelperNodeConverter>();
			break;
		case CAMERA_CLASS_ID:
			SCENE_UPDATE_STAT_INC(UpdateNode, CamerasEncontered);
			NodeTracker.CreateConverter<FCameraNodeConverter>();
			break;
		case LIGHT_CLASS_ID:
		{
			SCENE_UPDATE_STAT_INC(UpdateNode, LightsEncontered);

			if (EMaxLightClass::Unknown == FDatasmithMaxSceneHelper::GetLightClass(NodeTracker.Node))
			{
				SCENE_UPDATE_STAT_INC(UpdateNode, LightsSkippedAsUnknown);
				break;
			}

			NodeTracker.CreateConverter<FLightNodeConverter>();
			break;
		}
		case SHAPE_CLASS_ID:
		case GEOMOBJECT_CLASS_ID:
		{
			SCENE_UPDATE_STAT_INC(UpdateNode, GeomObjEncontered);
			Class_ID ClassID = ObjState.obj->ClassID();
			if (ClassID.PartA() == TARGET_CLASS_ID) // Convert camera target as regular actor
			{
				NodeTracker.CreateConverter<FHelperNodeConverter>();
			}
			else if (ClassID == RAILCLONE_CLASS_ID)
			{
				NodeTracker.CreateConverter<FRailCloneNodeConverter>();
				break;
			}
			else if (ClassID == ITOOFOREST_CLASS_ID)
			{
				NodeTracker.CreateConverter<FForestNodeConverter>();
				break;
			}
			else
			{
				if (FDatasmithMaxSceneHelper::HasCollisionName(NodeTracker.Node))
				{
					ConvertNamedCollisionNode(NodeTracker);
				}
				else
				{
					if (Obj->IsRenderable()) // Shape's Enable In Render flag(note - different from Node's Renderable flag)
					{
						SCENE_UPDATE_STAT_INC(UpdateNode, GeomObjConverted);

						NodeTracker.CreateConverter<FMeshNodeConverter>();
					}
					else
					{
						SCENE_UPDATE_STAT_INC(UpdateNode, GeomObjSkippedAsNonRenderable);
					}
				}
			}
			break;
		}
		case SYSTEM_CLASS_ID:
		{
			//When a referenced file is not found XRefObj is not resolved then it's kept as XREFOBJ_CLASS_ID instead of resolved class that it references
			if (Obj->ClassID() == XREFOBJ_CLASS_ID)
			{
				FString Path = FDatasmithMaxSceneExporter::GetActualPath(static_cast<IXRefObject8*>(Obj)->GetFile(FALSE).GetFileName());
				if (!FPaths::FileExists(Path))
				{
					LogWarning(FString("XRefObj file \"") + FPaths::GetCleanFilename(*Path) + FString("\" cannot be found"));
				}
			}

			break;
		}
		default:;
		}

		if (NodeTracker.HasConverter())
		{
			NodeTracker.GetConverter().Parse(*this, NodeTracker);
		}
	}

	void InvalidateInstances(FInstances& Instances)
	{
		InvalidatedInstances.Add(&Instances);
	}

	void UpdateInstances(FInstances& Instances)
	{
		if (Instances.NodeTrackers.IsEmpty())
		{
			// Invalidated instances without actual instances left(all removed)

			// NOTE: release mesh element and release usage of Instances pointer
			//   BEFORE RemoveInstances call(which deallocates Instances object)
			ReleaseMeshElement(Instances.Converted);

			InstancesManager.RemoveInstances(Instances);
			return;
		}

		bool bInitial = true;
		bool bShouldBakePivot = true;
		FTransform BakePivot = FTransform::Identity;

		for (FNodeTracker* NodeTrackerPtr : Instances.NodeTrackers)
		{
			FNodeTracker& NodeTracker = *NodeTrackerPtr;
			ClearNodeFromDatasmithScene(NodeTracker);
			if (ensure(NodeTracker.GetConverter().ConverterType == FNodeConverter::MeshNode))
			{
				FMeshNodeConverter& Converter = static_cast<FMeshNodeConverter&>(NodeTracker.GetConverter());
				Converter.bMaterialsAssignedToStaticMesh = false;

				// Pivot considerations
				// - Single instance
				//     a. identity pivot => bShouldBakePivot = false
				//     b. non-identity pivot => bShouldBakePivot = true (and set BakePivot)
				// - Multiple nodes
				//     a. Identity ALL pivots => bShouldBakePivot = false
				//     b. EQUAL non-Identity ALL pivots => bShouldBakePivot = true
				//     c. NOT equal pivots => bShouldBakePivot = false, add warning
				FTransform Pivot;
				GetPivotTransform(NodeTracker, Pivot);

				if (bInitial)
				{
					if (Pivot.Equals(BakePivot))
					{
						bShouldBakePivot = false;
					}
					else
					{
						BakePivot = Pivot;
						bShouldBakePivot = true;
					}

					bInitial = false;
				}
				else
				{
					if (!Pivot.Equals(BakePivot))
					{
						bShouldBakePivot = false;

						LogWarning(FString::Printf(TEXT("Multiple different pivots on instances of object %s."), NodeTracker.Node->GetName()));
					}
				}
			}
		}

		Instances.bShouldBakePivot = bShouldBakePivot;
		Instances.BakePivot = BakePivot;

		// Select node to export geometry from
		// Use Node with smallest value of Handle to make change of StaticMesh Name less likely
		// Re-exporting scene won't change StaticMesh even when nodes are reordered in hierarchy.
		FNodeTracker* SelectedNodeTracker = nullptr;
		ULONG SelectedHandle = 0;

		for (FNodeTracker* NodeTracker : Instances.NodeTrackers)
		{
			ULONG Handle = NodeTracker->Node->GetHandle();
			if (!SelectedNodeTracker || (Handle < SelectedHandle))
			{
				SelectedHandle = Handle;
				SelectedNodeTracker = NodeTracker;
			}
		}


		// Update geometry using selected Node
		if (SelectedNodeTracker)
		{
			FNodeTracker& NodeTracker = *SelectedNodeTracker;

			// todo: use single EnumProc instance to enumerate all nodes during update to:
			//    - have single call to BeginEnumeration and EndEnumeration
			//    - track all Begin'd nodes to End them together after all is updated(to prevent duplicated Begin's of referenced objects that might be shared by different nodes)
			NodesPreparer.PrepareNode(NodeTracker.Node);
			SCENE_UPDATE_STAT_INC(UpdateInstances, GeometryUpdated);
			UpdateInstancesGeometry(Instances, NodeTracker);


			// assign materials to static mesh for the first instance(others will use override on mesh actors)
			static_cast<FMeshNodeConverter&>(NodeTracker.GetConverter()).bMaterialsAssignedToStaticMesh = true;
			if (Mtl* Material = UpdateGeometryNodeMaterial(*this, Instances, NodeTracker))
			{
				MaterialsCollectionTracker.SetMaterialsForMeshElement(Instances.Converted, Material);
				Instances.AssignMaterialToStaticMesh(Material);
			}
		}

		for (FNodeTracker* NodeTrackerPtr : Instances.NodeTrackers)
		{
			NodeTrackerPtr->Validity.NarrowValidityToInterval(Instances.ValidityInterval);
		}
	}

	void UpdateNodeMetadata(FNodeTracker& NodeTracker)
	{
		TSharedPtr<IDatasmithMetaDataElement> MetadataElement = FDatasmithMaxSceneExporter::ParseUserProperties(NodeTracker.Node, NodeTracker.GetConverted().DatasmithActorElement.ToSharedRef(), ExportedScene.GetDatasmithScene().ToSharedRef());
		NodeDatasmithMetadata.Add(&NodeTracker, MetadataElement);
	}

	// Get parent node, transparently resolving XRefScene binding
	FNodeTracker* GetParentNodeTracker(FNodeTracker& NodeTracker)
	{
		INode* XRefParent = NodeTracker.GetXRefParent(); // Node may be at the root of an XRefScene, in this case get the scene node that this XRef if bound to(by XRef UI 'bind' or being a Container node)
		return GetNodeTracker(XRefParent ? XRefParent : NodeTracker.Node->GetParentNode());
	}

	// Not all nodes result in creation of DatasmithActor for them(e.g. skipped as invisible), find first ascestor that has it
	FNodeTracker* GetAncestorNodeTrackerWithDatasmithActor(FNodeTracker& InNodeTracker)
	{
		FNodeTracker* NodeTracker = &InNodeTracker;
		while (FNodeTracker* ParentNodeTracker = GetParentNodeTracker(*NodeTracker))
		{
			if (ParentNodeTracker->HasConverted())
			{
				return ParentNodeTracker;
			}
			NodeTracker = ParentNodeTracker;
		}
		return nullptr;
	}

	bool AttachNodeToDatasmithScene(FNodeTracker& NodeTracker)
	{
		if (!NodeTracker.HasConverted())
		{
			SCENE_UPDATE_STAT_INC(ReparentActors, SkippedWithoutDatasmithActor);

			return false;
		}
		SCENE_UPDATE_STAT_INC(ReparentActors, Attached);

		if (FNodeTracker* ParentNodeTracker = GetAncestorNodeTrackerWithDatasmithActor(NodeTracker))
		{
			ParentNodeTracker->GetConverted().DatasmithActorElement->AddChild(NodeTracker.GetConverted().DatasmithActorElement, EDatasmithActorAttachmentRule::KeepWorldTransform);
		}
		else
		{
			SCENE_UPDATE_STAT_INC(ReparentActors, AttachedToRoot);
			// If there's no ancestor node with DatasmithActor assume node it at root
			// (node's parent might be node that was skipped - e.g. it was hidden in Max or not selected when exporting only selected objects)
			GetDatasmithScene().AddActor(NodeTracker.GetConverted().DatasmithActorElement);
		}
		return true;
	}

	void GetNodeObjectTransform(FNodeTracker& NodeTracker, FTransform& ObjectTransform)
	{
		FDatasmithConverter Converter;

		FVector Translation, Scale;
		FQuat Rotation;

		const FMaxLightCoordinateConversionParams LightParams = FMaxLightCoordinateConversionParams(NodeTracker.Node);
		// todo: do we really need to call GetObjectTM if there's no WSM attached? Maybe just call GetObjTMAfterWSM always?
		Interval ValidityInterval;
		ValidityInterval.SetInfinite();
		if (NodeTracker.Node->GetWSMDerivedObject() != nullptr)
		{
			FDatasmithMaxSceneExporter::MaxToUnrealCoordinates(NodeTracker.Node->GetObjTMAfterWSM(CurrentSyncPoint.Time, &ValidityInterval), Translation, Rotation, Scale, Converter.UnitToCentimeter, LightParams);
		}
		else
		{
			FDatasmithMaxSceneExporter::MaxToUnrealCoordinates(NodeTracker.Node->GetObjectTM(CurrentSyncPoint.Time, &ValidityInterval), Translation, Rotation, Scale, Converter.UnitToCentimeter, LightParams);
		}
		LogDebug(FString::Printf(TEXT("Validity: (%ld, %ld)"), ValidityInterval.Start(), ValidityInterval.End()));
		Rotation.Normalize();
		ObjectTransform = FTransform(Rotation, Translation, Scale);

		NodeTracker.Validity.NarrowValidityToInterval(ValidityInterval);
	}

	FMaterialTracker* RegisterNodeForMaterial(FNodeTracker& NodeTracker, Mtl* Material)
	{
		if (!Material)
		{
			return nullptr;
		}

		FMaterialTracker* MaterialTracker = MaterialsCollectionTracker.AddMaterial(Material);
		NodeTracker.MaterialTrackers.Add(MaterialTracker);
		MaterialsAssignedToNodes.FindOrAdd(MaterialTracker).Add(&NodeTracker);

		if (NotificationsHandler)
		{
			NotificationsHandler->AddMaterial(Material);
		}
		return MaterialTracker;
	}

	virtual void UnregisterNodeForMaterial(FNodeTracker& NodeTracker) override 
	{
		for (FMaterialTracker* MaterialTracker: NodeTracker.MaterialTrackers)
		{
			MaterialsAssignedToNodes[MaterialTracker].Remove(&NodeTracker);
			if (MaterialsAssignedToNodes[MaterialTracker].IsEmpty())
			{
				MaterialsCollectionTracker.ReleaseMaterial(*MaterialTracker);
				MaterialsAssignedToNodes.Remove(MaterialTracker);
			}
		}
		NodeTracker.MaterialTrackers.Reset();
	}

	virtual const TCHAR* AcquireIesTexture(const FString& IesFilePath) override
	{
		return IesTextures.AcquireIesTexture(IesFilePath);
	}

	virtual void ReleaseIesTexture(const FString& IesFilePath) override
	{
		return IesTextures.ReleaseIesTexture(IesFilePath);
	}

	static Mtl* UpdateGeometryNodeMaterial(FSceneTracker& SceneTracker, FInstances& Instances, FNodeTracker& NodeTracker)
	{
		if (Instances.HasMesh())
		{
			if (Mtl* Material = NodeTracker.Node->GetMtl())
			{
				bool bMaterialRegistered = false;
				for(FMaterialTracker* MaterialTracker: NodeTracker.MaterialTrackers)
				{
					bMaterialRegistered = bMaterialRegistered || (MaterialTracker->Material == Material);
				}

				if (!bMaterialRegistered) // Different material
				{
					// Release old material
					SceneTracker.UnregisterNodeForMaterial(NodeTracker);
					// Record new connection
					SceneTracker.RegisterNodeForMaterial(NodeTracker, Material);
				}
				return Material;
			}

			// Release old material when node has no material now
			SceneTracker.UnregisterNodeForMaterial(NodeTracker);
		}
		return nullptr;
	}

	// Backtrack material to the converted mesh it's used on to fix uv channel mapping(Max channel id -> Datasmith uv channel index)
	// Max channel ids are a sparse set on a Max mesh and its id number is just an integer key. But Unreal(and Datasmith) uses a continuous array of uv sets and therefore Unreal uv ids are index into this array, ranging from 0 to NumUVChannels-1 (including lightmap uvs)
	// 
	// todo: simple remapping works only when all meshes using the material have their uv channels built in the same way(same set and order)
	//   in general a situation when one material is used on two different meshes with different channel order could be handled(like duplicating material to use different channel mappings)
	virtual void RemapConvertedMaterialUVChannels(Mtl* ActualMaterial, const TSharedPtr<IDatasmithBaseMaterialElement>& DatasmithMaterial) override
	{
		if (!ensure(DatasmithMaterial->IsA(EDatasmithElementType::UEPbrMaterial)))
		{
			return;
		}

		TSet<FMaterialTracker*>* MaterialTrackers = MaterialsCollectionTracker.ActualMaterialToMaterialTracker.Find(ActualMaterial);

		if (!MaterialTrackers)
		{
			return;
		}

		for (FMaterialTracker* MaterialTracker: *MaterialTrackers)
		{
			if (TSet<FNodeTracker*>* NodeTrackersPtr = MaterialsAssignedToNodes.Find(MaterialTracker))
			{
				for (FNodeTracker* NodeTracker: *NodeTrackersPtr)
				{
					if (!NodeTracker->HasConverter())
					{
						continue;
					}
							
					if (NodeTracker->GetConverter().ConverterType == FNodeConverter::MeshNode)
					{
						if (FInstances* Instances = InstancesManager.GetInstancesForNodeTracker(*NodeTracker))
						{
							RemapUVChannels(StaticCastSharedPtr<IDatasmithUEPbrMaterialElement>(DatasmithMaterial), Instances->Converted.UVChannelsMap);
							return;
						}
					}
					else if (NodeTracker->GetConverter().ConverterType == FNodeConverter::HismNode)
					{
						for (FMeshConverted& MeshConverted: static_cast<FHismNodeConverter&>(NodeTracker->GetConverter()).Meshes)
						{
							RemapUVChannels(StaticCastSharedPtr<IDatasmithUEPbrMaterialElement>(DatasmithMaterial), MeshConverted.UVChannelsMap);
							return;
						}
					}
				}
			}
		}
	}

	virtual void AddGeometryNodeInstance(FNodeTracker& NodeTracker, FMeshNodeConverter& MeshConverter, Object* Obj) override
	{
		InvalidateInstances(InstancesManager.AddNodeTracker(NodeTracker, MeshConverter, Obj));
	}

	virtual void RemoveGeometryNodeInstance(FNodeTracker& NodeTracker) override
	{
		if (FInstances* Instances = InstancesManager.RemoveNodeTracker(NodeTracker))
		{
			// Invalidate instances that had a node removed
			//  - need to rebuild for various reasons(mesh might have been built from removed node, material assignment needds rebuild), remove empty
			InvalidateInstances(*Instances);
		}
	}

	void GetPivotTransform(FNodeTracker& NodeTracker, FTransform& Pivot)
	{
		FDatasmithConverter Converter;
		Pivot = FDatasmithMaxSceneExporter::GetPivotTransform(NodeTracker.Node, Converter.UnitToCentimeter);
	}

	virtual void ConvertGeometryNodeToDatasmith(FNodeTracker& NodeTracker, FMeshNodeConverter& MeshConverter) override
	{
		FInstances* InstancesPtr = InstancesManager.GetInstancesForNodeTracker(NodeTracker);
		if (!InstancesPtr)
		{
			return;
		}
		FInstances& Instances = *InstancesPtr;

		FTransform ObjectTransform;
		GetNodeObjectTransform(NodeTracker, ObjectTransform);

		FTransform Pivot;
		GetPivotTransform(NodeTracker, Pivot);

		bool bPivotBakedInMesh = Instances.bShouldBakePivot;
		// Create separate actor only when there are multiple instances
		bool bNeedPivotComponent = !bPivotBakedInMesh && !Pivot.Equals(FTransform::Identity) && (Instances.NodeTrackers.Num() > 1) && Instances.HasMesh();  

		TSharedPtr<IDatasmithActorElement> DatasmithActorElement;
		TSharedPtr<IDatasmithMeshActorElement> DatasmithMeshActor;

		FString UniqueName = FString::FromInt(NodeTracker.Node->GetHandle());
		FString Label = NodeTrackersNames.GetNodeName(NodeTracker);

		// Create and setup mesh actor if there's a mesh
		if (Instances.HasMesh())
		{
			FString MeshActorName = UniqueName;
			if (bNeedPivotComponent)
			{
				MeshActorName += TEXT("_Pivot");
			}

			FString MeshActorLabel = NodeTrackersNames.GetNodeName(NodeTracker);
			DatasmithMeshActor = FDatasmithSceneFactory::CreateMeshActor(*MeshActorName);
			DatasmithMeshActor->SetLabel(*Label);

			TOptional<FDatasmithMaxStaticMeshAttributes> DatasmithAttributes = FDatasmithMaxStaticMeshAttributes::ExtractStaticMeshAttributes(NodeTracker.Node);
			if (DatasmithAttributes &&  (DatasmithAttributes->GetExportMode() == EStaticMeshExportMode::BoundingBox))
			{
				DatasmithMeshActor->AddTag(TEXT("Datasmith.Attributes.Geometry: BoundingBox"));
			}

			DatasmithMeshActor->SetStaticMeshPathName(Instances.GetStaticMeshPathName());
		}

		// Create a dummy actor in case pivot is non-degenerate or there's no mesh(so no mesh actor)
		if (bNeedPivotComponent || !Instances.HasMesh())
		{
			DatasmithActorElement = FDatasmithSceneFactory::CreateActor(*UniqueName);
			DatasmithActorElement->SetLabel(*Label);
		}
		else
		{
			DatasmithActorElement = DatasmithMeshActor;
		}

		// Set transforms

		// Remove pivot from the node actor transform when pivot is baked or separate component is created for pivot
		FTransform NodeTransform = (bPivotBakedInMesh || bNeedPivotComponent) ? (Pivot.Inverse() * ObjectTransform) : ObjectTransform;

		DatasmithActorElement->SetTranslation(NodeTransform.GetTranslation());
		DatasmithActorElement->SetScale(NodeTransform.GetScale3D());
		DatasmithActorElement->SetRotation(NodeTransform.GetRotation());

		if (bNeedPivotComponent) 
		{
			// Setup mesh actor with (relative)pivot transform
			DatasmithMeshActor->SetTranslation(Pivot.GetTranslation());
			DatasmithMeshActor->SetRotation(Pivot.GetRotation());
			DatasmithMeshActor->SetScale(Pivot.GetScale3D());
			DatasmithMeshActor->SetIsAComponent( true );

			DatasmithActorElement->AddChild(DatasmithMeshActor, EDatasmithActorAttachmentRule::KeepRelativeTransform);
		}

		FNodeConverted& Converted = NodeTracker.CreateConverted();
		Converted.DatasmithActorElement = DatasmithActorElement;
		Converted.DatasmithMeshActor = DatasmithMeshActor;

		UpdateNodeMetadata(NodeTracker);
		TagsConverter.ConvertNodeTags(NodeTracker);
		if (NodeTracker.Layer)
		{
			Converted.DatasmithActorElement->SetLayer(*NodeTracker.Layer->Name);
		}

		// Apply material
		if (Mtl* Material = UpdateGeometryNodeMaterial(*this, Instances, NodeTracker))
		{
			// Assign materials
			if (!MeshConverter.bMaterialsAssignedToStaticMesh)
			{
				if (Instances.Material != Material)
				{
					MaterialsCollectionTracker.SetMaterialsForMeshActor(NodeTracker.GetConverted().DatasmithMeshActor, Material, Instances.Converted.SupportedChannels, FVector3f(NodeTracker.GetConverted().DatasmithMeshActor->GetTranslation()));
				}
			}
		}

	}

	IDatasmithScene& GetDatasmithScene()
	{
		return *ExportedScene.GetDatasmithScene();
	}

	virtual void AddMeshElement(TSharedPtr<IDatasmithMeshElement>& DatasmithMeshElement, FDatasmithMesh& DatasmithMesh, FDatasmithMesh* CollisionMesh) override
	{
		GetDatasmithScene().AddMesh(DatasmithMeshElement);

		// todo: parallelize this
		FDatasmithMeshExporter DatasmithMeshExporter;
		if (DatasmithMeshExporter.ExportToUObject(DatasmithMeshElement, ExportedScene.GetSceneExporter().GetAssetsOutputPath(), DatasmithMesh, CollisionMesh, FDatasmithExportOptions::LightmapUV))
		{
			// todo: handle error exporting mesh?
		}
	}

	void UpdateInstancesGeometry(FInstances& Instances, FNodeTracker& NodeTracker)
	{
		INode* Node = NodeTracker.Node;

		FString MeshName = FString::FromInt(Node->GetHandle());

		FMeshConverterSource MeshSource = {
			Node, MeshName,
			GeomUtils::GetMeshForGeomObject(CurrentSyncPoint.Time, Node, Instances.bShouldBakePivot ? Instances.BakePivot : FTransform::Identity), false,
			GeomUtils::GetMeshForCollision(CurrentSyncPoint.Time, *this, Node, Instances.bShouldBakePivot),
		};

		Instances.ValidityInterval = MeshSource.GetValidityInterval();

		if (MeshSource.IsValid())
		{
			bool bHasInstanceWithMultimat = false;
			for (FNodeTracker* InstanceNodeTracker : Instances.NodeTrackers)
			{
				if (Mtl* Material = InstanceNodeTracker->Node->GetMtl())
				{
					if (FDatasmithMaxMatHelper::GetMaterialClass(Material) == EDSMaterialType::MultiMat)
					{
						bHasInstanceWithMultimat = true;
					}
				}
			}

			MeshSource.bConsolidateMaterialIds = !bHasInstanceWithMultimat;

			// Not sharing meshes for regular nodes when they don't resolve to the same Max object -
			// resusing identical(but different) mesh is done only for RailClone instances - RC tends to make
			// different in different RC objects instancing same Max meshes
			Meshes.AddMesh(*this, MeshSource, Instances.Converted, false, [&](bool bHasConverted, bool bWasReused)
			{
				if (bHasConverted)
				{
					Instances.Converted.GetDatasmithMeshElement()->SetLabel(*NodeTrackersNames.GetNodeName(NodeTracker));
				}
				else
				{
					Instances.Converted.DatasmithMeshElement.Reset();
				}
			});
		}
		else
		{
			// Whe RenderMesh can be null?
			// seems like the only way is to static_cast<GeomObject*>(Obj) return null in GetMeshFromRenderMesh?
			// Where Obj is return value of GetBaseObject(Node, CurrentTime)
			// Or ObjectState.obj is null
			ensure(false);
			ReleaseMeshElement(Instances.Converted);
		}
	}

	virtual void SetupActor(FNodeTracker& NodeTracker) override
	{
		NodeTracker.GetConverted().DatasmithActorElement->SetLabel(*NodeTrackersNames.GetNodeName(NodeTracker));

		UpdateNodeMetadata(NodeTracker);
		TagsConverter.ConvertNodeTags(NodeTracker);
		if (NodeTracker.Layer)
		{
			NodeTracker.GetConverted().DatasmithActorElement->SetLayer(*NodeTracker.Layer->Name);
		}

		FTransform ObjectTransform;
		GetNodeObjectTransform(NodeTracker, ObjectTransform);

		FTransform NodeTransform = ObjectTransform;
		TSharedRef<IDatasmithActorElement> DatasmithActorElement = NodeTracker.GetConverted().DatasmithActorElement.ToSharedRef();
		DatasmithActorElement->SetTranslation(NodeTransform.GetTranslation());
		DatasmithActorElement->SetScale(NodeTransform.GetScale3D());
		DatasmithActorElement->SetRotation(NodeTransform.GetRotation());
	}

	class FMeshes
	{
	public:

		TMap<FMD5Hash, TSharedPtr<FDatasmithMeshConverter>> CachedMeshes;
		TMap<TSharedPtr<IDatasmithMeshElement>, FMD5Hash> CachedMeshHashes;
		TMap<TSharedPtr<IDatasmithMeshElement>, int32> MeshUsers;

		// CompletionCallback is called when mehs is processed, passing flag identifing if actual Datasmith mesh was created(sometimes it's not created - e.g. mesh is empty)
		// or is a cached mesh was reused
		FORCENOINLINE
		void AddMesh(FSceneTracker& Scene, FMeshConverterSource& MeshSource, FMeshConverted& MeshConverted, bool bShouldCache, TFunction<void(bool bConverted, bool bFoundCached)> CompletionCallback)
		{

			TSharedPtr<FDatasmithMeshConverter> MeshConverter = MakeShared<FDatasmithMeshConverter>();
			bool bConverted = ConvertMaxMeshToDatasmith(Scene.CurrentSyncPoint.Time, MeshSource, *MeshConverter);
			bool bFoundCached = false;
			if (bConverted)
			{
				FMD5Hash MeshHash;
				if (bShouldCache)
				{
					MeshHash = MeshConverter->ComputeHash();

					if (TSharedPtr<FDatasmithMeshConverter>* CachedMeshConverter = CachedMeshes.Find(MeshHash))
					{
						// If there's mesh that is the same cached reuse it
						// For RailClone/ForestPack meshes we have no other way to find out if RC objects are using the same mesh than to compare those meshes
						// For regular Max nodes it's mush easier to find instances but RC gives away different Mesh objects for identical instances
						bFoundCached = true;
						MeshConverter = *CachedMeshConverter;
					}
					else
					{
						CachedMeshes.Add(MeshHash, MeshConverter);
					}
				}

				if (bFoundCached)
				{
					if (MeshConverter->Converted.GetDatasmithMeshElement() != MeshConverted.GetDatasmithMeshElement())
					{
						// Release old converted mesh and hold onto new if new mesh is different(same was already cached)
						Scene.ReleaseMeshElement(MeshConverted);
						Acquire(MeshConverter->Converted);
					}
				}
				else
				{
					// Release old converted mesh before building new
					Scene.ReleaseMeshElement(MeshConverted);
	
					MeshConverter->Converted.DatasmithMeshElement = FDatasmithSceneFactory::CreateMesh(*MeshSource.MeshName);
					MeshConverter->Converted.SupportedChannels = MeshConverter->SupportedChannels;
					MeshConverter->Converted.UVChannelsMap = MeshConverter->UVChannelsMap;
					if (bShouldCache)
					{
						CachedMeshHashes.Add(MeshConverter->Converted.DatasmithMeshElement, MeshHash);
					}

					int32 SelectedLightmapUVChannel = MeshConverter->SelectedLightmapUVChannel;
					if (SelectedLightmapUVChannel >= 0)
					{
						constexpr int32 MaxToUnrealUVOffset = -1;
						constexpr int32 DefaultValue = -1;
						const int32* ExportedSelectedChannel = MeshConverter->UVChannelsMap.Find(SelectedLightmapUVChannel + MaxToUnrealUVOffset);

						if (ExportedSelectedChannel)
						{
							MeshConverter->Converted.DatasmithMeshElement->SetLightmapCoordinateIndex(*ExportedSelectedChannel);
						}
						else if (SelectedLightmapUVChannel != DefaultValue)
						{
							LogWarning(*FString::Printf(TEXT("%s won't use the channel %i for its lightmap because it's not supported by the mesh. A new channel will be generated.")
							                            , static_cast<const TCHAR*>(MeshSource.Node->GetName())
							                            , SelectedLightmapUVChannel));
						}
					}

					// Actually export mesh to disk
					Scene.AddMeshElement(MeshConverter->Converted.DatasmithMeshElement, MeshConverter->RenderMesh, MeshConverter->bHasCollision ? &MeshConverter->CollisionMesh : nullptr);
					Acquire(MeshConverter->Converted);
				}
				MeshConverted = MeshConverter->Converted;
			}
			else
			{
				Scene.ReleaseMeshElement(MeshConverted);
				MeshConverted = FMeshConverted();
			}

			CompletionCallback(bConverted, bFoundCached);
		}

		void Reset()
		{
			CachedMeshes.Empty();
			CachedMeshHashes.Empty();
			MeshUsers.Empty();
		}

		void Acquire(const FMeshConverted& Converted)
		{
			MeshUsers.FindOrAdd(Converted.DatasmithMeshElement, 0) += 1;
		}

		// Returns true if mesh has no users afterwards
		bool Release(const FMeshConverted& Converted)
		{
			if (int32* Found = MeshUsers.Find(Converted.DatasmithMeshElement))
			{
				int32& UserCount = *Found;
				UserCount -= 1;

				if (UserCount == 0)
				{
					return true;
				}
			}
			return false;
		}

		void Remove(FMeshConverted& Converted)
		{
			MeshUsers.Remove(Converted.DatasmithMeshElement);
			FMD5Hash MeshHash;
			if (CachedMeshHashes.RemoveAndCopyValue(Converted.DatasmithMeshElement, MeshHash))
			{
				CachedMeshes.Remove(MeshHash);
			}

			Converted.ReleaseMeshConverted();
		}
	};

	virtual void SetupDatasmithHISMForNode(FNodeTracker& NodeTracker, FMeshConverterSource& MeshSource, Mtl* Material, int32 MeshIndex, const TArray<Matrix3>& Transforms) override
	{
		MeshSource.MeshName = FString::FromInt(NodeTracker.Node->GetHandle()) + TEXT("_") + FString::FromInt(MeshIndex);
		// note: when export Mesh goes to other place due to parallelizing it's result would be unknown here so MeshIndex handling will change(i.e. increment for any mesh)

		FMeshConverted MeshConverted; // todo: possible reuse of previous converted mesh
		Meshes.AddMesh(*this, MeshSource, MeshConverted, true, [&](bool bHasConverted, bool bReused)
		{
			if (bHasConverted)
			{
				FHismNodeConverter& NodeConverter = static_cast<FHismNodeConverter&>(NodeTracker.GetConverter());
				NodeConverter.Meshes.Add(MeshConverted);

				RegisterNodeForMaterial(NodeTracker, Material);
				FString MeshLabel = FString(MeshSource.Node->GetName()) + (TEXT("_") + FString::FromInt(MeshIndex));

				if (!bReused)
				{
					MaterialsCollectionTracker.SetMaterialsForMeshElement(MeshConverted, Material);
					MeshConverted.GetDatasmithMeshElement()->SetLabel(*MeshLabel);
				}

				FDatasmithConverter Converter;

				// todo: override material
				TSharedPtr< IDatasmithActorElement > InversedHISMActor;
				// todo: ExportHierarchicalInstanceStaticMeshActor CustomMeshNode only used for Material - can be simplified, Material anyway is dealt with outside too
				TSharedRef<IDatasmithActorElement> HismActorElement = ExportHierarchicalInstanceStaticMeshActor(NodeTracker.Node, MeshSource.Node, *MeshLabel, MeshConverted.SupportedChannels,
					Material, &Transforms, MeshConverted.GetDatasmithMeshElement()->GetName(), Converter.UnitToCentimeter, EStaticMeshExportMode::Default, InversedHISMActor);
				NodeTracker.GetConverted().DatasmithActorElement->AddChild(HismActorElement, EDatasmithActorAttachmentRule::KeepWorldTransform);
				if (InversedHISMActor)
				{
					NodeTracker.GetConverted().DatasmithActorElement->AddChild(InversedHISMActor, EDatasmithActorAttachmentRule::KeepWorldTransform);
				}

				MeshIndex++;
			}
		});
	}

	TSharedRef< IDatasmithActorElement > ExportHierarchicalInstanceStaticMeshActor(INode* Node, INode* CustomMeshNode, const TCHAR* Label, TSet<uint16>& SupportedChannels, Mtl* StaticMeshMtl, const TArray<Matrix3>* Instances,
		const TCHAR* MeshName, float UnitMultiplier, const EStaticMeshExportMode& ExportMode, TSharedPtr< IDatasmithActorElement >& OutInversedHISMActor)
	{
		check(Node && Instances && MeshName);

		TSharedPtr< IDatasmithMeshActorElement > MeshActor;
		FVector Pos, Scale;
		FQuat Rotation;

		const FVector3f RandomSeed(FMath::Rand(), FMath::Rand(), FMath::Rand());
		auto FinalizeHISMActor = [&](TSharedPtr< IDatasmithMeshActorElement >& FinalizingActor, FVector3f Seed, const TCHAR* ActorLabel)
		{
			FinalizingActor->SetStaticMeshPathName(MeshName);

			INode* MeshNode = CustomMeshNode ? CustomMeshNode : Node;

			if (ExportMode == EStaticMeshExportMode::Default && StaticMeshMtl != MeshNode->GetMtl())
			{
				TSharedRef< IDatasmithMeshActorElement > MeshActorRef = FinalizingActor.ToSharedRef();
				MaterialsCollectionTracker.SetMaterialsForMeshActor(MeshActorRef, StaticMeshMtl, SupportedChannels, Seed);
			}

			if (ActorLabel)
			{
				FinalizingActor->SetLabel(ActorLabel);
			}

			FinalizingActor->SetIsAComponent(true);

			switch (ExportMode)
			{
			case EStaticMeshExportMode::BoundingBox:
				FinalizingActor->AddTag(TEXT("Datasmith.Attributes.Geometry: BoundingBox"));
				break;
			default:
				break;
			}
		};

		if (Node->GetWSMDerivedObject() != nullptr)
		{
			FDatasmithMaxSceneExporter::MaxToUnrealCoordinates(Node->GetObjTMAfterWSM(GetCOREInterface()->GetTime()), Pos, Rotation, Scale, UnitMultiplier);
		}
		else
		{
			FDatasmithMaxSceneExporter::MaxToUnrealCoordinates(Node->GetObjectTM(GetCOREInterface()->GetTime()), Pos, Rotation, Scale, UnitMultiplier);
		}

		const int32 InstancesCount = Instances->Num();
		if ( InstancesCount == 1 )
		{
			// Export the the hism as a normal static mesh
			MeshActor = FDatasmithSceneFactory::CreateMeshActor(MeshName);

			// Apply the relative transfrom of the instance to the forest world transform
			FTransform WithoutInstance(Rotation, Pos, Scale);
			FVector InstancePos, InstanceScale;
			FQuat InstanceRotation;
			FDatasmithMaxSceneExporter::MaxToUnrealCoordinates((*Instances)[0], InstancePos, InstanceRotation, InstanceScale, UnitMultiplier);
			FTransform WithInstance(FTransform(InstanceRotation, InstancePos, InstanceScale) * WithoutInstance);

			MeshActor->SetTranslation(WithInstance.GetLocation());
			MeshActor->SetScale(WithInstance.GetScale3D());
			MeshActor->SetRotation(WithInstance.GetRotation());
		}
		else
		{
			TSharedRef< IDatasmithHierarchicalInstancedStaticMeshActorElement > HierarchicalInstanceStaticMeshActor = FDatasmithSceneFactory::CreateHierarchicalInstanceStaticMeshActor(MeshName);
			MeshActor = HierarchicalInstanceStaticMeshActor;

			FTransform HISMActorTransform(Rotation, Pos, Scale);
			MeshActor->SetTranslation(Pos);
			MeshActor->SetScale(Scale);
			MeshActor->SetRotation(Rotation);

			TArray< FTransform > InstancesTransform;
			InstancesTransform.Reserve(InstancesCount);
			for (int32 i = 0; i < InstancesCount; i++)
			{
				FDatasmithMaxSceneExporter::MaxToUnrealCoordinates((*Instances)[i], Pos, Rotation, Scale, UnitMultiplier);
				InstancesTransform.Emplace(Rotation, Pos, Scale);
			}

			TArray< FTransform > NonInvertedTransforms, InvertedTransforms;
			DatasmithMaxHelper::FilterInvertedScaleTransforms(InstancesTransform, NonInvertedTransforms, InvertedTransforms);

			HierarchicalInstanceStaticMeshActor->ReserveSpaceForInstances(NonInvertedTransforms.Num());
			for (const FTransform& InstanceTransform : NonInvertedTransforms)
			{
				HierarchicalInstanceStaticMeshActor->AddInstance(InstanceTransform);
			}

			//Adding an inverted HierarchicalInstancedStaticMeshActorElement for the instances with an inverted mesh due to negative scaling.
			if (InvertedTransforms.Num() > 0)
			{
				FString InvertedHISMName = FString(MeshName).Append("_inv");
				TSharedPtr< IDatasmithHierarchicalInstancedStaticMeshActorElement > InvertedHierarchicalInstanceStaticMeshActor = FDatasmithSceneFactory::CreateHierarchicalInstanceStaticMeshActor(*InvertedHISMName);
				OutInversedHISMActor = InvertedHierarchicalInstanceStaticMeshActor;

				OutInversedHISMActor->SetTranslation( HISMActorTransform.GetTranslation() );
				OutInversedHISMActor->SetScale( -1 * HISMActorTransform.GetScale3D() );
				OutInversedHISMActor->SetRotation( HISMActorTransform.GetRotation() );
				
				InvertedHierarchicalInstanceStaticMeshActor->ReserveSpaceForInstances(InvertedTransforms.Num());
				for (const FTransform& InstanceTransform : InvertedTransforms)
				{
					//Correcting the instance transform to reverse the negative scaling of the parent.
					InvertedHierarchicalInstanceStaticMeshActor->AddInstance( FTransform( InstanceTransform.GetRotation(), -1 * InstanceTransform.GetTranslation(), -1 * InstanceTransform.GetScale3D() ) );
				}

				TSharedPtr< IDatasmithMeshActorElement > InvertedMeshActor = InvertedHierarchicalInstanceStaticMeshActor;
				if (Label)
				{
					FString InversedLabelString = FString(Label).Append("_inv");
					FinalizeHISMActor(InvertedMeshActor, RandomSeed, *InversedLabelString);
				}
				else
				{
					FinalizeHISMActor(InvertedMeshActor, RandomSeed, nullptr);
				}
			}
		}

		FinalizeHISMActor(MeshActor, RandomSeed, Label);

		return MeshActor.ToSharedRef();
	}



	void ConvertNamedCollisionNode(FNodeTracker& NodeTracker)
	{
		// Split collision prefix and find node that might use this node as collision mesh
		FString NodeName = NodeTrackersNames.GetNodeName(NodeTracker);
		FString LeftString, RightString;
		NodeName.Split(TEXT("_"), &LeftString, &RightString);

		FNodeTracker* CollisionUserNodeTracker = GetNodeTrackerByNodeName(*RightString);
		
		if (!CollisionUserNodeTracker)
		{
			return;
		}

		if (CollisionUserNodeTracker->GetConverterType() == FNodeConverter::MeshNode)
		{
			if (FInstances* Instances = InstancesManager.GetInstancesForNodeTracker(*CollisionUserNodeTracker))
			{
				InvalidateInstances(*Instances);
			}
		}
	}

	/******************* Events *****************************/

	virtual void NodeAdded(INode* Node) override
	{
		LogDebugNode(TEXT("NodeAdded"), Node);
		// Node sometimes is null. 'Added' NodeEvent might come after node was actually deleted(immediately after creation)
		// e.g.[mxs]: b = box(); delete b
		// NodeEvents are delayed(not executed in the same stack frame as command that causes them) so they come later.
		if (!Node)
		{
			return;
		}

		FNodeKey NodeKey = NodeEventNamespace::GetKeyByNode(Node);

		if (FNodeTracker* NodeTracker = GetNodeTracker(NodeKey))
		{
			// Re-added trackd node, probably deleted before(e.g. delete then undo)
			InvalidateNode(*NodeTracker);
			NodeTracker->bDeleted = false;
		}
		else
		{
			AddNode(Node);
		}
	}

	virtual void NodeXRefMerged(INode* Node) override
	{
		if (!Node)
		{
			return;
		}

		// Search where this XRef Tree is attached to the scene
		int32 XRefIndex = -1; // Node that has this xref scene attached to(e.g. to place in hierarchy and to transform)
		INode* SceneRootNode = GetCOREInterface()->GetRootNode();
		for (int XRefChild = 0; XRefChild < SceneRootNode->GetXRefFileCount(); ++XRefChild)
		{
			if (Node == SceneRootNode->GetXRefTree(XRefChild))
			{
				XRefIndex = XRefChild;
			}
		}

		NotificationsHandler->AddNode(Node);

		if (!bIncludeXRefWhileParsing)
		{
			ParseScene(Node, FXRefScene{SceneRootNode, XRefIndex}); // Parse xref hierarchy - it won't add itself! Or will it?
		}
	}

	virtual void NodeDeleted(INode* Node) override
	{
		LogDebugNode(TEXT("NodeDeleted"), Node);

		if (FNodeTracker* NodeTracker = GetNodeTracker(Node))
		{
			InvalidatedNodeTrackers.Add(*NodeTracker);
			NodeTracker->bDeleted = true;
			PromoteValidity(*NodeTracker, NodeTracker->Validity);
		}
	}

	virtual void NodeTransformChanged(INode* Node) override
	{
		if (FNodeTracker* NodeTracker =  GetNodeTracker(Node))
		{
			//todo: handle more precisely
			InvalidateNode(*NodeTracker);
		}
	}

	virtual void NodeMaterialAssignmentChanged(FNodeKey NodeKey) override
	{
		//todo: handle more precisely
		InvalidateNode(NodeKey);
	}

	virtual void NodeMaterialAssignmentChanged(INode* Node) override
	{
		if (FNodeTracker* NodeTracker =  GetNodeTracker(Node))
		{
			//todo: handle more precisely
			InvalidateNode(*NodeTracker);
		}
	}

	virtual void NodeMaterialGraphModified(FNodeKey NodeKey) override
	{
		if (FNodeTracker* NodeTracker =  GetNodeTracker(NodeKey))
		{
			NodeMaterialGraphModified(*NodeTracker);
		}
	}

	virtual void NodeMaterialGraphModified(INode* Node) override
	{
		if (FNodeTracker* NodeTracker =  GetNodeTracker(Node))
		{
			NodeMaterialGraphModified(*NodeTracker);
		}
	}

	void NodeMaterialGraphModified(FNodeTracker& NodeTracker)
	{
		if (Mtl* Material = NodeTracker.Node->GetMtl())
		{
			MaterialsCollectionTracker.InvalidateMaterial(Material);
		}
		InvalidateNode(NodeTracker); // Invalidate node that has this material assigned. This is needed to trigger rebuild - exported geometry might change(e.g. multimaterial changed to slots will change on static mesh)
	}

	virtual void MaterialGraphModified(Mtl* Material) override
	{
		MaterialsCollectionTracker.InvalidateMaterial(Material);

		FMaterialTracker* MaterialTracker = MaterialsCollectionTracker.AddMaterial(Material);
		
		if (TSet<FNodeTracker*>* NodeTrackersPtr = MaterialsAssignedToNodes.Find(MaterialTracker))
		{
			for (FNodeTracker* NodeTracker : *NodeTrackersPtr)
			{
				InvalidateNode(*NodeTracker);
			}
		}
	}

	virtual void NodeGeometryChanged(INode* Node) override
	{
		// GeometryChanged is executed to handle:
		// - actual geometry modification(in any way)
		// - change of baseObject

		if (FNodeTracker* NodeTracker =  GetNodeTracker(Node))
		{
			InvalidateNode(*NodeTracker);
		}
	}

	virtual void NodeHideChanged(INode* Node) override
	{
		// todo: invalidate visibility only - note to handle this not enought add/remove
		// actor. make sure to invalidate instances(in case geometry usage changed - like hidden node with multimat), materials

		if (FNodeTracker* NodeTracker =  GetNodeTracker(Node))
		{
			InvalidateNode(*NodeTracker);
		}
	}

	virtual void HideByCategoryChanged()
	{
		bHideByCategoryChanged = true;
	}

	virtual void NodeNameChanged(FNodeKey NodeKey) override
	{
		if (FNodeTracker* NodeTracker =  GetNodeTracker(NodeKey))
		{
			if (!NodeTracker->bDeleted)
			{
				NodeTrackersNames.Update(*NodeTracker);
			}
			InvalidateNode(*NodeTracker);
		}
	}

	virtual void NodePropertiesChanged(INode* Node) override
	{
		// todo: invalidate visibility only - note to handle this not enought add/remove
		// actor. make sure to invalidate instances(in case geometry usage changed - like hidden node with multimat), materials

		if (FNodeTracker* NodeTracker =  GetNodeTracker(Node))
		{
			InvalidateNode(*NodeTracker);
		}
	}

	virtual void NodeLinkChanged(FNodeKey NodeKey) override
	{
		InvalidateNode(NodeKey);
	}

	virtual FSceneUpdateStats& GetStats() override
	{
		return Stats;
	}

	virtual FNodeTracker* GetNodeTrackerByNodeName(const TCHAR* Name) override
	{
		FNodeTracker* Result = nullptr;
		NodeTrackersNames.EnumerateForName(Name, [&](FNodeTracker& NodeTracker)
		{
			// todo: currently support only one name/node(as Max api GetINodeByName)
			//   so collision with prefix only takes one node. But this can change.
			Result = &NodeTracker;
		});

		return Result;
	}
	
	void Reset()
	{

		NodeTrackers.Reset();
		NodeTrackersNames.Reset();
		InstancesManager.Reset();
		CollisionNodes.Reset();

		LayersForAnimHandle.Reset();
		NodesPerLayer.Reset();

		Meshes.Reset();

		MaterialsCollectionTracker.Reset();
		MaterialsAssignedToNodes.Reset();

		NodeDatasmithMetadata.Reset();

		InvalidatedNodeTrackers.Reset();
		InvalidatedInstances.Reset();

		IesTextures.Reset();
	}

	///////////////////////////////////////////////

	const FExportOptions& Options;
	FDatasmith3dsMaxScene& ExportedScene;

	FNotifications* NotificationsHandler;

	bool bUpdateInProgress = false;

	// Scene tracked/converted entities and connections beetween them
	TMap<FNodeKey, FNodeTrackerHandle> NodeTrackers; // All scene nodes
	FNodeTrackersNames NodeTrackersNames; // Nodes grouped by name, for faster access
	FInstancesManager InstancesManager; // Groups geometry nodes by their shared mesh
	TMap<FNodeTracker*, TSet<FNodeTracker*>> CollisionNodes; // Nodes used as collision meshes for other nodes, counted by each user

	TMap<AnimHandle, TUniquePtr<FLayerTracker>> LayersForAnimHandle;
	TMap<FLayerTracker*, TSet<FNodeTracker*>> NodesPerLayer;

	FMeshes Meshes;

	FMaterialsCollectionTracker MaterialsCollectionTracker;
	TMap<FMaterialTracker*, TSet<FNodeTracker*>> MaterialsAssignedToNodes;

	FIesTexturesCollection IesTextures;

	TMap<FNodeTracker*, TSharedPtr<IDatasmithMetaDataElement>> NodeDatasmithMetadata; // All scene nodes

	bool bHideByCategoryChanged = false;
	DWORD HideByCategoryFlags = 0;

	// Nodes/instances that need to be rebuilt
	FInvalidatedNodeTrackers InvalidatedNodeTrackers;
	TSet<FInstances*> InvalidatedInstances;

	// Utility
	FSceneUpdateStats Stats;
	FTagsConverter TagsConverter; // Converts max node information to Datasmith tags
	FNodesPreparer NodesPreparer;

};

class FExporter: public IExporter
{
public:
	FExporter(FExportOptions& InOptions): Options(InOptions), NotificationsHandler(*this), SceneTracker(Options, ExportedScene, &NotificationsHandler)
	{
		ResetSceneTracking();
		InitializeDirectLinkForScene(); // Setup DL connection immediately when plugin loaded
	}

	virtual void Shutdown() override;

	virtual void SetOutputPath(const TCHAR* Path) override
	{
		OutputPath = Path;
		ExportedScene.SetOutputPath(*OutputPath);
	}

	virtual void SetName(const TCHAR* Name) override
	{
		ExportedScene.SetName(Name);
	}

	virtual void InitializeScene() override
	{
		ExportedScene.SetupScene();
	}

	virtual void InitializeDirectLinkForScene() override
	{
		if (DirectLinkImpl)
		{
			return;
		}

		InitializeScene();

		// XXX: PreExport needs to be called before DirectLink instance is constructed -
		// Reason - it calls initialization of FTaskGraphInterface. Callstack:
		// PreExport:
		//  - FDatasmithExporterManager::Initialize
		//	-- DatasmithGameThread::InitializeInCurrentThread
		//  --- GEngineLoop.PreInit
		//  ---- PreInitPreStartupScreen
		//  ----- FTaskGraphInterface::Startup
		ExportedScene.PreExport();

		SetOutputPath(GetDirectlinkCacheDirectory());
		FString SceneName = FPaths::GetBaseFilename(GetCOREInterface()->GetCurFileName().data());
		SetName(*SceneName);

		DirectLinkImpl.Reset(new FDatasmithDirectLink);
		DirectLinkImpl->InitializeForScene(ExportedScene.GetDatasmithScene().ToSharedRef());
	}

	virtual void UpdateDirectLinkScene() override
	{
		if (!DirectLinkImpl)
		{
			// InitializeDirectLinkForScene wasn't called yet. This rarely happens when Sync is pressed right before event like PostSceneReset(for New All UI command) was handled
			// Very quickly! Unfortunately code needs to wait for PostSceneReset to get proper scene name there(no earlier event signals that name is available)
			InitializeDirectLinkForScene();
		}

		LogDebug(TEXT("UpdateDirectLinkScene"));
		DirectLinkImpl->UpdateScene(ExportedScene.GetDatasmithScene().ToSharedRef());
		StartSceneChangeTracking(); // Always track scene changes if it's synced with DirectLink
	}

	static VOID AutoSyncTimerProc(HWND, UINT, UINT_PTR TimerIdentifier, DWORD)
	{
		reinterpret_cast<FExporter*>(TimerIdentifier)->UpdateAutoSync();
	}

	// Update is user was idle for some time
	void UpdateAutoSync()
	{
		LASTINPUTINFO LastInputInfo;
		LastInputInfo.cbSize = sizeof(LASTINPUTINFO);
		LastInputInfo.dwTime = 0;
		if (GetLastInputInfo(&LastInputInfo))
		{

// Disable "Consider using 'GetTickCount64' instead of 'GetTickCount'" warning - we don't GetLastInputInfo returns results of GetTickCount
// and we won't miss much a single update when 32-bit timer wraparound would happen
#pragma warning( push )
#pragma warning( disable: 28159 )
			DWORD CurrentTime = GetTickCount();
			int32 IdlePeriod = GetTickCount() - LastInputInfo.dwTime;
#pragma warning( pop )
			LogDebug(FString::Printf(TEXT("CurrentTime: %ld, Idle time: %ld, IdlePeriod: %ld"), CurrentTime, LastInputInfo.dwTime, IdlePeriod));

			if (IdlePeriod > FMath::RoundToInt(AutoSyncIdleDelaySeconds*1000))
			{
				PerformAutoSync();
			}
		}
	}

	virtual bool IsAutoSyncEnabled() override
	{
		return bAutoSyncEnabled;
	}

	virtual bool ToggleAutoSync() override
	{
		if (bAutoSyncEnabled)
		{
			KillTimer(GetCOREInterface()->GetMAXHWnd(), reinterpret_cast<UINT_PTR>(this));
		}
		else
		{
			// Perform full Sync when AutoSync is first enabled
			PerformSync(false);

			const uint32 AutoSyncCheckIntervalMs = FMath::RoundToInt(AutoSyncDelaySeconds*1000);
			SetTimer(GetCOREInterface()->GetMAXHWnd(), reinterpret_cast<UINT_PTR>(this), AutoSyncCheckIntervalMs, AutoSyncTimerProc);
		}
		bAutoSyncEnabled = !bAutoSyncEnabled;

		LogDebug(bAutoSyncEnabled ? TEXT("AutoSync ON") : TEXT("AutoSync OFF"));
		return bAutoSyncEnabled;
	}

	virtual void SetAutoSyncDelay(float Seconds) override
	{
		AutoSyncDelaySeconds = Seconds;
	}

	virtual void SetAutoSyncIdleDelay(float Seconds) override
	{
		AutoSyncIdleDelaySeconds = Seconds;
	}

	// Install change notification systems
	virtual void StartSceneChangeTracking() override
	{
		NotificationsHandler.StartSceneChangeTracking();
	}

	virtual bool UpdateScene(bool bQuiet) override
	{
		FUpdateProgress ProgressManager(!bQuiet, 1);

		bool bResult = SceneTracker.Update(ProgressManager.MainStage, true);

		ProgressManager.Finished();

		if (Options.bStatSync)
		{
			ProgressManager.PrintStatisticss();
		}
		return bResult;
	}

	virtual void PerformSync(bool bQuiet) override
	{
		FUpdateProgress ProgressManager(!bQuiet, 1);
		FUpdateProgress::FStage& MainStage = ProgressManager.MainStage;

		if (SceneTracker.Update(MainStage, true))
		{
			PROGRESS_STAGE("Sync With DirectLink")
			UpdateDirectLinkScene();
		}

		ProgressManager.Finished();

		if (Options.bStatSync)
		{
			LogCompletion("Sync completed");

			ProgressManager.PrintStatisticss();
		}
	}

	virtual void PerformAutoSync()
	{
		// Don't create progress bar for autosync - it steals focus, closes listener and what else
		// todo: consider creating progress when a big change in scene is detected, e.g. number of nodes?
		bool bQuiet = true;

		FUpdateProgress ProgressManager(!bQuiet, 1);
		FUpdateProgress::FStage& MainStage = ProgressManager.MainStage;

		if (SceneTracker.Update(MainStage, true))  // Don't sent redundant update if scene change wasn't detected
		{
			PROGRESS_STAGE("Sync With DirectLink")
			UpdateDirectLinkScene();
		}

		ProgressManager.Finished();

		if (Options.bStatSync)
		{
			LogCompletion("AutoSync completed:");
			ProgressManager.PrintStatisticss();
		}
	}

	virtual void ResetSceneTracking() override
	{
		NotificationsHandler.StopSceneChangeTracking();
		if (IsAutoSyncEnabled())
		{
			ToggleAutoSync();
		}

		ExportedScene.ResetScene();

		SceneTracker.Reset();

		DirectLinkImpl.Reset();
	}

	virtual ISceneTracker& GetSceneTracker() override
	{
		return SceneTracker;
	}

	FExportOptions& Options;

	FDatasmith3dsMaxScene ExportedScene;
	TUniquePtr<FDatasmithDirectLink> DirectLinkImpl;
	FString OutputPath;

	FNotifications NotificationsHandler;
	FSceneTracker SceneTracker;

	bool bAutoSyncEnabled = false;
	float AutoSyncDelaySeconds = 0.5f; // AutoSync is attempted periodically using this interval
	float AutoSyncIdleDelaySeconds = 0.5f; // Time period user should be idle to run AutoSync

};

FPersistentExportOptions PersistentExportOptions;

TUniquePtr<IExporter> Exporter;

bool CreateExporter(bool bEnableUI, const TCHAR* EnginePath)
{
	FDatasmithExporterManager::FInitOptions Options;
	Options.bEnableMessaging = true; // DirectLink requires the Messaging service.
	Options.bSuppressLogs = false;   // Log are useful, don't suppress them
	Options.bUseDatasmithExporterUI = bEnableUI;
	Options.RemoteEngineDirPath = EnginePath;

	if (!FDatasmithExporterManager::Initialize(Options))
	{
		LogError(TEXT("Failed to initialize Datasmith Exporter Manager"));
		return false;
	}

	if (int32 ErrorCode = FDatasmithDirectLink::ValidateCommunicationSetup())
	{
		LogError(TEXT("Failed to validate DatasmithDirect Link Communication setup"));
		return false;
	}

	PersistentExportOptions.Load(); // Access GConfig only after FDatasmithExporterManager::Initialize finishes, which ensures that Unreal game thread was initialized(GConfig is created there)
	Exporter = MakeUnique<FExporter>(PersistentExportOptions.Options);
	return true;
}

void ShutdownExporter()
{
	ShutdownScripts();
	Exporter.Reset();
	FDatasmithDirectLink::Shutdown();
	FDatasmithExporterManager::Shutdown();
}

IExporter* GetExporter()
{
	return Exporter.Get();
}

IPersistentExportOptions& GetPersistentExportOptions()
{
	return PersistentExportOptions;
}

bool FInvalidatedNodeTrackers::PurgeDeletedNodeTrackers(FSceneTracker& Scene)
{
	TArray<FNodeTracker*> DeletedNodeTrackers;
	for (FNodeTracker* NodeTracker : InvalidatedNodeTrackers)
	{
		if (NodeTracker->bDeleted)
		{
			DeletedNodeTrackers.Add(NodeTracker);
		}
	}

	for (FNodeTracker* NodeTrackerPtr : DeletedNodeTrackers)
	{
		Scene.RemoveNodeTracker(*NodeTrackerPtr);
	}

	return !DeletedNodeTrackers.IsEmpty(); // If the only change is deleted node than we need to record it(deleted will be removed from InvalidatedNodeTrackers)
}

void FExporter::Shutdown()
{
	Exporter.Reset();
	FDatasmithDirectLink::Shutdown();
	FDatasmithExporterManager::Shutdown();
}

bool Export(const TCHAR* Name, const TCHAR* OutputPath, bool bQuiet, bool bSelected)
{
	FUpdateProgress ProgressManager(!bQuiet, 3);
	FUpdateProgress::FStage& MainStage = ProgressManager.MainStage;

	FDatasmith3dsMaxScene ExportedScene;
	ExportedScene.SetupScene();
	ExportedScene.SetName(Name);
	ExportedScene.SetOutputPath(OutputPath);

	FExportOptions ExportOptions = PersistentExportOptions.Options;
	ExportOptions.bSelectedOnly = bSelected;
	FSceneTracker SceneTracker((ExportOptions), ExportedScene, nullptr);

	bool bCancelled = false;

	if (!SceneTracker.Update(MainStage, true))
	{
		bCancelled = true;
	}

	if (ExportOptions.bAnimatedTransforms && !bCancelled)
	{
		if (!SceneTracker.ExportAnimations(MainStage))
		{
			bCancelled = true;
		}
	}

	if (!bCancelled)
	{
		PROGRESS_STAGE("Save Datasmith Scene");

		IDatasmithScene& Scene = *ExportedScene.GetDatasmithScene();
		ExportedScene.GetSceneExporter().Export(ExportedScene.GetDatasmithScene().ToSharedRef(), false);

		PROGRESS_STAGE_RESULT(FString::Printf(TEXT("Actors: %d; Meshes: %d, Materials: %d Textures: %d"), 
			Scene.GetActorsCount(),
			Scene.GetMeshesCount(),
			Scene.GetMaterialsCount(),
			Scene.GetTexturesCount()
			));
	}

	ProgressManager.Finished();

	if (!bCancelled)
	{
		LogInfo(TEXT("Export completed:"));
	}
	else
	{
		LogWarning(TEXT("Export cancelled by User"));
	}

	ProgressManager.PrintStatisticss();

	return true;
}

bool OpenDirectLinkUI()
{
	if (IDatasmithExporterUIModule* Module = IDatasmithExporterUIModule::Get())
	{
		if (IDirectLinkUI* UI = Module->GetDirectLinkExporterUI())
		{
			UI->OpenDirectLinkStreamWindow();
			return true;
		}
	}
	return false;
}

const TCHAR* GetDirectlinkCacheDirectory()
{
	if (IDatasmithExporterUIModule* Module = IDatasmithExporterUIModule::Get())
	{
		if (IDirectLinkUI* UI = Module->GetDirectLinkExporterUI())
		{
			return UI->GetDirectLinkCacheDirectory();
		}
	}
	return nullptr;
}

FDatasmithConverter::FDatasmithConverter(): UnitToCentimeter(FMath::Abs(GetSystemUnitScale(UNITS_CENTIMETERS)))
{
}

}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // NEW_DIRECTLINK_PLUGIN
