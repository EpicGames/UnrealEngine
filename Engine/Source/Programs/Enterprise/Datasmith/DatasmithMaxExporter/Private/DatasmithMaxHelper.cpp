// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithMaxHelper.h"

FScopedBitMapPtr::FScopedBitMapPtr(const BitmapInfo& InMapInfo, Bitmap* InMap) : MapInfo(InMapInfo)
{
	if (InMap)
	{
		Map = InMap;
	}
	else
	{
		Map = TheManager->Load(&MapInfo);
		bNeedsDelete = Map != nullptr;
	}
}

FScopedBitMapPtr::~FScopedBitMapPtr()
{
	//If we load the bitmap it's our job to delete it as well.
	if (bNeedsDelete && Map)
	{
		Map->DeleteThis();
	}
}

void DatasmithMaxHelper::FilterInvertedScaleTransforms(const TArray<FTransform>& Transforms, TArray<FTransform>& OutNormal, TArray<FTransform>& OutInverted)
{
	for (const FTransform& CurrentTransform : Transforms)
	{
		FVector Scale(CurrentTransform.GetScale3D());

		if ((Scale.X * Scale.Y * Scale.Z) < 0)
		{
			OutInverted.Emplace(CurrentTransform);
		}
		else
		{
			OutNormal.Emplace(CurrentTransform);
		}
	}
}
