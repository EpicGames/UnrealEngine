// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxLogger.h"

#include "DatasmithMaxDirectLink.h"

#include "DatasmithMaxExporterDefines.h"

#include "HAL/UnrealMemory.h"

#include "Windows/AllowWindowsPlatformTypes.h"

MAX_INCLUDES_START
#include "max.h"
MAX_INCLUDES_END

DatasmithMaxLogger& DatasmithMaxLogger::Get()
{
	static TSharedRef< DatasmithMaxLogger > Instance = MakeShared< DatasmithMaxLogger >(); // Create a shared ref because we support TSharedFromThis

	return Instance.Get();
}

void DatasmithMaxLogger::Purge()
{
	ResetGeneralErrors();
	ResetTextureErrors();
	ResetMissingAssetErrors();
	PartialSupportedMats.Empty();
	UnsupportedMats.Empty();
	PartialSupportedMaps.Empty();
	UnsupportedMaps.Empty();
	UnsupportedLight.Empty();
	FailUVs.Empty();
	FailObjs.Empty();
	InvalidTransforms.Empty();
}

void DatasmithMaxLogger::AddPartialSupportedMat(Mtl* Mat)
{
	for (int i = 0; i < PartialSupportedMats.Num(); i++)
	{
		if (Mat->ClassID() == PartialSupportedMats[i]->ClassID())
		{
			return;
		}
	}

	PartialSupportedMats.Add(Mat);
}

void DatasmithMaxLogger::AddUnsupportedMat(Mtl* Mat)
{
	for (int i = 0; i < UnsupportedMats.Num(); i++)
	{
		if (Mat->ClassID() == UnsupportedMats[i]->ClassID())
		{
			return;
		}
	}

	UnsupportedMats.Add(Mat);
}

void DatasmithMaxLogger::AddPartialSupportedMap(Texmap* Map)
{
	for (int i = 0; i < PartialSupportedMaps.Num(); i++)
	{
		if (Map->ClassID() == PartialSupportedMaps[i]->ClassID())
		{
			return;
		}
	}

	PartialSupportedMaps.Add(Map);
}

void DatasmithMaxLogger::AddUnsupportedMap(Texmap* Map)
{

	for (int i = 0; i < UnsupportedMaps.Num(); i++)
	{
		if (Map->ClassID() == UnsupportedMaps[i]->ClassID())
		{
			return;
		}
	}

	MSTR Classname;
	Map->GetClassName(Classname);
	DatasmithMaxDirectLink::LogWarning(FString::Printf(TEXT("Unsupported texmap \"%s\" of type %s (0x%08x-0x%08x)"), Map->GetName().ToBSTR(), Classname.ToBSTR(), Map->ClassID().PartA(), Map->ClassID().PartB()));

	UnsupportedMaps.Add(Map);
}

void DatasmithMaxLogger::AddUnsupportedLight(INode* Light)
{
	for (int i = 0; i < UnsupportedLight.Num(); i++)
	{
		if (Light->ClassID() == UnsupportedLight[i]->ClassID())
		{
			return;
		}
	}

	UnsupportedLight.Add(Light);
}

void DatasmithMaxLogger::AddUnsupportedUV(INode* Node)
{
	FailUVs.AddUnique(Node);
}

void DatasmithMaxLogger::AddInvalidObj(INode* Node)
{
	FailObjs.AddUnique(Node);
}

TArray<INode*> DatasmithMaxLogger::GetInvalidObjects() const
{
	return FailObjs;
}

void DatasmithMaxLogger::AddInvalidTransform(INode* Node)
{
	InvalidTransforms.AddUnique(Node);
}

bool DatasmithMaxLogger::HasWarnings()
{
	if (GetGeneralErrorsCount() > 0 || GetTextureErrorsCount() > 0 || GetMissingAssetErrorsCount() > 0 ||
		PartialSupportedMats.Num() > 0 || UnsupportedMats.Num() > 0 ||
		PartialSupportedMaps.Num()>0 || UnsupportedMaps.Num()>0 || UnsupportedLight.Num() > 0 || FailUVs.Num() > 0)
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool DatasmithMaxLogger::CopyToClipBoard()
{
	if ( ShowMessage.IsEmpty() )
	{
		return false;
	}

	if (OpenClipboard(NULL))
	{
		EmptyClipboard();

		HGLOBAL HClipboardData;
		size_t Num = (ShowMessage.Len() + 1) * sizeof(WCHAR);
		HClipboardData = GlobalAlloc(NULL, Num);

		WCHAR* PchData = (WCHAR*)GlobalLock(HClipboardData);
		FMemory::Memcpy(PchData, *ShowMessage, Num);

		SetClipboardData(CF_UNICODETEXT, HClipboardData);
		GlobalUnlock(HClipboardData);
		CloseClipboard();

		return true;
	}

	return false;
}

void DatasmithMaxLogger::AddItem(const TCHAR* Msg, HWND Handle, FString& FullMsg)
{
	SendDlgItemMessage(Handle, IDC_ERROR_MSG_LIST, LB_ADDSTRING, NULL, (LPARAM)Msg);
	FullMsg += Msg;
	FullMsg += LINE_TERMINATOR;
}

void DatasmithMaxLogger::AddObjectList(TArray< INode* > ObjectList, HWND Handle, const TCHAR* Header, const TCHAR* Description)
{
	const int32 MAX_ITEMS_TO_SHOW = 20;

	if ( ObjectList.Num() > 0 )
	{
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		AddItem(*(FString( TEXT("\t") ) + Header), Handle, ShowMessage);
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);

		if ( Description && FCString::Strlen( Description ) )
		{
			AddItem(Description, Handle, ShowMessage);
		}

		FString ObjectListString;
		int j = 0;
		for (int i = 0; i < FMath::Min( ObjectList.Num(), MAX_ITEMS_TO_SHOW ); i++)
		{
			if (j > 0)
			{
				ObjectListString += TEXT(", [");
			}
			else
			{
				ObjectListString += TEXT("[");
			}

			ObjectListString += ObjectList[i]->GetName();
			ObjectListString += TEXT("]");
			j++;

			if (j == 4)
			{
				j = 0;
				AddItem(*ObjectListString, Handle, ShowMessage);
				ObjectListString.Empty();
			}
		}

		if (!ObjectListString.IsEmpty())
		{
			AddItem(*ObjectListString, Handle, ShowMessage);
		}

		if (ObjectList.Num() > MAX_ITEMS_TO_SHOW)
		{
			ObjectListString = TEXT("...and ") + FString::FromInt(ObjectList.Num() - MAX_ITEMS_TO_SHOW) + TEXT(" more");
			AddItem(*ObjectListString, Handle, ShowMessage);
		}
		AddItem(TEXT("\n\n"), Handle, ShowMessage);
	}
}

FString DatasmithMaxLogger::GetLightDescription(INode* LightNode)
{
	ObjectState State = LightNode->EvalWorldState(0);
	LightObject *Light = (LightObject*)State.obj;
	MSTR Classname;
	Light->GetClassName(Classname);
	return FString::Printf(TEXT("\"%s\" of type %s (0x%08x-0x%08x)"), LightNode->GetName(), Classname.ToBSTR(), Light->ClassID().PartA(), Light->ClassID().PartB());
}

void DatasmithMaxLogger::Show(HWND Handle)
{
	ShowMessage = TEXT("");

	if (GetGeneralErrorsCount() > 0)
	{
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		AddItem(TEXT("\tGENERAL MESSAGES"), Handle, ShowMessage);
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);

		for (int i = 0; i < GetGeneralErrorsCount(); i++)
		{
			AddItem(GetGeneralError(i), Handle, ShowMessage);
		}
		
		AddItem(TEXT("\n\n"), Handle, ShowMessage);
	}


	if (GetMissingAssetErrorsCount() > 0)
	{
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		AddItem(TEXT("\tMISSING EXTERNAL FILES"), Handle, ShowMessage);
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		AddItem(TEXT("One or more file required to export correctly to Unreal is missing,"), Handle, ShowMessage);
		AddItem(TEXT("which can produce incorrect results. We recommend that you solve the issue"), Handle, ShowMessage);
		AddItem(TEXT("with the 3ds Max's Asset Tracking tool prior to export."), Handle, ShowMessage);

		for (int i = 0; i < GetMissingAssetErrorsCount(); i++)
		{
			AddItem(GetMissingAssetError(i), Handle, ShowMessage);
		}

		AddItem(TEXT("\n\n"), Handle, ShowMessage);
	}

	// Invalid Objects
	AddObjectList( FailObjs, Handle, TEXT("INVALID OBJECTS (unsupported, corrupted or too small geometry?)") );

	if (PartialSupportedMats.Num() > 0 || PartialSupportedMaps.Num()>0)
	{
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		AddItem(TEXT("\tPARTIALLY SUPPORTED"), Handle, ShowMessage);
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);

		for (int i = 0; i < PartialSupportedMats.Num(); i++)
		{
			MSTR Classname;
			PartialSupportedMats[i]->GetClassName(Classname);
			FString Msg = TEXT("Materials of type ") + FString(Classname.ToBSTR()) + TEXT(" are not supported and its first children will be used.");
			AddItem(*Msg, Handle, ShowMessage);
		}

		for (int i = 0; i < PartialSupportedMaps.Num(); i++)
		{
			MSTR Classname;
			PartialSupportedMaps[i]->GetClassName(Classname);;
			FString Msg = TEXT("Texmaps of type ") + FString(Classname.ToBSTR()) + TEXT(" are not supported and its first children will be used.");
			AddItem(*Msg, Handle, ShowMessage);
		}

		ShowMessage += TEXT("\n\n");
		SendDlgItemMessage(Handle, IDC_ERROR_MSG_LIST, LB_ADDSTRING, NULL, (LPARAM)TEXT("\n"));
		SendDlgItemMessage(Handle, IDC_ERROR_MSG_LIST, LB_ADDSTRING, NULL, (LPARAM)TEXT("\n"));
	}

	if (UnsupportedMats.Num() > 0)
	{
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		AddItem(TEXT("\tMATERIALS"), Handle, ShowMessage);
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		AddItem(TEXT("Some materials used in the scene are not supported by the Unreal Datasmith format in the moment. They will not be imported inside Unreal Engine."), Handle, ShowMessage);

		for (int i = 0; i < UnsupportedMats.Num(); i++)
		{
			MSTR Classname;
			UnsupportedMats[i]->GetClassName(Classname);
			Mtl* Mat = UnsupportedMats[i];
			FString Msg = FString::Printf(TEXT("\"%s\" of type %s (0x%08x-0x%08x)"), Mat->GetName().ToBSTR(), Classname.ToBSTR(), Mat->ClassID().PartA(), Mat->ClassID().PartB());
			AddItem(*Msg, Handle, ShowMessage);
		}
		AddItem(TEXT("\n\n"), Handle, ShowMessage);
	}


	if (UnsupportedMaps.Num() > 0 || GetTextureErrorsCount() > 0)
	{
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		AddItem(TEXT("\tTEXTUREMAPS"), Handle, ShowMessage);
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		if (UnsupportedMaps.Num() > 0)
		{
			AddItem(TEXT("Some texture maps used in the scene are not supported by the Unreal Datasmith format in the moment. They will not be imported inside Unreal Engine."), Handle, ShowMessage);
			for (int i = 0; i < UnsupportedMaps.Num(); i++)
			{
				MSTR Classname;
				UnsupportedMaps[i]->GetClassName(Classname);
				Texmap* Map = UnsupportedMaps[i];
				FString Msg = FString::Printf(TEXT("\"%s\" of type %s (0x%08x-0x%08x)"), Map->GetName().ToBSTR(), Classname.ToBSTR(), Map->ClassID().PartA(), Map->ClassID().PartB());
				AddItem(*Msg, Handle, ShowMessage);
			}
		}
		if (GetTextureErrorsCount() > 0)
		{
			for (int i = 0; i < GetTextureErrorsCount(); i++)
			{
				AddItem(GetTextureError(i), Handle, ShowMessage);
			}
		}
		AddItem(TEXT("\n\n"), Handle, ShowMessage);
	}

	if (UnsupportedLight.Num() > 0)
	{
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		AddItem(TEXT("\tLIGHTS"), Handle, ShowMessage);
		AddItem(TEXT("------------------------------------------------------------------------------------------"), Handle, ShowMessage);
		AddItem(TEXT("Some lights used in the scene are not supported by the Unreal Datasmith format in the moment. They will not be imported inside Unreal Engine."), Handle, ShowMessage);

		for (int i = 0; i < UnsupportedLight.Num(); i++)
		{
			FString Msg = GetLightDescription(UnsupportedLight[i]);
			AddItem(*Msg, Handle, ShowMessage);
		}
		AddItem(TEXT("\n\n"), Handle, ShowMessage);
	}
	
	// Invalid UVs
	AddObjectList( FailUVs, Handle, TEXT("MESHES WITHOUT UV CHANNEL 2 (USED FOR LIGHT BAKING)") );

	// Invalid Transforms
	FString InvalidTransformsDescription = TEXT("Some objects have customized pivots and nonuniform scales.") + FString( LINE_TERMINATOR );
	InvalidTransformsDescription += TEXT("Those combinations are not supported by Unreal and will produce incorrect results.") + FString( LINE_TERMINATOR );
	InvalidTransformsDescription += TEXT("It is recommended to use the Reset XForm Utility on those objects:") + FString( LINE_TERMINATOR );

	AddObjectList( InvalidTransforms, Handle, TEXT("INVALID TRANSFORMS"), *InvalidTransformsDescription );
}

INT_PTR CALLBACK MsgListDlgProc(HWND hDlg, UINT iMsg, WPARAM wParam, LPARAM lParam)
{
	int Tabs[3] = { 24, 24, 24 };
	

	switch (iMsg)
	{
		case WM_INITDIALOG:
			CenterWindow(hDlg, GetWindow(hDlg, GW_OWNER));
			SendDlgItemMessage(hDlg, IDC_ERROR_MSG_LIST, LB_RESETCONTENT, 0, 0);
			SendDlgItemMessage(hDlg, IDC_ERROR_MSG_LIST, LB_SETTABSTOPS, (WPARAM)3, (LPARAM)Tabs);

			EnableWindow(GetDlgItem(hDlg, IDSELECTINVALID), DatasmithMaxLogger::Get().GetInvalidObjects().Num()>0);
			DatasmithMaxLogger::Get().Show(hDlg);
			return TRUE;
			
		case WM_COMMAND:
			switch (LOWORD(wParam))
			{
				case IDCOPY:
					DatasmithMaxLogger::Get().CopyToClipBoard();
					break;
				case IDSELECTINVALID:
					INodeTab SelectedNodes;
					for (int i = 0; i < DatasmithMaxLogger::Get().GetInvalidObjects().Num(); i++)
					{
						SelectedNodes.AppendNode(DatasmithMaxLogger::Get().GetInvalidObjects()[i]);
					}

					GetCOREInterface()->ClearNodeSelection();
					GetCOREInterface()->SelectNodeTab(SelectedNodes,TRUE,TRUE);
					break;
			}
			break;

		case WM_CLOSE:
			EndDialog(hDlg, TRUE);
			break;

		case WM_SIZE:
		{
			int DialogWidth = (int)LOWORD(lParam);
			int DialogHeight = (int)HIWORD(lParam);
			HWND hList = GetDlgItem(hDlg, IDC_ERROR_MSG_LIST);
			HWND hCopy = GetDlgItem(hDlg, IDCOPY);
			HWND hInvalid = GetDlgItem(hDlg, IDSELECTINVALID);

			MoveWindow(hList, 6, 7, (DialogWidth - 12), (DialogHeight - 42), true);
			MoveWindow(hCopy, 24, DialogHeight - 31, (DialogWidth / 2 - 24), 26, true);
			MoveWindow(hInvalid, (DialogWidth / 2) + 3  , DialogHeight - 31, (DialogWidth / 2 - 23), 26, true);

			break;
		}
	}
	return FALSE;
}

#include "Windows/HideWindowsPlatformTypes.h"
