// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxWriter.h"
#include "DatasmithMaxSceneHelper.h"

void FDatasmithMaxMatWriter::ExportArchDesignMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	TSharedPtr< IDatasmithShaderElement > MaterialShader = FDatasmithSceneFactory::CreateShader((TCHAR*)Material->GetName().data());

	int NumParamBlocks = Material->NumParamBlocks();

	bool bDiffuseTexEnable = true;
	bool bReflectanceTexEnable = true;
	bool bRefractTexEnable = true;
	bool bEmitTexEnable = true;
	bool bGlossyTexEnable = true;
	bool bBumpTexEnable = true;
	bool bOpacityTexEnable = true;

	float DiffuseTexAmount = 0.f;
	float ReflectanceTexAmount = 0.f;
	float RefractTexAmount = 0.f;

	bool bRefleFresnel = true;
	float BumpAmount = 0;

	Texmap* EmitTex = NULL;
	bool bEmitUseTemperature = true;

	BMM_Color_fl ColorReflection;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM0")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bDiffuseTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM2")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bReflectanceTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM4")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bRefractTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("cutout_map")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bOpacityTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM3")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bGlossyTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_color_mode")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bEmitUseTemperature = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bEmitTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_map")) == 0)
			{
				EmitTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bBumpTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map_amt")) == 0)
			{
				BumpAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("diff_color_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bDiffuseTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_color_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bReflectanceTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refr_color_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bRefractTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_gloss_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bGlossyTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bBumpTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_opacity_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bOpacityTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_func_fresnel")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 1)
				{
					bRefleFresnel = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("diff_weight")) == 0)
			{
				DiffuseTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_weight")) == 0)
			{
				ReflectanceTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refr_weight")) == 0)
			{
				RefractTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_color")) == 0 )
			{
				ColorReflection = (BMM_Color_fl)ParamBlock2->GetAColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refr_ior")) == 0) 
			{
				MaterialShader->SetIORRefra( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
				if (bRefleFresnel)
				{
					MaterialShader->SetIOR( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM0")) == 0 && bDiffuseTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				DumpTexture(DatasmithScene, MaterialShader->GetDiffuseComp(), LocalTex, DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("diff_color")) == 0 && bDiffuseTexEnable == false)
			{
				BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetAColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
				Color.r *= DiffuseTexAmount;
				Color.g *= DiffuseTexAmount;
				Color.b *= DiffuseTexAmount;
				MaterialShader->GetDiffuseComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM2")) == 0 && bReflectanceTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (ReflectanceTexAmount == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetRefleComp(), LocalTex, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
				}
				else
				{
					DumpWeightedTexture(DatasmithScene, MaterialShader->GetRefleComp(), LocalTex, BMM_Color_fl(0, 0, 0, 1), ReflectanceTexAmount, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_color")) == 0 && bReflectanceTexEnable == false)
			{
				BMM_Color_fl Color = ColorReflection;
				Color.r *= ReflectanceTexAmount;
				Color.g *= ReflectanceTexAmount;
				Color.b *= ReflectanceTexAmount;
				MaterialShader->GetRefleComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor( Color));
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM3")) == 0 && bGlossyTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				DumpTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, true, true);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_gloss")) == 0 && bGlossyTexEnable == false)
			{
				float Glossy = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());

				if (bReflectanceTexEnable == false && ColorReflection.r == 0 && ColorReflection.g == 0 && ColorReflection.b == 0)
				{
					MaterialShader->GetRoughnessComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(0.75f, TEXT("Roughness")) );
				}
				else
				{
					MaterialShader->GetRoughnessComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(1.0f - Glossy, TEXT("Roughness")) );
				}
			}			
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM4")) == 0 && bRefractTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (RefractTexAmount == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetTransComp(), LocalTex, DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, true, false);
				}
				else
				{
					DumpWeightedTexture(DatasmithScene, MaterialShader->GetTransComp(), LocalTex, BMM_Color_fl(0, 0, 0, 1), RefractTexAmount, DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, true,false);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refr_color")) == 0 && bRefractTexEnable == false && RefractTexAmount > 0)
			{
				BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetAColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
				Color.r *= RefractTexAmount;
				Color.g *= RefractTexAmount;
				Color.b *= RefractTexAmount;
				MaterialShader->GetTransComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("cutout_map")) == 0 && bOpacityTexEnable)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				DumpTexture(DatasmithScene, MaterialShader->GetMaskComp(), LocalTex, DATASMITH_CLIPTEXNAME, DATASMITH_CLIPTEXNAME, false, true);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map")) == 0)
			{
				if (bBumpTexEnable == true && BumpAmount>0)
				{
					MaterialShader->SetBumpAmount(BumpAmount);
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					if (FDatasmithMaxMatHelper::GetTextureClass(LocalTex) == EDSBitmapType::NormalMap)
					{
						DumpTexture(DatasmithScene, MaterialShader->GetNormalComp(), LocalTex, DATASMITH_NORMALTEXNAME, DATASMITH_NORMALTEXNAME, false, false);
						DumpTexture(DatasmithScene, MaterialShader->GetBumpComp(), LocalTex, DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
					}
					else
					{
						DumpTexture(DatasmithScene, MaterialShader->GetBumpComp(), LocalTex, DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
					}
				}
			}
			else if (bEmitUseTemperature)
			{
				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_color_kelvin")) == 0 && bEmitTexEnable)
				{
					MaterialShader->SetEmitTemperature( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
					MaterialShader->SetEmitPower( 100.0 );
				}
			}
			else
			{
				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_map")) == 0 && bEmitTexEnable && EmitTex != NULL)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					DumpTexture(DatasmithScene, MaterialShader->GetEmitComp(), LocalTex, DATASMITH_EMITTEXNAME, DATASMITH_EMITTEXNAME, false , false);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_color_filter")) == 0 && bEmitTexEnable && EmitTex == NULL)
				{
					BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetAColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
					MaterialShader->GetEmitComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
				}

			}
		}
		ParamBlock2->ReleaseDesc();

	}

	MaterialElement->AddShader(MaterialShader);
}
