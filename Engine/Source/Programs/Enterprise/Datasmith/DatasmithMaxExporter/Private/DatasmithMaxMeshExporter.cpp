// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxMeshExporter.h"

#include "DatasmithMaxAttributes.h"
#include "DatasmithExportOptions.h"
#include "DatasmithUtils.h"
#include "DatasmithMaxExporterDefines.h"
#include "DatasmithMaxSceneHelper.h"
#include "DatasmithMesh.h"
#include "DatasmithMeshExporter.h"
#include "DatasmithMaxLogger.h"
#include "DatasmithMaxSceneExporter.h"
#include "DatasmithMaxWriter.h"

#include "Math/NumericLimits.h"
#include "Misc/Paths.h"

#include "Windows/AllowWindowsPlatformTypes.h"

#pragma warning(push)
#pragma warning(disable: 4535)
MAX_INCLUDES_START
	#include "inode.h"
	#include "maxscript/maxscript.h"
	#include "mesh.h"
	#include "meshnormalspec.h"
	#include "modstack.h"
#if MAX_PRODUCT_YEAR_NUMBER > 2022
#include "geom/point3.h"
#else
#include "point3.h"
#endif
MAX_INCLUDES_END
#pragma warning(pop)


//Node used for temporal operations like reading proxy meshes
#define TEMPORALNODENAME TEXT("TEMPORALNODE123456789@987654321")

#define VRAY_PROXY_DISPLAY_AS_MESH	4	// Value to set on a VRay Mesh Proxy to get the mesh

namespace
{
	class NullView : public View
	{
	public:
		NullView()
		{
			worldToView.IdentityMatrix(); screenW = 640.0f; screenH = 480.0f;
		}

		virtual Point2 ViewToScreen(Point3 p) override
		{
			return Point2(p.x, p.y);
		}
	};

	Object* GetBaseObject(INode* Node, TimeValue Time)
	{
		ObjectState ObjState = Node->EvalWorldState(Time);
		Object* Obj = ObjState.obj;

		if (Obj)
		{
			SClass_ID SuperClassID;
			SuperClassID = Obj->SuperClassID();
			while (SuperClassID == GEN_DERIVOB_CLASS_ID)
			{
				Obj = ((IDerivedObject*)Obj)->GetObjRef();
				SuperClassID = Obj->SuperClassID();
			}
		}

		return Obj;
	}

	int SetObjectParamValue(Object *Obj, const FString& ParamName, int DesiredValue)
	{
		bool bFoundDisplayValue = false;
		int PrevDisplayValue = DesiredValue; // Display value to see mesh
		int NumParamBlocks = Obj->NumParamBlocks();
		for (short BlockIndex = 0; BlockIndex < NumParamBlocks && !bFoundDisplayValue; ++BlockIndex)
		{
			IParamBlock2* ParamBlock2 = Obj->GetParamBlockByID(BlockIndex);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
			for (int ParamIndex = 0; ParamIndex < ParamBlockDesc->count; ++ParamIndex)
			{
				ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[ParamIndex];
				if (FCString::Stricmp(ParamDefinition.int_name, *ParamName) == 0)
				{
					PrevDisplayValue = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
					if (PrevDisplayValue != DesiredValue)
					{
						ParamBlock2->SetValue(ParamDefinition.ID, GetCOREInterface()->GetTime(), DesiredValue);
					}
					bFoundDisplayValue = true;
					break;
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		return PrevDisplayValue;
	}

	Mesh* GetMeshFromRenderMesh(INode* Node, BOOL& bNeedsDelete, TimeValue CurrentTime)
	{
		Object* Obj = GetBaseObject(Node, CurrentTime);
		const Class_ID& ObjectClassID = Obj->ClassID();
		const FString VRayProxyParamName(TEXT("display"));
		const FString BodyObjectViewportMeshParamName(TEXT("RenderViewportMeshRA"));
		int PreviousMeshDisplayValue = 0;

		if (ObjectClassID == VRAYPROXY_CLASS_ID)
		{
			// Need the high resolution render mesh associated with the VRay Mesh Proxy for the export
			PreviousMeshDisplayValue = SetObjectParamValue(Obj, VRayProxyParamName, VRAY_PROXY_DISPLAY_AS_MESH);
		}
		else if(ObjectClassID == BODYOBJECT_CLASS_ID)
		{
			// Need to make sure we are using the viewport mesh on BodyObject, otherwise the RenderMesh gives a tessellated low resolution mesh.
			PreviousMeshDisplayValue = SetObjectParamValue(Obj, BodyObjectViewportMeshParamName, 1);
		}

		GeomObject* GeomObj = static_cast<GeomObject*>(Obj);
		if (GeomObj == nullptr)
		{
			return nullptr;
		}

		NullView View;
		Mesh* RenderMesh = GeomObj->GetRenderMesh(CurrentTime, Node, View, bNeedsDelete);

		// Restore display state if different from mesh display
		if (ObjectClassID == VRAYPROXY_CLASS_ID && PreviousMeshDisplayValue != VRAY_PROXY_DISPLAY_AS_MESH)
		{
			SetObjectParamValue(Obj, VRayProxyParamName, PreviousMeshDisplayValue);
		}
		else if(ObjectClassID == BODYOBJECT_CLASS_ID && PreviousMeshDisplayValue != 1)
		{
			SetObjectParamValue(Obj, BodyObjectViewportMeshParamName, PreviousMeshDisplayValue);
		}

		return RenderMesh;
	}

	/** Convert Max to UE coordinates, handle scene master unit
	 * @param MaxTransform      source transform
	 * @param UnitMultiplier    Master scene unit
	 */
	FTransform FTransformFromMatrix3(const Matrix3& MaxTransform, float UnitMultiplier)
	{
		FVector Translation;
		FQuat Rotation;
		FVector Scale;
		FDatasmithMaxSceneExporter::MaxToUnrealCoordinates( MaxTransform, Translation, Rotation, Scale, UnitMultiplier);
		return FTransform( Rotation, Translation, Scale );
	}


	/*
	 * Set a normal to the three points of a face
	 * @param DatasmithMesh		The mesh on which the normal will be applied
	 * @param Index				The index of the face on which we want to applied the normal
	 * @param NormalVector		The normal
	 */
	void SetNormalForAFace(FDatasmithMesh& DatasmithMesh, int32 Index, const FVector& NormalVector)
	{
		DatasmithMesh.SetNormal(Index * 3, NormalVector.X, NormalVector.Y, NormalVector.Z);
		DatasmithMesh.SetNormal(Index * 3 + 1, NormalVector.X, NormalVector.Y, NormalVector.Z);
		DatasmithMesh.SetNormal(Index * 3 + 2, NormalVector.X, NormalVector.Y, NormalVector.Z);
	}

	FColor ConvertPoint3ToColor(Point3& Point)
	{
		//The 3ds Max vertex colors are encoded from 0 to 1 in float
		return FColor(Point.x * MAX_uint8, Point.y * MAX_uint8, Point.z * MAX_uint8);
	}
}

INode* FDatasmithMaxMeshExporter::GetTemporalNode() const
{
	return GetCOREInterface()->GetINodeByName(TEMPORALNODENAME);
}

bool FDatasmithMaxMeshExporter::DeleteTemporalNode() const
{
	INode* TemporalNode = GetTemporalNode();
	if (TemporalNode == NULL)
	{
		return false;
	}

	return GetCOREInterface()->DeleteNode(TemporalNode, FALSE) != 0;
}

INode* FDatasmithMaxMeshExporter::GetCollisionNode(INode* OriginalNode, const FDatasmithMaxStaticMeshAttributes* DatasmithAttributes, bool& bOutFromDatasmithAttributes) 
{
	if (DatasmithAttributes)
	{
		INode* ModifierSpecifiedCustomCollisionNode = DatasmithAttributes->GetCustomCollisonNode();
		if (ModifierSpecifiedCustomCollisionNode)
		{
			bOutFromDatasmithAttributes = true;
			return ModifierSpecifiedCustomCollisionNode;
		}
	}

	FString OriginalName = OriginalNode->GetName();
	for ( const FString& CollisionNodePrefix : FDatasmithMaxSceneHelper::CollisionNodesPrefixes )
	{
		INode* CollisionNode = GetCOREInterface()->GetINodeByName( *( CollisionNodePrefix + TEXT("_") + OriginalName ) );
		if (CollisionNode)
		{
			bOutFromDatasmithAttributes = false;
			return CollisionNode;
		}
	}

	return nullptr;
}

MeshNormalSpec* FDatasmithMaxMeshExporter::GetSpecifiedNormalsFromMesh(Mesh* MaxMesh) const
{
	MeshNormalSpec* Normal = MaxMesh->GetSpecifiedNormals();
	bool bNeedsRecreateNormals = false;
	if (!Normal)
	{
		bNeedsRecreateNormals = true;
	}
	else if (Normal->GetNumNormals() < 3)
	{
		bNeedsRecreateNormals = true;
	}

	if (bNeedsRecreateNormals)
	{
		MaxMesh->SpecifyNormals();
		Normal = MaxMesh->GetSpecifiedNormals(); // vertex normals fix 
		Normal->BuildNormals();
		Normal->CheckNormals();
		Normal->MakeNormalsExplicit(false);
	}
	else
	{
		Normal->CheckNormals();
	}

	return Normal;
}

FDatasmithMaxMeshExporter::FDatasmithMaxMeshExporter()
{
}

FDatasmithMaxMeshExporter::~FDatasmithMaxMeshExporter()
{
}

bool FDatasmithMaxMeshExporter::CalcSupportedChannelsOnly(INode* Node, TSet<uint16>& SupportedChannels, bool bForceSingleMat) const
{
	if (Node == NULL)
	{
		return false;
	}

	TimeValue CurrentTime = GetCOREInterface()->GetTime();
	BOOL bNeedsDelete;

	Mesh* NodeMesh = GetMeshFromRenderMesh(Node, bNeedsDelete, CurrentTime);
	if (NodeMesh == NULL)
	{
		return false;
	}

	int NumFaces = NodeMesh->getNumFaces();
	if (NumFaces < 1)
	{
		if (bNeedsDelete)
		{
			NodeMesh->DeleteThis();
		}

		DeleteTemporalNode();
		return false;
	}

	if (bForceSingleMat)
	{
		SupportedChannels.Add(0);
	}
	else
	{
		for (int i = 0; i < NumFaces; i++)
		{
			const uint16 FaceId = NodeMesh->faces[i].getMatID();
			SupportedChannels.Add(FaceId);
		}
	}

	if (bNeedsDelete)
	{
		NodeMesh->DeleteThis();
	}

	DeleteTemporalNode();

	return true;
}

FVector FDatasmithMaxMeshExporter::Unit3ToFVector(Point3 Point) const
{
	return FVector(	UnitMultiplier *  Point.x,
					UnitMultiplier * -Point.y,
					UnitMultiplier *  Point.z );
}

FVector FDatasmithMaxMeshExporter::Point3ToFVector(Point3 Point) const
{
	return FVector(	 Point.x,
					-Point.y,
					 Point.z );
}

bool FDatasmithMaxMeshExporter::CreateDatasmithMesh(FDatasmithMesh& DSMesh, FMaxExportMeshArgs& ExportMeshArgs, TSet<uint16>& SupportedChannels)
{
	BOOL bNeedsDelete = false;

	Mesh* MaxMesh = ExportMeshArgs.MaxMesh ? ExportMeshArgs.MaxMesh : GetMeshFromRenderMesh(ExportMeshArgs.Node, bNeedsDelete, GetCOREInterface()->GetTime());
	if (MaxMesh == NULL)
	{
		return false;
	}

	if (MaxMesh->getNumFaces() == 0)
	{
		if (bNeedsDelete)
		{
			MaxMesh->DeleteThis();
		}
		return false;
	}

	Mesh CachedMesh;
	CachedMesh.DeepCopy(MaxMesh, TOPO_CHANNEL | GEOM_CHANNEL | TEXMAP_CHANNEL | VERTCOLOR_CHANNEL);

	if (bNeedsDelete)
	{
		MaxMesh->DeleteThis();
	}

	CachedMesh.DeleteIsoVerts();
	CachedMesh.RemoveDegenerateFaces();
	CachedMesh.RemoveIllegalFaces();
	CachedMesh.InvalidateStrips();
	CachedMesh.BuildStripsAndEdges();


	int NumFaces = CachedMesh.getNumFaces();
	int NumVerts = CachedMesh.getNumVerts();

	//0 faces or less than 0.5 cm2
	if (NumFaces < 1 || NumVerts < 1)
	{
		CachedMesh.FreeAll();
		DeleteTemporalNode();
		return false;
	}

	if (CachedMesh.getBoundingBox().IsEmpty())
	{
		CachedMesh.buildBoundingBox();
	}

	FillDatasmithMeshFromMaxMesh( DSMesh, CachedMesh, ExportMeshArgs.Node, ExportMeshArgs.bForceSingleMat, SupportedChannels, ExportMeshArgs.ExportName, ExportMeshArgs.Pivot );

	CachedMesh.FreeAll();
	DeleteTemporalNode();

	return true;
}

TSharedPtr< IDatasmithMeshElement > FDatasmithMaxMeshExporter::ExportDummyMesh(const TCHAR* ExportPath, const TCHAR* ExportName)
{
	TSharedPtr< IDatasmithMeshElement > MeshElement;
	FDatasmithMesh DSMesh;

	FDatasmithMeshExporter DatasmithMeshExporter;
	MeshElement = DatasmithMeshExporter.ExportToUObject(ExportPath, ExportName, DSMesh, nullptr, FDatasmithExportOptions::LightmapUV);

	return MeshElement;
}

void FDatasmithMaxMeshExporter::FillDatasmithMeshFromMaxMesh(FDatasmithMesh& DatasmithMesh, Mesh& MaxMesh, INode* ExportedNode, bool bForceSingleMat, TSet<uint16>& SupportedChannels,
	const TCHAR* MeshName, FTransform Pivot)
{
	const int NumFaces = MaxMesh.getNumFaces();
	const int NumVerts = MaxMesh.getNumVerts();

	DatasmithMesh.SetVerticesCount(NumVerts);
	DatasmithMesh.SetFacesCount(NumFaces);

	// Vertices
	for (int i = 0; i < NumVerts; i++)
	{
		Point3 Point = MaxMesh.getVert(i);

		FVector Vertex = Unit3ToFVector(Point);
		Vertex = Pivot.TransformPosition( Vertex ); // Bake object-offset in the mesh data when possible

		DatasmithMesh.SetVertex(i, Vertex.X, Vertex.Y, Vertex.Z);
	}

	// Vertex Colors
	if (MaxMesh.curVCChan == 0 && MaxMesh.numCVerts > 0)
	{
		// Default vertex color channel
		for (int32 i = 0; i < NumFaces; i++)
		{
			TVFace& Face = MaxMesh.vcFace[i];
			DatasmithMesh.SetVertexColor(i * 3, ConvertPoint3ToColor(MaxMesh.vertCol[Face.t[0]]));
			DatasmithMesh.SetVertexColor(i * 3 + 1, ConvertPoint3ToColor(MaxMesh.vertCol[Face.t[1]]));
			DatasmithMesh.SetVertexColor(i * 3 + 2, ConvertPoint3ToColor(MaxMesh.vertCol[Face.t[2]]));
		}
	}

	// UVs
	FUVChannelsMap UVChannelsMap;
	TMap<uint32, int32> HashToChannel;
	bool bIsFirstUVChannelValid = true;

	for ( int32 i = 1; i <= MaxMesh.getNumMaps(); ++i )
	{
		if ( MaxMesh.mapSupport(i) == BOOL(true) && MaxMesh.getNumMapVerts(i) > 0)
		{
			DatasmithMesh.AddUVChannel();
			const int32 UVChannelIndex = DatasmithMesh.GetUVChannelsCount() - 1;
			const int32 UVsCount = MaxMesh.getNumMapVerts(i);

			DatasmithMesh.SetUVCount( UVChannelIndex, UVsCount );

			UVVert* Vertex = MaxMesh.mapVerts( i );

			for ( int32 j = 0; j < UVsCount; ++j )
			{
				const UVVert& MaxUV = Vertex[j];
				DatasmithMesh.SetUV( UVChannelIndex, j, MaxUV.x, 1.f - MaxUV.y );
			}

			TVFace* Faces = MaxMesh.mapFaces(i);
			for (int32 j = 0; j < MaxMesh.getNumFaces(); ++j)
			{
				DatasmithMesh.SetFaceUV(j, UVChannelIndex, Faces[j].t[0], Faces[j].t[1], Faces[j].t[2]);
			}

			if (UVChannelIndex == 0)
			{
				//Verifying that the UVs are properly unfolded, which is required to calculate the tangent in unreal.
				bIsFirstUVChannelValid = FDatasmithMeshUtils::IsUVChannelValid(DatasmithMesh, UVChannelIndex);
			}

			uint32 Hash = DatasmithMesh.GetHashForUVChannel( UVChannelIndex );
			int32* PointerToChannel = HashToChannel.Find( Hash );

			if ( PointerToChannel )
			{
				// Remove the channel because there is another one that is identical
				DatasmithMesh.RemoveUVChannel();

				// Map the user-specified UV Channel (in 3dsmax) to the actual UV channel that will be exported to Unreal
				UVChannelsMap.Add( i - 1, *PointerToChannel );
			}
			else
			{
				// Map the user-specified UV Channel (in 3dsmax) to the actual UV channel that will be exported to Unreal
				UVChannelsMap.Add( i - 1, UVChannelIndex );
				HashToChannel.Add( Hash, UVChannelIndex );
			}
		}
	}

	if (!bIsFirstUVChannelValid)
	{
		DatasmithMaxLogger::Get().AddGeneralError(*FString::Printf(TEXT("%s's UV channel #1 contains degenerated triangles, this can cause issues in Unreal. It is recommended to properly unfold and flatten exported UV data.")
			, static_cast<const TCHAR*>(ExportedNode->GetName())));
	}

	if (MeshName != nullptr)
	{
		MeshNamesToUVChannels.Add(MeshName, MoveTemp(UVChannelsMap));
	}

	// Faces
	for (int i = 0; i < NumFaces; i++)
	{
		// Create polygons. Assign texture and texture UV indices.
		// all faces of the cube have the same texture

		Face& MaxFace = MaxMesh.faces[i];
		int MaterialId = bForceSingleMat ? 0 : MaxFace.getMatID();

		SupportedChannels.Add(MaterialId);

		//Max's channel UI is not zero-based, so we register an incremented ChannelID for better visual consistency after importing in Unreal.
		DatasmithMesh.SetFace(i, MaxFace.getVert(0), MaxFace.getVert(1), MaxFace.getVert(2), MaterialId + 1);
		DatasmithMesh.SetFaceSmoothingMask(i, (uint32)MaxFace.getSmGroup() );
	}

	//Normals
	MeshNormalSpec* Normal = GetSpecifiedNormalsFromMesh(&MaxMesh);
	if (Normal)
	{
		Matrix3 RotationMatrix;
		RotationMatrix.IdentityMatrix();
		Quat ObjectOffsetRotation = ExportedNode->GetObjOffsetRot();
		RotateMatrix(RotationMatrix, ObjectOffsetRotation);

		Point3 Point;

		for (int i = 0; i < NumFaces; i++)
		{
			Point = Normal->GetNormal(i, 0).Normalize() * RotationMatrix;
			FVector NormalVector = Point3ToFVector(Point);
			DatasmithMesh.SetNormal(i * 3, NormalVector.X, NormalVector.Y, NormalVector.Z);

			Point = Normal->GetNormal(i, 1).Normalize() * RotationMatrix;
			NormalVector = Point3ToFVector(Point);
			DatasmithMesh.SetNormal(i * 3 + 1, NormalVector.X, NormalVector.Y, NormalVector.Z);

			Point = Normal->GetNormal(i, 2).Normalize() * RotationMatrix;
			NormalVector = Point3ToFVector(Point);
			DatasmithMesh.SetNormal(i * 3 + 2, NormalVector.X, NormalVector.Y, NormalVector.Z);
		}
	}
}


bool FDatasmithMaxMeshExporter::FillDatasmithMeshFromBoundingBox(FDatasmithMesh& DatasmithMesh, INode* ExportNode, FTransform Pivot) const
{
	if (!ExportNode)
	{
		return false;
	}

	BOOL bNeedsDelete;

	Mesh* MaxMesh = GetMeshFromRenderMesh(ExportNode, bNeedsDelete, GetCOREInterface()->GetTime());

	if (!MaxMesh)
	{
		return false;
	}

	if (MaxMesh->getBoundingBox().IsEmpty())
	{
		MaxMesh->buildBoundingBox();
	}

	Box3 BoundingBox = MaxMesh->getBoundingBox();

	DatasmithMesh.SetVerticesCount(8);
	DatasmithMesh.SetFacesCount(12);

	for (int32 i = 0; i < 8; i++)
	{
		FVector Vertex = Unit3ToFVector(BoundingBox[i]);
		Vertex = Pivot.TransformPosition(Vertex);
		DatasmithMesh.SetVertex(i, Vertex.X, Vertex.Y, Vertex.Z);
	}

	// Make a cube from the vertices

	Matrix3 RotationMatrix;
	RotationMatrix.IdentityMatrix();
	Quat ObjectOffsetRotation = ExportNode->GetObjOffsetRot();
	RotateMatrix(RotationMatrix, ObjectOffsetRotation);
	Point3 Point;
	FVector NormalVector;

	// Points toward negative z (without a transform and in max coordinates)
	DatasmithMesh.SetFace(0, 1, 0, 2);
	DatasmithMesh.SetFace(1, 1, 2, 3);
	Point = Point3(0.f, 0.f, -1.f) * RotationMatrix;
	NormalVector = Point3ToFVector(Point);
	SetNormalForAFace(DatasmithMesh, 0, NormalVector);
	SetNormalForAFace(DatasmithMesh, 1, NormalVector);

	// Points toward positive y (without a transform and in max coordinates)
	DatasmithMesh.SetFace(2, 3, 2, 7);
	DatasmithMesh.SetFace(3, 2, 6, 7);
	Point = Point3(0.f, 1.f, 0.f) * RotationMatrix;
	NormalVector = Point3ToFVector(Point);
	SetNormalForAFace(DatasmithMesh, 2, NormalVector);
	SetNormalForAFace(DatasmithMesh, 3, NormalVector);
	
	// Points toward positive x (without a transform and in max coordinates)
	DatasmithMesh.SetFace(4, 1, 3, 7);
	DatasmithMesh.SetFace(5, 5, 1, 7);
	Point = Point3(1.f, 0.f, 0.f) * RotationMatrix;
	NormalVector = Point3ToFVector(Point);
	SetNormalForAFace(DatasmithMesh, 4, NormalVector);
	SetNormalForAFace(DatasmithMesh, 5, NormalVector);

	// Points toward negative x (without a transform and in max coordinates)
	DatasmithMesh.SetFace(6, 2, 0, 6);
	DatasmithMesh.SetFace(7, 0, 4, 6);
	Point = Point3(-1.f, 0.f, 0.f) * RotationMatrix;
	NormalVector = Point3ToFVector(Point);
	SetNormalForAFace(DatasmithMesh, 6, NormalVector);
	SetNormalForAFace(DatasmithMesh, 7, NormalVector);

	// Points toward negative y (without a transform and in max coordinates)
	DatasmithMesh.SetFace(8, 0, 1, 5);
	DatasmithMesh.SetFace(9, 4, 0, 5);
	Point = Point3(0.f, -1.f, 0.f) * RotationMatrix;
	NormalVector = Point3ToFVector(Point);
	SetNormalForAFace(DatasmithMesh, 8, NormalVector);
	SetNormalForAFace(DatasmithMesh, 9, NormalVector);

	// Points toward positive z (without a transform and in max coordinates)
	DatasmithMesh.SetFace(10, 4, 5, 6);
	DatasmithMesh.SetFace(11, 6, 5, 7);
	Point = Point3(0.f, 0.f, 1.f) * RotationMatrix;
	NormalVector = Point3ToFVector(Point);
	SetNormalForAFace(DatasmithMesh, 10, NormalVector);
	SetNormalForAFace(DatasmithMesh, 11, NormalVector);

	if (bNeedsDelete)
	{
		MaxMesh->DeleteThis();
	}

	return true;
}

const FDatasmithMaxMeshExporter::FUVChannelsMap& FDatasmithMaxMeshExporter::GetUVChannelsMapForMesh(const TCHAR* MeshName) const
{
	static FUVChannelsMap DefaultUVChannels;
	// Just return default map if the mesh wasn't found, no mapping will be done
	if (const FUVChannelsMap* UVChannels = MeshNamesToUVChannels.Find(MeshName))
	{
		return *UVChannels;
	}
	return DefaultUVChannels;
}

#include "Windows/HideWindowsPlatformTypes.h"
