// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxProgressManager.h"
#include "DatasmithMaxExporterDefines.h"
#include "DatasmithMaxLogger.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
#include "Max.h"
MAX_INCLUDES_END

DWORD WINAPI fn(LPVOID Arg)
{
	return (0);
}

FDatasmithMaxProgressManager::FDatasmithMaxProgressManager()
	: ProgressStart(0.f)
	, ProgressEnd(100.f)
	, ProgressBarCounter(1)
{
	LPVOID Arg = 0;
	FString emptyMsg("");

	GetCOREInterface()->ProgressStart(*emptyMsg, TRUE, fn, Arg);
}

FDatasmithMaxProgressManager::~FDatasmithMaxProgressManager()
{
	while (ProgressBarCounter--)
	{
		GetCOREInterface()->ProgressEnd();
	}
}

void FDatasmithMaxProgressManager::SetMainMessage(const TCHAR* InProgressMessage)
{
	LPVOID Arg = 0;

	//Calling ProgressStart is the only way to change the main progress bar message without doing a whole UI (possibly a few seconds long) refresh.
	GetCOREInterface()->ProgressStart(InProgressMessage, TRUE, fn, Arg);
	++ProgressBarCounter;
}

void FDatasmithMaxProgressManager::ProgressEvent(float InProgressRatio, const TCHAR* InProgressString)
{
	FString Msg;
	int Progress = (int)(ProgressStart + (ProgressEnd - ProgressStart) * InProgressRatio);
	Msg = TEXT("(") + FString::FromInt(Progress) + TEXT("%) ") + InProgressString;
	GetCOREInterface()->ProgressUpdate(Progress, TRUE, *Msg.Left(255)); // Max doesn't allow for more than 256 characters (undocumented, but crashes)
}

#include "Windows/HideWindowsPlatformTypes.h"