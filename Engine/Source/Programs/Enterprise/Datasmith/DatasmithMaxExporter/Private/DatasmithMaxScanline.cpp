// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithMaxWriter.h"

#include "DatasmithMaxDirectLink.h"

#include "DatasmithSceneFactory.h"
#include "DatasmithSceneExporter.h"
#include "DatasmithMaxSceneHelper.h"
#include "DatasmithMaxLogger.h"
#include "DatasmithExportOptions.h"
#include "DatasmithMaxSceneExporter.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithMaxHelper.h"

#include "Misc/Paths.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
#include "bitmap.h"
#include "pbbitmap.h"
#include "plugapi.h"
MAX_INCLUDES_END 
#include "Windows/HideWindowsPlatformTypes.h"


struct CropParameters
{
	bool bApplyCropping = false;
	float HorizontalStart = 0.0f;
	float VerticalStart = 0.0f;
	float Width = 1.0f;
	float Height = 1.0f;
};



void GetBitmapCroppingInfo(BitmapTex* InBitmapTex, CropParameters &Cropping)
{
	bool bCropEnabled = false;
	int CropPlace = 0;
	float ClipU = 0.0f;
	float ClipV = 0.0f;
	float ClipW = 1.0f;
	float ClipH = 1.0f;

	int NumParamBlocks = InBitmapTex->NumParamBlocks();
	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InBitmapTex->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("apply")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0)
				{
					bCropEnabled = true;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("cropPlace")) == 0)
			{
				CropPlace = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clipu")) == 0)
			{
				ClipU = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clipv")) == 0)
			{
				ClipV = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clipw")) == 0)
			{
				ClipW = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("cliph")) == 0)
			{
				ClipH = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	if (bCropEnabled == true && CropPlace == 0 && (ClipU != 0 || ClipV != 0 || ClipW != 0 || ClipH != 0))
	{
		Cropping.bApplyCropping = true;
		Cropping.HorizontalStart = ClipU;
		Cropping.VerticalStart = ClipV;
		Cropping.Width = ClipW;
		Cropping.Height = ClipH;
	}
	else
	{
		Cropping.bApplyCropping = false;
	}
}

FString FDatasmithMaxMatWriter::GetActualBitmapPath(BitmapTex* InBitmapTex)
{
	if ( !InBitmapTex )
	{
		return FString();
	}

	FString ActualFilePath = FDatasmithMaxSceneExporter::GetActualPath(InBitmapTex->GetMap().GetFullFilePath().data());	

	// if path is empty 3dsmax was not able to resolve the file path but we'll get it in other way just to force logging it
	if (ActualFilePath.IsEmpty())
	{
		ActualFilePath = InBitmapTex->GetMap().GetFileName();
	}

	if (FPaths::FileExists(ActualFilePath) == false)
	{
		return ActualFilePath;
	}

	CropParameters Cropping;
	GetBitmapCroppingInfo(InBitmapTex, Cropping);

	if ( Cropping.bApplyCropping )
	{
		FString Base = FPaths::GetBaseFilename(ActualFilePath);
		FString NewBasename = Base + FString("_3dsmaxcrop");
		FString NewFilename = FPaths::GetPath(ActualFilePath) + FString("\\") + NewBasename;
		NewFilename = NewFilename + FString(".") + FPaths::GetExtension(ActualFilePath);
		ActualFilePath = NewFilename;
	}

	return ActualFilePath;
}

FString FDatasmithMaxMatWriter::GetActualBitmapPath(BitmapInfo* InBitmapInfo)
{
	if (!InBitmapInfo)
	{
		return FString();
	}

	FString ActualBitmapPath = FDatasmithMaxSceneExporter::GetActualPath(InBitmapInfo->GetPathEx().GetCStr());
	
	if (ActualBitmapPath.IsEmpty())
	{
		ActualBitmapPath = InBitmapInfo->Name();
	}

	return ActualBitmapPath;
}

template<typename T>
FString GetActualBitmapNameImpl(T* InBitmap)
{
	FString ActualBitmapPath = FDatasmithMaxMatWriter::GetActualBitmapPath(InBitmap);

	if (ActualBitmapPath.IsEmpty())
	{
		return FString();
	}

	float Gamma = FDatasmithMaxMatHelper::GetBitmapGamma(InBitmap);

	// note: call SanitizeObjectName for stringified float - converted floating point can contain invalid character too(e.g. when decimal symbol is invalid in the locale)
	FString ActualBitmapName = FDatasmithUtils::SanitizeObjectName(FPaths::GetBaseFilename(ActualBitmapPath) + TEXT("_") + FDatasmithUtils::SanitizeObjectName(FString::SanitizeFloat(Gamma)) + FDatasmithMaxMatWriter::TextureSuffix);

	return ActualBitmapName;
}

FString FDatasmithMaxMatWriter::GetActualBitmapName(BitmapTex* InBitmapTex)
{
	return GetActualBitmapNameImpl(InBitmapTex);
}

FString FDatasmithMaxMatWriter::GetActualBitmapName(BitmapInfo* InBitmapInfo)
{
	return GetActualBitmapNameImpl(InBitmapInfo);
}

FString FDatasmithMaxMatWriter::CropBitmap(BitmapTex* InBitmapTex)
{
	const FString ActualFilePath = GetActualBitmapPath(InBitmapTex);

	Bitmap* ActualBitmap = InBitmapTex->GetBitmap(GetCOREInterface()->GetTime());

	// No need to check on cropping info if actual bitmap is invalid 
	if (ActualBitmap != nullptr)
	{
		CropParameters Cropping;
		GetBitmapCroppingInfo(InBitmapTex, Cropping);

		if (Cropping.bApplyCropping == true)
		{
			BitmapInfo BitmapInformation;

			// 3dsmax v2017 and newer allows the developers to read the current bitmap info 
			// so our output will be even more similar to the input
#ifdef MAX_RELEASE_R19 
			BitmapInformation = ActualBitmap->GetBitmapInfo();
			switch (ActualBitmap->GetBitmapInfo().Type())
			{
			case BMM_TRUE_24:
				BitmapInformation.SetType(BMM_TRUE_32);
				break;
			case BMM_TRUE_48:
				BitmapInformation.SetType(BMM_TRUE_64);
				break;
			case BMM_YUV_422:
				BitmapInformation.SetType(BMM_TRUE_16);
				break;
			case BMM_BMP_4:
				BitmapInformation.SetType(BMM_TRUE_16);
				break;
			case BMM_PAD_24:
				BitmapInformation.SetType(BMM_TRUE_32);
				break;
			default:
				BitmapInformation.SetType(ActualBitmap->GetBitmapInfo().Type());
				break;
			}
#else
			BitmapInformation.SetGamma(ActualBitmap->Gamma());
			if (ActualBitmap->IsHighDynamicRange())
			{
				BitmapInformation.SetType(BMM_FLOAT_RGBA_32);
			}
			else
			{
				BitmapInformation.SetType(BMM_TRUE_32);
			}
#endif


			BitmapInformation.SetWidth(ActualBitmap->Width() * Cropping.Width);
			BitmapInformation.SetHeight(ActualBitmap->Height() * Cropping.Height);
			BitmapInformation.SetName(*ActualFilePath);

			if ( Bitmap* NewBitmap = TheManager->Create(&BitmapInformation) )
			{
				BMM_Color_fl *Line;
				for (int y = 0; y < NewBitmap->Height(); y++)
				{
					Line = (BMM_Color_fl *)calloc(NewBitmap->Width(), sizeof(BMM_Color_fl));
					int OffsetX = ActualBitmap->Width() * Cropping.HorizontalStart;
					int OffsetY = y + ActualBitmap->Height() * Cropping.VerticalStart;
					ActualBitmap->GetLinearPixels(OffsetX, OffsetY, NewBitmap->Width(), Line);
					NewBitmap->PutPixels(0, y, NewBitmap->Width(), Line);
					free(Line);
				}

				NewBitmap->OpenOutput(&BitmapInformation);
				NewBitmap->Write(&BitmapInformation);
				NewBitmap->Close(&BitmapInformation);
				NewBitmap->DeleteThis();
			}
			else
			{
				FString Error = FString( TEXT("Unable to crop texture ") ) + InBitmapTex->GetName();
				DatasmithMaxLogger::Get().AddTextureError(*Error);
			}
		}
	}

	return ActualFilePath;
}

FDatasmithTextureSampler SetupTextureSampler(Texmap* Texture, TimeValue CurrentTime, float Multiplier, bool bForceInvert)
{
	StdUVGen* UV = nullptr;
	if (Texture)
	{
		UVGen* BaseUV = Texture->GetTheUVGen();
		if (BaseUV && BaseUV->IsStdUVGen())
		{
			UV = static_cast<StdUVGen*>(BaseUV);
		}
	}

	if (!UV)
	{
		return 	FDatasmithTextureSampler(0, 1, 1, 0, 0, 0, 1, bForceInvert, 0, false, 0, 0);
	}

	bool bIsUsingRealWorldScale = !!UV->GetUseRealWorldScale();
	float UScale = UV->GetUScl(CurrentTime);
	float VScale = UV->GetVScl(CurrentTime);
	bool bCroppedTexture = false;

	// Determine the sub-class of Texmap
	Class_ID ClassID = Texture->ClassID();
	bool bIsBitmapTex = ClassID.PartA() == BMTEX_CLASS_ID;
	if (!bIsBitmapTex)
	{
		// Only procedural textures requires some processing of the UV scale values
		if (!bIsUsingRealWorldScale)
		{
			// If RealWorldScale is not used, the UV values smaller than 1 are treated as 1
			UScale = FMath::Max(UScale, 1.0f);
			VScale = FMath::Max(VScale, 1.0f);

			// When a scale is greater than 1, the tiling is baked into the texture (does not apply when using RealWorldScale because the scales are inverted)
			if (UScale > 1.0f || VScale > 1.0f)
			{
				UScale = 1.0f;
				VScale = 1.0f;
			}
		}
		else
		{
			// Determine if texture has some fraction of the repeating pattern
			// If so, UV cropping will be needed to have seamless tiling
			bool bHasFractionalUV = false;
			if (UScale != VScale)
			{
				// Check how many times the smallest value fits within the greater value and look if there's a fractional part
				float Numerator = FMath::Max(UScale, VScale);
				float Denominator = FMath::Min(UScale, VScale);

				float Quotient = Numerator / Denominator;
				bHasFractionalUV = FMath::Frac(Quotient) > THRESH_UVS_ARE_SAME;
			}
			bCroppedTexture = bHasFractionalUV;
		}
	}

	float IntegerPart;
	float UOffset = (UV->GetUOffs(CurrentTime)) * UScale;
	if (!bIsUsingRealWorldScale)
	{
		UOffset += (-0.5f + 0.5f * UScale);
	}

	UOffset = 1.0f - UOffset;
	UOffset = FMath::Modf(UOffset, &IntegerPart);

	float VOffset = (UV->GetVOffs(CurrentTime)) * VScale;
	if (!bIsUsingRealWorldScale)
	{
		VOffset += (0.5f - 0.5f * VScale);
	}
	else
	{
		VOffset -= VScale;
	}

	VOffset = FMath::Modf(VOffset, &IntegerPart);

	int TextureTilingDirection = UV->GetTextureTiling();
	int MirrorU = 0;
	int MirrorV = 0;
	if (TextureTilingDirection & U_MIRROR)
	{
		MirrorU = 2;
	}
	if (TextureTilingDirection & V_MIRROR)
	{
		MirrorV = 2;
	}

	int Slot = 0;

	int CoordinateIndex = 0;
	if (UV->GetSlotType() == MAPSLOT_TEXTURE)
	{
		if (UV->GetUVWSource() == UVWSRC_EXPLICIT)
		{
			CoordinateIndex = Texture->GetMapChannel() - 1;
		}
	}

	return FDatasmithTextureSampler(CoordinateIndex, UScale, VScale, UOffset, VOffset, -(UV->GetWAng(CurrentTime)) / (2 * PI), Multiplier, bForceInvert, Slot, bCroppedTexture, MirrorU, MirrorV);
}

FString FDatasmithMaxMatWriter::DumpBitmap(TSharedPtr<IDatasmithCompositeTexture>& CompTex, BitmapTex* InBitmapTex, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale)
{
	CropBitmap(InBitmapTex); // Crop if necessary

	int CurrentFrame = GetCOREInterface()->GetTime() / GetTicksPerFrame();
	TimeValue CurrentTime = TimeValue(TIME_TICKSPERSEC * ((float)CurrentFrame / GetFrameRate()));

	bool bInvertTexture = false;
	if (InBitmapTex->GetTexout()->GetInvert() == (BOOL)true)
	{
		bInvertTexture = true;
	}

	if (bInvertTexture)
	{
		bForceInvert = !bForceInvert;
	}

	float Multiplier = InBitmapTex->GetTexout()->GetOutputLevel(CurrentTime);

	FDatasmithTextureSampler TextureSampler(SetupTextureSampler(InBitmapTex, CurrentTime, Multiplier, bForceInvert));

	if(bIsGrayscale)
	{
		if (InBitmapTex->GetAlphaAsMono(true))
		{
			TextureSampler.OutputChannel = 4;
		}
	}
	else
	{
		if (InBitmapTex->GetAlphaAsRGB(true))
		{
			TextureSampler.OutputChannel = 4;
		}
	}

	const FString ActualBitmapName = GetActualBitmapName(InBitmapTex);
	CompTex->AddSurface( *ActualBitmapName, TextureSampler );
	
	return ActualBitmapName;
}

FString FDatasmithMaxMatWriter::DumpAutodeskBitmap(TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale)
{
	DatasmithMaxTexmapParser::FAutodeskBitmapParameters AutodeskBitmapParameters = DatasmithMaxTexmapParser::ParseAutodeskBitmap(InTexmap);
	if (AutodeskBitmapParameters.SourceFile == nullptr) 
	{
		return FString();
	}

	FScopedBitMapPtr ActualBitmap(AutodeskBitmapParameters.SourceFile->bi, AutodeskBitmapParameters.SourceFile->bm);
	if (ActualBitmap.Map == nullptr)
	{
		return FString();
	}

	bool bInvertTexture = false;
	if (AutodeskBitmapParameters.bInvertImage)
	{
		bInvertTexture = true;
	}

	if (bInvertTexture)
	{
		bForceInvert = !bForceInvert;
	}

	int CurrentFrame = GetCOREInterface()->GetTime() / GetTicksPerFrame();
	TimeValue CurrentTime = TimeValue(TIME_TICKSPERSEC * ((float)CurrentFrame / GetFrameRate()));
	FDatasmithTextureSampler TextureSampler(SetupTextureSampler(InTexmap, CurrentTime, 1, bForceInvert));

	if (!bIsGrayscale && ActualBitmap.Map->HasAlpha())
	{
		TextureSampler.OutputChannel = 4;
	}

	const FString ActualBitmapName = GetActualBitmapName(&ActualBitmap.MapInfo);
	CompTex->AddSurface(*ActualBitmapName, TextureSampler);

	return ActualBitmapName;
}


enum class EScanlineMaterialMaps
{
	Ambient,
	Diffuse,
	SpecularColor,
	SpecularLevel,
	Glossiness,
	SelfIllumination,
	Opacity,
	FilterColor,
	Bump,
	Reflection,
	Refraction,
};

void FDatasmithMaxMatWriter::ExportStandardMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	TSharedPtr< IDatasmithShaderElement > MaterialShader = FDatasmithSceneFactory::CreateShader((TCHAR*)Material->GetName().data());

	int NumParamBlocks = Material->NumParamBlocks();

	bool bDiffuseTexEnable = true;
	bool bReflectanceTexEnable = true;
	bool bMaskTexEnable = true;
	bool bGlossyTexEnable = true;
	bool bBumpTexEnable = true;

	float DiffuseTexAmount = 0.f;
	float ReflectanceTexAmount = 0.f;
	float MaskTexAmount = 0.f;
	float GlossyTexAmount = 0.f;
	float BumpTexAmount = 0.f;

	bool bUseSelfIllumColor = true;

	Texmap* SelfIllumTex = nullptr;
	float SpecularLevel = 0.f;

	float Glossiness = 0.1f;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("maps")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Diffuse) == nullptr)
				{
					bDiffuseTexEnable = false;
				}

				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SpecularColor) == nullptr)
				{
					bReflectanceTexEnable = false;
				}

				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Glossiness) == nullptr)
				{
					bGlossyTexEnable = false;
				}

				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Opacity) == nullptr)
				{
					bMaskTexEnable = false;
				}

				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Bump) == nullptr)
				{
					bBumpTexEnable = false;
				}
			}

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapEnables")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Diffuse) == 0)
				{
					bDiffuseTexEnable = false;
				}

				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SpecularColor) == 0)
				{
					bReflectanceTexEnable = false;
				}

				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Glossiness) == 0)
				{
					bGlossyTexEnable = false;
				}

				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Opacity) == 0)
				{
					bMaskTexEnable = false;
				}

				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Bump) == 0)
				{
					bBumpTexEnable = false;
				}
			}

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapAmounts")) == 0)
			{
				DiffuseTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Diffuse);
				ReflectanceTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SpecularColor);
				GlossyTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Glossiness);
				MaskTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Opacity);
				BumpTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Bump);
			}

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("SpecularLevel")) == 0)
			{
				SpecularLevel = (float)ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) / 200.0f;
			}
			
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("SpecularLevel")) == 0)
			{
				Glossiness = (float)ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) / 100.0f;
			}
			
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bUseSelfIllumColor")) == 0)
			{
				bUseSelfIllumColor = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
			}
			
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("selfillumMap")) == 0)
			{
				SelfIllumTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 0);
			}

		}
		ParamBlock2->ReleaseDesc();
	}

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapAmounts")) == 0)
			{
				if (bBumpTexEnable == true)
				{
					MaterialShader->SetBumpAmount( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime(), 8) );
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("ior")) == 0)
			{
				MaterialShader->SetIOR( 0.0 ); // ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				MaterialShader->SetIORRefra( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("twoSided")) == 0)
			{
				MaterialShader->SetTwoSided( ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0 );
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("maps")) == 0)
			{
				BMM_Color_fl Color = (BMM_Color_fl)Material->GetDiffuse();
				if (bDiffuseTexEnable == true && DiffuseTexAmount > 0)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Diffuse);
					if (DiffuseTexAmount == 1)
					{
						DumpTexture(DatasmithScene, MaterialShader->GetDiffuseComp(), LocalTex, DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
					}
					else
					{
						DumpWeightedTexture(DatasmithScene, MaterialShader->GetDiffuseComp(), LocalTex, Color, DiffuseTexAmount, DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
					}
				}
				else
				{
					MaterialShader->GetDiffuseComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
				}

				float RValue = Material->GetShinStr();
				Color = BMM_Color_fl(0.0f, 0.0f, 0.0f, 0.0f);
				if (RValue > 0.05f)
				{
					Color = (BMM_Color_fl)Material->GetSpecular();
				}
				if (bReflectanceTexEnable == true && ReflectanceTexAmount > 0)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SpecularColor);
					if (ReflectanceTexAmount == 1)
					{
						DumpTexture(DatasmithScene, MaterialShader->GetRefleComp(), LocalTex, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
					}
					else
					{
						DumpWeightedTexture(DatasmithScene, MaterialShader->GetRefleComp(), LocalTex, Color, ReflectanceTexAmount, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
					}
				}
				else
				{
					if (RValue > 0.05f)
					{
						MaterialShader->GetRefleComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
					}
				}

				float Rough = 1.0f - Glossiness;
				if (bGlossyTexEnable == true && GlossyTexAmount > 0)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Glossiness);
					if (GlossyTexAmount == 1)
					{
						DumpTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, true, true);
					}
					else
					{
						Color = BMM_Color_fl(Rough, Rough, Rough, Rough);
						DumpWeightedTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, Color, GlossyTexAmount, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, true, true);
					}
				}
				else
				{
					MaterialShader->GetRoughnessComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(Rough, TEXT("Roughness")) );
				}

				if (bMaskTexEnable == true && MaskTexAmount > 0)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Opacity);
					DumpTexture(DatasmithScene, MaterialShader->GetMaskComp(), LocalTex, DATASMITH_CLIPTEXNAME, DATASMITH_CLIPTEXNAME, false, true);
				}

				if (bBumpTexEnable == true && BumpTexAmount > 0)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Bump);
					if (FDatasmithMaxMatHelper::GetTextureClass(LocalTex) == EDSBitmapType::NormalMap)
					{
						DumpTexture(DatasmithScene, MaterialShader->GetNormalComp(), LocalTex, DATASMITH_NORMALTEXNAME, DATASMITH_NORMALTEXNAME, false, false);
						DumpTexture(DatasmithScene, MaterialShader->GetBumpComp(), LocalTex, DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
					}
					else
					{
						DumpTexture(DatasmithScene, MaterialShader->GetBumpComp(), LocalTex, DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
					}
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("selfillumMap")) == 0 && bUseSelfIllumColor == true && SelfIllumTex != nullptr)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 0);
				DumpTexture(DatasmithScene, MaterialShader->GetEmitComp(), LocalTex, DATASMITH_EMITTEXNAME, DATASMITH_EMITTEXNAME, false, false);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("selfIllumColor")) == 0 && bUseSelfIllumColor == true && SelfIllumTex == nullptr)
			{
				BMM_Color_fl Color = (BMM_Color_fl)Material->GetSelfIllumColor();
				MaterialShader->GetEmitComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
				MaterialShader->SetEmitPower(100.0);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacity")) == 0 && bMaskTexEnable == false)
			{
				float ColorVal = 1.0f - ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (ColorVal != 0)
				{
					BMM_Color_fl Color(ColorVal, ColorVal, ColorVal, 1.0);
					MaterialShader->GetTransComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
					MaterialShader->SetEmitPower(100.0);
				}
			}
		}
		ParamBlock2->ReleaseDesc();
	}
	MaterialElement->AddShader(MaterialShader);
}

void FDatasmithMaxMatWriter::ExportBlendMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	FDatasmithMaxMatExport::WriteXMLMaterial(DatasmithScene, MaterialElement, Material->GetSubMtl(0));
	FDatasmithMaxMatExport::WriteXMLMaterial(DatasmithScene, MaterialElement, Material->GetSubMtl(1));

	int NumParamBlocks = Material->NumParamBlocks();

	bool bMaskEnabled = true;
	Texmap* Mask = nullptr;
	float MixAmount = 0.5f;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Mask")) == 0)
			{
				Mask = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bMaskEnabled")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bMaskEnabled = false;
				}
			}

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("MixAmount")) == 0)
			{
				MixAmount = (float)ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}
	
	if (MaterialElement->GetShadersCount() != 0)
	{
		TSharedPtr< IDatasmithShaderElement >& Shader = MaterialElement->GetShader( MaterialElement->GetShadersCount() - 1 );
		Shader->SetBlendMode(EDatasmithBlendMode::Alpha);
		Shader->SetIsStackedLayer(true);

		if (Mask && bMaskEnabled)
		{
			DumpTexture(DatasmithScene, Shader->GetWeightComp(), Mask, DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
		}
		else
		{
			Shader->GetWeightComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(MixAmount, TEXT("MixAmount")) );
		}
	}
	else 
	{
		DatasmithMaxLogger::Get().AddGeneralError(*FString::Printf(TEXT("The material %s won't be exported."),  MaterialElement->GetLabel()));
	}
}

// output order:
// slot0: input texture color
// slot1: color filter
// value list 1: hue sat lift/Brightness gamma/Contrast tintamount
// value list 2: RewireR RewireG RewireB
FString FDatasmithMaxMatWriter::DumpColorCorrect(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale)
{
	DatasmithMaxTexmapParser::FColorCorrectionParameters ColorCorrectionParameters = DatasmithMaxTexmapParser::ParseColorCorrection(InTexmap);

	FString ActualPrefix = FString(Prefix) + FString(TEXT("comp"));
	if (ColorCorrectionParameters.bAdvancedLightnessMode == false)
	{
		CompTex->SetMode( EDatasmithCompMode::ColorCorrectContrast );
	}
	else
	{
		CompTex->SetMode(EDatasmithCompMode::ColorCorrectGamma );
	}

	FString Result;
	if (ColorCorrectionParameters.TextureSlot1 != nullptr)
	{
		Result = DumpTexture(DatasmithScene, CompTex, ColorCorrectionParameters.TextureSlot1, DATASMITH_TEXTURENAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
	}
	else
	{
		CompTex->AddSurface(ColorCorrectionParameters.Color1);
	}

	CompTex->AddSurface(ColorCorrectionParameters.Tint);

	CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(ColorCorrectionParameters.HueShift, TEXT("HueShift")) );
	CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(ColorCorrectionParameters.Saturation, TEXT("Saturation")) );
	if (ColorCorrectionParameters.bAdvancedLightnessMode == false)
	{
		CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(ColorCorrectionParameters.Brightness, TEXT("Brightness")) );
		CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(ColorCorrectionParameters.Contrast, TEXT("Contrast")) );
	}
	else
	{
		CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(ColorCorrectionParameters.LiftRGB, TEXT("LiftRGB")) );
		CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(ColorCorrectionParameters.GammaRGB, TEXT("GammaRgb")) );
	}
	CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(ColorCorrectionParameters.TintStrength, TEXT("TintStrength")) );

	CompTex->AddParamVal2( IDatasmithCompositeTexture::ParamVal((float)ColorCorrectionParameters.RewireR, TEXT("RewireR")) );
	CompTex->AddParamVal2( IDatasmithCompositeTexture::ParamVal((float)ColorCorrectionParameters.RewireG, TEXT("RewireG")) );
	CompTex->AddParamVal2( IDatasmithCompositeTexture::ParamVal((float)ColorCorrectionParameters.RewireB, TEXT("RewireB")) );

	return Result;
}

FString FDatasmithMaxMatWriter::DumpMix(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale)
{
	FString ActualPrefix = FString(Prefix) + FString(TEXT("comp"));
	CompTex->SetMode( EDatasmithCompMode::Mix );

	int NumParamBlocks = InTexmap->NumParamBlocks();

	Texmap* TextureSlot1 = nullptr;
	Texmap* TextureSlot2 = nullptr;
	Texmap* textureMask = nullptr;
	bool bUseTexmap1 = true;
	bool bUseTexmap2 = true;
	bool useMask = true;
	BMM_Color_fl Color1;
	BMM_Color_fl Color2;
	double MixAmount = 0.0;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map1")) == 0)
			{
				TextureSlot1 = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map2")) == 0)
			{
				TextureSlot2 = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Mask")) == 0)
			{
				textureMask = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("color1")) == 0)
			{
				Color1 = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("color2")) == 0)
			{
				Color2 = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("MixAmount")) == 0)
			{
				MixAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map1enabled")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bUseTexmap1 = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map2enabled")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bUseTexmap2 = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("maskEnabled")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					useMask = false;
				}
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	FString Result = TEXT("");
	if (TextureSlot1 != nullptr && bUseTexmap1)
	{
		Result = DumpTexture(DatasmithScene, CompTex, TextureSlot1, DATASMITH_TEXTURENAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
	}
	else
	{
		CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color1));
	}

	if (TextureSlot2 != nullptr && bUseTexmap2)
	{
		DumpTexture(DatasmithScene, CompTex, TextureSlot2, DATASMITH_TEXTURENAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
	}
	else
	{
		CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color2));
	}

	CompTex->AddParamVal1(IDatasmithCompositeTexture::ParamVal(1.f, TEXT("BaseLayerMixAmount")));
	if (textureMask != nullptr && useMask)
	{
		TSharedPtr<IDatasmithCompositeTexture> MaskTextureMap = FDatasmithSceneFactory::CreateCompositeTexture();

		DumpTexture(DatasmithScene, MaskTextureMap, textureMask, DATASMITH_MASKNAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
		CompTex->AddMaskSurface(MaskTextureMap);
		CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(-1.f, TEXT("WeightUsesMask")) );
	}
	else
	{
		CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal((float)MixAmount, TEXT("MixAmount")) );
	}

	return Result;
}

FString FDatasmithMaxMatWriter::DumpCompositetex(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale)
{
	FString ActualPrefix = FString(Prefix) + FString(TEXT("comp"));
	CompTex->SetMode( EDatasmithCompMode::Composite );

	DatasmithMaxTexmapParser::FCompositeTexmapParameters TexmapParameters = DatasmithMaxTexmapParser::ParseCompositeTexmap( InTexmap );

	FString Result;

	bool bIsFirstLayer = true;

	for ( DatasmithMaxTexmapParser::FCompositeTexmapParameters::FLayer& Layer : TexmapParameters.Layers )
	{
		if ( Layer.Map.Map != nullptr && Layer.Map.bEnabled && ( !FMath::IsNearlyZero( Layer.Map.Weight ) || Layer.Mask.Map != nullptr ) )
		{
			DumpTexture( DatasmithScene, CompTex, Layer.Map.Map, DATASMITH_TEXTURENAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale );

			if ( !bIsFirstLayer )
			{
				if ( Layer.Mask.Map != nullptr )
				{
					TSharedPtr<IDatasmithCompositeTexture> MaskTextureMap = FDatasmithSceneFactory::CreateCompositeTexture();

					DumpTexture(DatasmithScene, MaskTextureMap, Layer.Mask.Map, DATASMITH_MASKNAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
					CompTex->AddMaskSurface(MaskTextureMap);
					CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(-1.f, TEXT("WeightUsesMask")) );
				}
				else
				{
					CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(Layer.Map.Weight, TEXT("Weight")) );
				}

				CompTex->AddParamVal2( IDatasmithCompositeTexture::ParamVal(float( Layer.CompositeMode ), TEXT("Mode")) );
			}
			else // layer 0
			{
				CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(1.f, TEXT("BaseLayerWeight")) ); // Weights
				CompTex->AddParamVal2( IDatasmithCompositeTexture::ParamVal(0.f, TEXT("Mode")) );   // mode
			}

			bIsFirstLayer = false;
		}
	}

	return Result;
}

FString FDatasmithMaxMatWriter::DumpFalloff(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale)
{
	int NumParamBlocks = InTexmap->NumParamBlocks();

	Texmap* TextureSlot1 = nullptr;
	Texmap* TextureSlot2 = nullptr;
	bool bUseTexmap1 = true;
	bool bUseTexmap2 = true;

	BMM_Color_fl Color1;
	BMM_Color_fl Color2;

	int Type = 1;      // perpendicular/parallel
	int Direction = 0; // z axis of camera

	float IORn = 1.5f;
	float IORk = 0.0f;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map1")) == 0)
			{
				TextureSlot1 = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map2")) == 0)
			{
				TextureSlot2 = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("color1")) == 0)
			{
				Color1 = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("color2")) == 0)
			{
				Color2 = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map1on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bUseTexmap1 = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map2on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bUseTexmap2 = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Type")) == 0)
			{
				Type = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Direction")) == 0)
			{
				Direction = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("IOR")) == 0)
			{
				IORn = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	FString ActualPrefix = FString(Prefix) + FString(TEXT("comp"));
	if (Type != 2)
	{
		CompTex->SetMode( EDatasmithCompMode::Fresnel );
	}
	else
	{
		CompTex->SetMode(EDatasmithCompMode::Ior );
	}

	FString Result = TEXT("");
	if (TextureSlot1 != nullptr && bUseTexmap1)
	{
		Result = DumpTexture(DatasmithScene, CompTex, TextureSlot1, DATASMITH_TEXTURENAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
	}
	else
	{
		CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color1));
	}

	if (TextureSlot2 != nullptr && bUseTexmap2)
	{
		DumpTexture(DatasmithScene, CompTex, TextureSlot2, DATASMITH_TEXTURENAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);
	}
	else
	{
		CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color2));
	}

	if (Type != 2)
	{
		CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(2.0f, TEXT("DefaultFalloffVal")) );  // max falloff uses 2 as constant
		CompTex->AddParamVal2( IDatasmithCompositeTexture::ParamVal(0.0f, TEXT("DefaultFalloffBase")) ); // max falloff uses 0 as Base
	}
	else
	{
		CompTex->AddParamVal1( IDatasmithCompositeTexture::ParamVal(IORn, TEXT("iorNvalue")) );
		CompTex->AddParamVal2( IDatasmithCompositeTexture::ParamVal(IORk, TEXT("iorKvalue")) );
	}

	if (Type != 1 && Type != 2)
	{
		DatasmithMaxLogger::Get().AddGeneralError(TEXT("Falloff maps work only on perpendicular/parallel or ior mode"));
	}

	if (Direction != 0)
	{
		DatasmithMaxLogger::Get().AddGeneralError(TEXT("Falloff maps work only on Z-axis"));
	}

	return Result;
}

FString FDatasmithMaxMatWriter::DumpBakeable(TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale)
{
	MSTR ClassName;
	InTexmap->GetClassName(ClassName);
	FString FileName = FString(InTexmap->GetName().data()) + FString(ClassName.data()) + FString::FromInt(InTexmap->GetHandleByAnim(InTexmap));

	FString Base = FDatasmithUtils::SanitizeFileName(FileName);
	FString OrigBase = FDatasmithUtils::SanitizeFileName(FileName);
	Base += TextureSuffix;

	int CurrentFrame = GetCOREInterface()->GetTime() / GetTicksPerFrame();
	TimeValue CurrentTime = TimeValue(TIME_TICKSPERSEC * ((float)CurrentFrame / GetFrameRate()));

	FDatasmithTextureSampler TextureSampler(SetupTextureSampler(InTexmap, CurrentTime, 1, bForceInvert));
	CompTex->AddSurface(*OrigBase, TextureSampler);

	return Base;
}
