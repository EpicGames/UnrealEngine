// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithMaxSceneExporter.h"

#include "DatasmithMaxDirectLink.h"
#include "DatasmithMaxConverters.h"

#include "DatasmithMaxAttributes.h"
#include "DatasmithMaxCameraExporter.h"
#include "DatasmithMaxExporterDefines.h"
#include "DatasmithMaxHelper.h"
#include "DatasmithMaxLogger.h"
#include "DatasmithMaxSceneHelper.h"
#include "DatasmithMaxWriter.h"
#include "DatasmithSceneExporter.h"
#include "DatasmithSceneFactory.h"
#include "VRayLights.h"


#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "icustattribcontainer.h"
	#include "IFileResolutionManager.h"
	#include "iparamb2.h"
	#include "decomp.h"
	#include "lslights.h"
	#include "ilayer.h"
	#include "Scene/IPhysicalCamera.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

FMaxLightCoordinateConversionParams::FMaxLightCoordinateConversionParams(INode* LightNode, EDatasmithLightShape Shape)
{
	ObjectState ObjState = LightNode->EvalWorldState(0);
	if (ObjState.obj != nullptr && ObjState.obj->SuperClassID() == LIGHT_CLASS_ID)
	{
		bIsLight = true;
		bIsCoronaLight = FDatasmithMaxSceneHelper::GetLightClass(LightNode) == EMaxLightClass::CoronaLight;
		LightShape = Shape;
	}
}

FString FDatasmithMaxSceneExporter::GetActualPath(const TCHAR* OriginalPath)
{
	IFileResolutionManager* FileResolutionManager = IFileResolutionManager::GetInstance();
	MaxSDK::Util::Path Path(OriginalPath);

	if (FileResolutionManager->GetFullFilePath(Path, MaxSDK::AssetManagement::kBitmapAsset))
	{
		return FString(Path.GetCStr());
	}
	else if (FileResolutionManager->GetFullFilePath(Path, MaxSDK::AssetManagement::kOtherAsset))
	{
		return FString(Path.GetCStr());
	}
	else if (FileResolutionManager->GetFullFilePath(Path, MaxSDK::AssetManagement::kPhotometricAsset))
	{
		return FString(Path.GetCStr());
	}
	else if (FileResolutionManager->GetFullFilePath(Path, MaxSDK::AssetManagement::kXRefAsset))
	{
		return FString(Path.GetCStr());
	}
	else
	{
		return FString(Path.GetCStr());
	}
}

// From FindBetween_Helper, for 3ds max classes
Quat FindBetweenVectors(const Point3& A, const Point3& B)
{
	// Vector A and B must be normalized
	const float NormAB = 1.0f;
	float W = NormAB + DotProd(A, B);
	Quat Result;

	if (W >= 1e-6f * NormAB)
	{
		//Axis = FVector::CrossProduct(A, B);
		Result = Quat(A.y * B.z - A.z * B.y,
			A.z * B.x - A.x * B.z,
			A.x * B.y - A.y * B.x,
			W);
	}
	else
	{
		// A and B point in opposite directions
		W = 0.f;
		Result = FMath::Abs(A.x) > FMath::Abs(A.y)
			? Quat(-A.z, 0.f, A.x, W)
			: Quat(0.f, -A.z, A.y, W);
	}

	Result.Normalize();
	return Result;
}

void FDatasmithMaxSceneExporter::MaxToUnrealCoordinates(Matrix3 Matrix, FVector& Translation, FQuat& Rotation, FVector& Scale, float UnitMultiplier, const FMaxLightCoordinateConversionParams& LightParams)
{
	Point3 Pos = Matrix.GetTrans();
	Translation.X = Pos.x * UnitMultiplier;
	Translation.Y = -Pos.y * UnitMultiplier;
	Translation.Z = Pos.z * UnitMultiplier;

	// Clear the transform on the matrix
	Matrix.NoTrans();

	// We're only doing Scale - save out the
	// rotation so we can put it back
	AffineParts Parts;
	decomp_affine(Matrix, &Parts);
	ScaleValue ScaleVal = ScaleValue(Parts.k * Parts.f, Parts.u);
	Scale = FVector( ScaleVal.s.x, ScaleVal.s.y, ScaleVal.s.z );

	Rotation = FQuat( Parts.q.x, -Parts.q.y, Parts.q.z, Parts.q.w );

	// Special case for light with negative scale
	// Note that the negative scale can be applied on the pivot itself so it's possible it doesn't show up as a negative scale on the object transform
	if (LightParams.bIsLight && Parts.f < 0)
	{
		// Treat the scale as positive 
		ScaleVal = ScaleValue(Parts.k, Parts.u);
		Scale = FVector(ScaleVal.s.x, ScaleVal.s.y, ScaleVal.s.z);

		// Need to find the rotation transform that will give the same result as Matrix
		// Do that by creating an arbitrary vector, applying the Matrix transform on it, and finding the rotation quaternion that transforms the vector into the other
		// Create an arbitrary vector
		Point3 Down(0, 0, -1);

		// Apply the Matrix transform on it
		Point3 DownTM = Matrix.VectorTransform(Down);
		DownTM = DownTM.Normalize();

		// Find the rotation matrix between the 2 normalized vectors
		Quat Rotation1 = FindBetweenVectors(DownTM, Down);

		// Do the same with another arbitrary vector orthogonal to the first one to compose with the first rotation to get the complete rotation
		Point3 Right(1, 0, 0);
		Point3 RightTM = Matrix.VectorTransform(Right);
		RightTM = RightTM.Normalize();
		Quat Rotation2 = FindBetweenVectors(RightTM, Right);

		Quat QuatRes = Rotation1 * Rotation2;

		Rotation = FQuat(QuatRes.x, -QuatRes.y, QuatRes.z, QuatRes.w);
	}

	if (LightParams.bIsLight)
	{
		if (LightParams.LightShape != EDatasmithLightShape::Cylinder)
		{
			// Lights in UE are looking towards +X with an identity transform while in Max, they are looking towards -Z
			Rotation = Rotation * FRotator(-90.f, 90.f, 0.f).Quaternion();
		}
		else if(!LightParams.bIsCoronaLight)
		{
			// Cylinder lights have a different orientations so we need to correct that one too, except for Corona cylinder lights which have the same orientation as in UE.
			Rotation = Rotation * FRotator(0.f, 0.f, 90.f).Quaternion();
		}
	}
}

float FDatasmithMaxSceneExporter::GetLightPhysicalScale()
{
	ToneOperatorInterface* ToneOpInterface = static_cast< ToneOperatorInterface* >( GetCOREInterface( TONE_OPERATOR_INTERFACE ) );

	if ( !ToneOpInterface )
	{
		return 1500.f;
	}

	ToneOperator* ToneOp = ToneOpInterface->GetToneOperator();

	if ( !ToneOp || !ToneOp->Active( GetCOREInterface()->GetTime() ) )
	{
		return 1500.f;
	}

	Interval ValidityInterval;
	return ToneOp->GetPhysicalUnit(GetCOREInterface()->GetTime(), ValidityInterval );
}

int FDatasmithMaxSceneExporter::GetSeedFromMaterial(Mtl* Material)
{
	int Seed = -1;
	int NumParamBlocks = Material->NumParamBlocks();
	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Seed")) == 0)
			{
				Seed = ParamBlock2->GetInt(ParamDefinition.ID);
			}
		}
		ParamBlock2->ReleaseDesc();
	}
	return Seed;
}

Mtl* FDatasmithMaxSceneExporter::GetRandomSubMaterial(Mtl* Material, FVector3f RandomSeed)
{
	uint32 RandomIndex = int(FMath::Abs(RandomSeed.X - RandomSeed.Y + RandomSeed.Z)) + GetSeedFromMaterial(Material);

	TArray<Mtl*> SubMats;

	for (int i = 0; i < Material->NumSubMtls(); i++)
	{
		if (Material->GetSubMtl(i) != nullptr)
		{
			SubMats.Add(Material->GetSubMtl(i));
		}
	}

	if (SubMats.Num() < 1)
	{
		return nullptr;
	}

	return SubMats[RandomIndex % SubMats.Num()];
}

FTransform FDatasmithMaxSceneExporter::GetPivotTransform( INode* Node, float UnitMultiplier )
{
	FTransform Pivot = FTransform::Identity;
	bool bIsInWorldSpace = ( Node->GetObjTMAfterWSM( GetCOREInterface()->GetTime() ).IsIdentity() != 0 ); // If we're in world space, don't apply the object-offset since it was baked into the vertices

	if ( !bIsInWorldSpace )
	{
// Matrix3::Matrix3(BOOL) is deprecated in 3ds max 2022 SDK
#if MAX_PRODUCT_YEAR_NUMBER < 2022
		Matrix3 ObjectOffset(1);
#else
		Matrix3 ObjectOffset;
#endif
		Point3 OffsetPos = Node->GetObjOffsetPos(); // Object translation in Node (pivot) space
		ObjectOffset.PreTranslate( OffsetPos );

		Quat OffsetRot = Node->GetObjOffsetRot();
		PreRotateMatrix( ObjectOffset, OffsetRot );

		ScaleValue OffsetScale = Node->GetObjOffsetScale();
		ApplyScaling( ObjectOffset, OffsetScale );

		ObjectOffset.ValidateFlags(); // Needed before calling IsIdentity to update the matrix IDENT flags

		if ( ObjectOffset.IsIdentity() == 0 )
		{
			FVector PivotTranslation;
			FQuat PivotRotation;
			FVector PivotScale;
			MaxToUnrealCoordinates( ObjectOffset, PivotTranslation, PivotRotation, PivotScale, UnitMultiplier );

			Pivot = FTransform( PivotRotation, PivotTranslation, PivotScale );
		}
	}

	return Pivot;
}

void FDatasmithMaxSceneExporter::ExportAnimation( TSharedRef< IDatasmithLevelSequenceElement > LevelSequence, INode* ParentNode, INode* Node, const TCHAR* Name, float UnitMultiplier, const FMaxLightCoordinateConversionParams& LightParams)
{
	TSharedRef< IDatasmithTransformAnimationElement > Animation = FDatasmithSceneFactory::CreateTransformAnimation( Name );

	if ( ParseTransformAnimation( ParentNode, Node, Animation, UnitMultiplier, LightParams) )
	{
		LevelSequence->AddAnimation( Animation );
	}
}

TSharedPtr< IDatasmithLightActorElement > FDatasmithMaxSceneExporter::CreateLightElementForNode(INode* Node, const TCHAR* Name)
{
	EMaxLightClass LightClass = FDatasmithMaxSceneHelper::GetLightClass(Node);
	if (LightClass == EMaxLightClass::Unknown)
	{
		return TSharedPtr< IDatasmithLightActorElement >();
	}
	else if (LightClass == EMaxLightClass::SkyEquivalent)
	{
		return TSharedPtr< IDatasmithLightActorElement >();
	}
	else if (LightClass == EMaxLightClass::SunEquivalent)
	{
		return FDatasmithSceneFactory::CreateDirectionalLight( Name );
	}

	EDatasmithElementType LightType = EDatasmithElementType::PointLight;

	ObjectState ObjState = Node->EvalWorldState(0);

	if ( !ObjState.obj )
	{
		return TSharedPtr< IDatasmithLightActorElement >();
	}

	LightObject* Light = static_cast< LightObject* >( ObjState.obj );

	LightState LState;
	Interval ValidInterval = FOREVER;
	Light->EvalLightState(0, ValidInterval, &LState);

	if (LState.type == SPOT_LGT)
	{
		LightType = EDatasmithElementType::SpotLight;
	}
	else if (LState.type == DIRECT_LGT && !(LightClass == EMaxLightClass::TheaLightPlane))
	{
		LightType = EDatasmithElementType::DirectionalLight;
	}
	else if (LightClass == EMaxLightClass::SkyPortal)
	{
		LightType = EDatasmithElementType::LightmassPortal;
	}

	if (LightClass == EMaxLightClass::PhotometricLight)
	{
		LightscapeLight* PhotometricLight = static_cast< LightscapeLight* >( Light );
		LightscapeLight::DistTypes LightDistribution = PhotometricLight->GetDistribution();

		switch ( PhotometricLight->Type() )
		{
		case LightscapeLight::TARGET_POINT_TYPE:
		case LightscapeLight::POINT_TYPE:
			if (LightDistribution == LightscapeLight::SPOTLIGHT_DIST || LightDistribution == LightscapeLight::DIFFUSE_DIST )
			{
				LightType = EDatasmithElementType::SpotLight;
			}
			else
			{
				LightType = EDatasmithElementType::PointLight;
			}
			break;
		default:
			LightType = EDatasmithElementType::AreaLight;
			break;
		}
	}
	else if (LightClass == EMaxLightClass::TheaLightOmni || LightClass == EMaxLightClass::TheaLightIES || LightClass == EMaxLightClass::OmniLight ||
		LightClass == EMaxLightClass::VRayLightIES || LightClass == EMaxLightClass::VRayLight || LightClass == EMaxLightClass::ArnoldLight)
	{
		LightType = EDatasmithElementType::PointLight;
	}
	else if (LightClass == EMaxLightClass::TheaLightPlane || LightClass == EMaxLightClass::PhotoplaneLight || LightClass == EMaxLightClass::CoronaLight)
	{
		LightType = EDatasmithElementType::AreaLight;
	}
	else if (LightClass == EMaxLightClass::SpotLight)
	{
		LightType = EDatasmithElementType::SpotLight;
	}
	else if (LightClass == EMaxLightClass::DirectLight)
	{
		LightType = EDatasmithElementType::DirectionalLight;
	}

	bool bIsArnoldPortal = false;

	int NumParamBlocks = Light->NumParamBlocks();

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Light->GetParamBlockByID((short)j);
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			// check VRAY portal
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("skylightPortal")) == 0 && LightClass == EMaxLightClass::VRayLight)
			{
				int IsPortal = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (IsPortal != 0)
				{
					LightType = EDatasmithElementType::LightmassPortal;
				}
			}

			// VRAY Light
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("type")) == 0 && LightClass == EMaxLightClass::VRayLight)
			{
				LightType = EDatasmithElementType::AreaLight;

				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 1) // DOME
				{
					LightType = EDatasmithElementType::EnvironmentLight;
				}
			}

			// VRAY Light IES
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("override_shape")) == 0 && LightClass == EMaxLightClass::VRayLightIES)
			{
				// Check that override shape is checked and that we're not using a point shape
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0 && ParamBlock2->GetInt(ParamDefinition.ID + 1) != (int)EVRayIESLightShapes::Point)
				{
					LightType = EDatasmithElementType::AreaLight;
				}
			}

			// Arnold Light transform
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("shapeType")) == 0 && LightClass == EMaxLightClass::ArnoldLight)
			{
				// Quad
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 3)
				{
					LightType = EDatasmithElementType::AreaLight;
				}
				// Disk
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 4)
				{
					LightType = EDatasmithElementType::AreaLight;
				}
				// Spot
				else if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 2)
				{
					LightType = EDatasmithElementType::SpotLight;
				}
				// Distant
				else if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 1)
				{
					LightType = EDatasmithElementType::DirectionalLight;
				}
				// SkyDome
				else if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 6)
				{
					LightType = EDatasmithElementType::EnvironmentLight;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("portal_mode")) == 0 && LightClass == EMaxLightClass::ArnoldLight)
			{
				int IsPortal = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
				bIsArnoldPortal = IsPortal != 0;
			}
		}

		// Arnold portal attribute is only relevant for SkyDome shape
		if (bIsArnoldPortal && (LightType == EDatasmithElementType::EnvironmentLight))
		{
			LightType = EDatasmithElementType::LightmassPortal;
		}

		ParamBlock2->ReleaseDesc();
	}

	TSharedPtr< IDatasmithElement > Element = FDatasmithSceneFactory::CreateElement( LightType, Name );

	if ( Element.IsValid() && Element->IsA( EDatasmithElementType::Light ) )
	{
		return StaticCastSharedPtr< IDatasmithLightActorElement >( Element );
	}

	return TSharedPtr< IDatasmithLightActorElement >();
}

TSharedPtr<IDatasmithMetaDataElement> FDatasmithMaxSceneExporter::ParseUserProperties(INode* Node, TSharedRef< IDatasmithActorElement > ActorElement, TSharedRef< IDatasmithScene > DatasmithScene)
{
	// Check if the node has some metadata associated as custom user properties (User Defined Properties in Object Properties)
	MSTR Buffer;
	Node->GetUserPropBuffer(Buffer);

	FString StringBuffer = Buffer.data();
	if (!StringBuffer.IsEmpty())
	{
		// As per the Max SDK: "The property name must appear at the start of the line (although it can be preceded by spaces or tabs)
		// and there can be only one property per line."

		// Split buffer into separate lines (one key-value per line)
		TArray<FString> Lines;
		FString LeftString;
		FString RightString;
		while (StringBuffer.Split(TEXT("\n"), &LeftString, &RightString))
		{
			Lines.Add(LeftString);
			StringBuffer = RightString;
		}
		Lines.Add(StringBuffer);

		TSharedRef< IDatasmithMetaDataElement > MetaDataElement = FDatasmithSceneFactory::CreateMetaData(ActorElement->GetName());
		MetaDataElement->SetAssociatedElement(ActorElement);

		// Then extract the property key-value pair and fill them into the MetadataElement
		// As per the Max SDK: "Using the Object Properties dialog a user may enter text in the following format:
		// PropertyName1=PropertyValue 1"
		// Note that the Key cannot contain space, tab, '='. Those characters are automatically replaced by '_' by Max
		for (const FString& Line : Lines)
		{
			FString Key;
			FString Value;
			if (Line.Split(TEXT(" = "), &Key, &Value))
			{
				Key.TrimStartAndEndInline();
				Value.TrimStartAndEndInline();

				TSharedRef< IDatasmithKeyValueProperty > KeyValueProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
				KeyValueProperty->SetValue(*Value);
				KeyValueProperty->SetPropertyType(EDatasmithKeyValuePropertyType::String);
				MetaDataElement->AddProperty(KeyValueProperty);
			}
		}

		DatasmithScene->AddMetaData(MetaDataElement);
		return MetaDataElement;
	}
	return TSharedPtr<IDatasmithMetaDataElement>();
}

bool FDatasmithMaxSceneExporter::ParseActor(INode* Node, TSharedRef< IDatasmithActorElement > ActorElement, float UnitMultiplier, TSharedRef< IDatasmithScene > DatasmithScene)
{
	if ( Node == nullptr )
	{
		return false;
	}

	ParseUserProperties(Node, ActorElement, DatasmithScene);

	ActorElement->SetLabel( Node->GetName() );

	ILayer* Layer = (ILayer*)Node->GetReference(NODE_LAYER_REF);
	if (Layer)
	{
		ActorElement->SetLayer( Layer->GetName().data() );
	}

	FVector Translation, Scale;
	FQuat Rotation;

	const FMaxLightCoordinateConversionParams LightParams = FMaxLightCoordinateConversionParams(Node);
	if (Node->GetWSMDerivedObject() != nullptr)
	{
		MaxToUnrealCoordinates(Node->GetObjTMAfterWSM(GetCOREInterface()->GetTime()), Translation, Rotation, Scale, UnitMultiplier, LightParams);
	}
	else
	{
		MaxToUnrealCoordinates(Node->GetObjectTM(GetCOREInterface()->GetTime()), Translation, Rotation, Scale, UnitMultiplier, LightParams);
	}

	Rotation.Normalize();

	ActorElement->SetTranslation(Translation);
	ActorElement->SetScale(Scale);
	ActorElement->SetRotation(Rotation);

	return true;
}

bool FDatasmithMaxSceneExporter::ParseTransformAnimation(INode* ParentNode, INode* Node, TSharedRef< IDatasmithTransformAnimationElement > AnimationElement, float UnitMultiplier, const FMaxLightCoordinateConversionParams& LightParams)
{
	if (Node == nullptr)
	{
		return false;
	}

	ObjectState ObjState = Node->EvalWorldState(0);
	bool bIsCameraNode = ObjState.obj->SuperClassID() == CAMERA_CLASS_ID;

	// Extra rotation needed for camera nodes
	FRotator CameraRotator(-90.f, 0.f, -90.f);
	FQuat CameraRotation(CameraRotator.Quaternion());

	// Query the node transform in local space at -infinity to determine if it has any animation
	Interval ValidInterval = FOREVER;
	Matrix3 NodeTransform = Node->GetNodeTM(TIME_NegInfinity, &ValidInterval);

// Matrix3::Matrix3(BOOL) is deprecated in 3ds max 2022 SDK
#if MAX_PRODUCT_YEAR_NUMBER < 2022
	Matrix3 ParentTransform = ParentNode ? ParentNode->GetNodeTM(TIME_NegInfinity, &ValidInterval) : Matrix3(true);
#else
	Matrix3 ParentTransform = ParentNode ? ParentNode->GetNodeTM(TIME_NegInfinity, &ValidInterval) : Matrix3();
#endif
	Matrix3 LocalTransform;

	// An infinite interval indicates there's no animation in the transform
	if (ValidInterval.IsInfinite())
	{
		return false;
	}

	const int TicksPerFrame = GetTicksPerFrame();
	const int StartTime = GetCOREInterface()->GetAnimRange().Start();
	const int EndTime = GetCOREInterface()->GetAnimRange().End();

	// Sample the transform controller at every frame in the animation range
	for (int CurrentTime = StartTime; CurrentTime <= EndTime; CurrentTime += TicksPerFrame)
	{
		int Frame = CurrentTime / TicksPerFrame;
		ValidInterval = FOREVER;

		// The parent node could change at each frame because of the Link Constraint
		NodeTransform = Node->GetNodeTM(CurrentTime, &ValidInterval);
		
// Matrix3::Matrix3(BOOL) is deprecated in 3ds max 2022 SDK
#if MAX_PRODUCT_YEAR_NUMBER < 2022
		ParentTransform = ParentNode ? ParentNode->GetNodeTM(CurrentTime, &ValidInterval) : Matrix3(true);
#else
		ParentTransform = ParentNode ? ParentNode->GetNodeTM(CurrentTime, &ValidInterval) : Matrix3();
#endif

		Matrix3 InvParentTransform = Inverse(ParentTransform);
		LocalTransform = NodeTransform * InvParentTransform;

		FVector Translation;
		FQuat Rotation;
		FVector Scale; 
		FDatasmithMaxSceneExporter::MaxToUnrealCoordinates(LocalTransform, Translation, Rotation, Scale, UnitMultiplier, LightParams);

		if (bIsCameraNode)
		{
			Rotation *= CameraRotation;
		}

		// The scale returned by decomp_affine for LocalTransform is incorrect so compute it separately
		// by extracting the scales from the node's and inverse parent's transform
		// The translation and rotation extracted from LocalTransform are still valid
		FVector TempTranslation;
		FQuat TempRotation;
		FVector NodeScale; 
		FDatasmithMaxSceneExporter::MaxToUnrealCoordinates(NodeTransform, TempTranslation, TempRotation, NodeScale, UnitMultiplier, LightParams);

		FVector InvParentScale; 
		FDatasmithMaxSceneExporter::MaxToUnrealCoordinates(InvParentTransform, TempTranslation, TempRotation, InvParentScale, UnitMultiplier, LightParams);

		Scale = InvParentScale * NodeScale;

		// The first frame might not actually by at time 0, but at the end of the validity interval
		if (ValidInterval.Start() == TIME_NegInfinity)
		{
			if (ValidInterval.End() == TIME_PosInfinity)
			{
				// If we end up with an infinite validity interval while evaluating a value in the time range
				// it means that there's actually no animation, so remove any frames that were previously added
				// This can happen if an object is transformed just before the export; its validity interval
				// at time 0 will be [0, 0], but the following time evaluation will be infinite
				for (int32 FrameIndex = 0; FrameIndex < AnimationElement->GetFramesCount(EDatasmithTransformType::Translation); ++FrameIndex)
				{
					AnimationElement->RemoveFrame(EDatasmithTransformType::Translation, FrameIndex);
				}
				for (int32 FrameIndex = 0; FrameIndex < AnimationElement->GetFramesCount(EDatasmithTransformType::Rotation); ++FrameIndex)
				{
					AnimationElement->RemoveFrame(EDatasmithTransformType::Rotation, FrameIndex);
				}
				for (int32 FrameIndex = 0; FrameIndex < AnimationElement->GetFramesCount(EDatasmithTransformType::Scale); ++FrameIndex)
				{
					AnimationElement->RemoveFrame(EDatasmithTransformType::Scale, FrameIndex);
				}
				break;
			}
			// Add the frame at the end of the validity interval
			Frame = FMath::RoundHalfToEven((ValidInterval.End() / (float)TicksPerFrame));
		}

		// Add keyframes for all transform types
		AnimationElement->AddFrame(EDatasmithTransformType::Translation, FDatasmithTransformFrameInfo(Frame, Translation.X, Translation.Y, Translation.Z));

		FVector EulerAngles = Rotation.Euler();
		AnimationElement->AddFrame(EDatasmithTransformType::Rotation, FDatasmithTransformFrameInfo(Frame, EulerAngles.X, EulerAngles.Y, EulerAngles.Z));

		AnimationElement->AddFrame(EDatasmithTransformType::Scale, FDatasmithTransformFrameInfo(Frame, Scale.X, Scale.Y, Scale.Z));

		// Since there's no change during the validity interval, jump to the end of the interval if it's greater than the current time and not infinity (to avoid infinite loops)
		if (ValidInterval.End() > CurrentTime && ValidInterval.End() != TIME_PosInfinity)
		{
			// Make sure the CurrentTime is a multiple of TicksPerFrame
			CurrentTime = FMath::RoundHalfToEven((ValidInterval.End() / (float)TicksPerFrame)) * TicksPerFrame;
		}
	}

	return true;
}

bool FDatasmithMaxSceneExporter::ParseLight(DatasmithMaxDirectLink::FLightNodeConverter& Converter, INode* Node, TSharedRef< IDatasmithLightActorElement > LightElement, TSharedRef< IDatasmithScene > DatasmithScene)
{
	EMaxLightClass LightClass = FDatasmithMaxSceneHelper::GetLightClass(Node);
	if (LightClass == EMaxLightClass::Unknown)
	{
		return false;
	}
	else if (LightClass == EMaxLightClass::SkyEquivalent)
	{
		DatasmithScene->SetUsePhysicalSky(true);
		return false;
	}
	else if (LightClass == EMaxLightClass::SunEquivalent)
	{
		ParseSun(Node, LightElement);
		return true;
	}

	ObjectState ObjState = Node->EvalWorldState(0);

	if ( !ObjState.obj )
	{
		return false;
	}

	LightObject* Light = static_cast< LightObject* >( ObjState.obj );

	// Apply generic settings
	ParseLightObject( *Light, LightElement, DatasmithScene );

	if ( LightClass == EMaxLightClass::PhotometricLight && LightElement->IsA( EDatasmithElementType::PointLight ) )
	{
		TSharedRef< IDatasmithPointLightElement > PointLightElement = StaticCastSharedRef< IDatasmithPointLightElement >( LightElement );

		ParsePhotometricLight(Converter, *Light, PointLightElement, DatasmithScene );
	}
	else if ( LightClass == EMaxLightClass::VRayLight && LightElement->IsA( EDatasmithElementType::AreaLight ) )
	{
		TSharedRef< IDatasmithAreaLightElement > AreaLightElement = StaticCastSharedRef< IDatasmithAreaLightElement >( LightElement );

		ParseVRayLight( *Light, AreaLightElement, DatasmithScene );
	}
	else if ( LightClass == EMaxLightClass::VRayLight && LightElement->IsA( EDatasmithElementType::LightmassPortal ) )
	{
		TSharedRef< IDatasmithLightmassPortalElement > LightPortalElement = StaticCastSharedRef< IDatasmithLightmassPortalElement >( LightElement );

		ParseVRayLightPortal( *Light, LightPortalElement, DatasmithScene );
	}
	else if ( LightClass == EMaxLightClass::VRayLightIES && LightElement->IsA( EDatasmithElementType::PointLight ) )
	{
		TSharedRef< IDatasmithPointLightElement > PointLightElement = StaticCastSharedRef< IDatasmithPointLightElement >( LightElement );

		ParseVRayLightIES( Converter,  *Light, PointLightElement, DatasmithScene );
	}
	else if ( LightClass == EMaxLightClass::CoronaLight && LightElement->IsA( EDatasmithElementType::AreaLight ) )
	{
		TSharedRef< IDatasmithAreaLightElement > AreaLightElement = StaticCastSharedRef< IDatasmithAreaLightElement >( LightElement );

		ParseCoronaLight( Converter, *Light, AreaLightElement, DatasmithScene );
	}
	else
	{
		ParseLightParameters( Converter, LightClass, *Light, LightElement, DatasmithScene );
	}

	if ( LightElement->GetUseIes() )
	{
		if ( !Converter.IsIesProfileValid() )
		{
			LightElement->SetUseIes( false );
			DatasmithMaxDirectLink::LogWarning(FString::Printf(TEXT("IES light definition \"%s\" cannot be found for light \"%s\""), *FPaths::GetCleanFilename(Converter.GetIesProfile()), LightElement->GetName()));
		}
	}

	if ( LightElement->IsA( EDatasmithElementType::PointLight ) )
	{
		TSharedRef< IDatasmithPointLightElement > PointLightElement = StaticCastSharedRef< IDatasmithPointLightElement >( LightElement );
		
		if ( PointLightElement->GetIntensityUnits() == EDatasmithLightUnits::Unitless )
		{
			PointLightElement->SetIntensity( PointLightElement->GetIntensity() * GetLightPhysicalScale() );
			PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Candelas );
		}
	}

	//Cylinder shaped lights don't have the same default orientations, so we recalculate their transform and add the shape information.
	if (LightElement->IsA(EDatasmithElementType::AreaLight) 
		&& StaticCastSharedRef< IDatasmithAreaLightElement >(LightElement)->GetLightShape() == EDatasmithLightShape::Cylinder)
	{
		FVector Translation, Scale;
		FQuat Rotation;

		const float UnitMultiplier = (float)GetSystemUnitScale(UNITS_CENTIMETERS);
		const FMaxLightCoordinateConversionParams LightParams = FMaxLightCoordinateConversionParams(Node, EDatasmithLightShape::Cylinder);
		if (Node->GetWSMDerivedObject() != nullptr)
		{
			MaxToUnrealCoordinates(Node->GetObjTMAfterWSM(GetCOREInterface()->GetTime()), Translation, Rotation, Scale, UnitMultiplier, LightParams);
		}
		else
		{
			MaxToUnrealCoordinates(Node->GetObjectTM(GetCOREInterface()->GetTime()), Translation, Rotation, Scale, UnitMultiplier, LightParams);
		}

		Rotation.Normalize();
		LightElement->SetTranslation(Translation);
		LightElement->SetScale(Scale);
		LightElement->SetRotation(Rotation);
	}

	return true;
}

bool FDatasmithMaxSceneExporter::ParseLightObject(LightObject& Light, TSharedRef< IDatasmithLightActorElement > LightElement, TSharedRef< IDatasmithScene > DatasmithScene)
{
	LightState LState;
	Interval ValidInterval = FOREVER;
	RefResult EvalLightStateResult = Light.EvalLightState( 0, ValidInterval, &LState );

	LightElement->SetEnabled( Light.GetUseLight() != 0 );

	Point3 LightColor = LState.color;
	LightElement->SetColor( FLinearColor( LightColor.x, LightColor.y, LightColor.z ) );

	float Intensity = Light.GetIntensity( GetCOREInterface()->GetTime() );
	LightElement->SetIntensity( Intensity );

	if ( LightElement->IsA( EDatasmithElementType::PointLight ) )
	{
		TSharedRef< IDatasmithPointLightElement > PointLightElement = StaticCastSharedRef< IDatasmithPointLightElement >( LightElement );

		if ( LState.useAtten != 0 ) // Using the LightState here because calling Light.GetUseAtten() on a Corona Light crashes
		{
			PointLightElement->SetAttenuationRadius( LState.attenEnd * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
		}
	}

	if ( LightElement->IsA( EDatasmithElementType::SpotLight ) )
	{
		TSharedRef< IDatasmithSpotLightElement > SpotLightElement = StaticCastSharedRef< IDatasmithSpotLightElement >( LightElement );

		SpotLightElement->SetInnerConeAngle( Light.GetHotspot( GetCOREInterface()->GetTime() ) * 0.5f );
		SpotLightElement->SetOuterConeAngle( Light.GetFallsize( GetCOREInterface()->GetTime() ) * 0.5f );
	}

	return true;
}

bool FDatasmithMaxSceneExporter::ParseCoronaLight(DatasmithMaxDirectLink::FLightNodeConverter& Converter, LightObject& Light, TSharedRef< IDatasmithAreaLightElement > AreaLightElement, TSharedRef< IDatasmithScene > DatasmithScene)
{
	enum class ECoronaIntensityUnits
	{
		Default,
		Lumen,
		Candelas,
		Lux
	};

	enum class ECoronaLightShapes
	{
		Sphere,
		Rectangle,
		Disk,
		Cylinder
	};

	AreaLightElement->SetLightType( EDatasmithAreaLightType::Point );

	bool bLightShapeVisible = true;
	bool bLightIntensityIsInDefaultCoronaUnit = false;
	bool bLightIntensityIsInLux = false;
	short IntensityBlockID = -1;
	ParamID	IntensityUnitID = -1;
	ParamID IntensityID = -1;

	const int NumParamBlocks = Light.NumParamBlocks();

	for ( int j = 0; j < NumParamBlocks; j++ )
	{
		IParamBlock2* ParamBlock2 = Light.GetParamBlockByID( (short)j );
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("shape")) == 0 )
			{
				switch ( (ECoronaLightShapes)ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) )
				{
				case ECoronaLightShapes::Disk:
					AreaLightElement->SetLightShape( EDatasmithLightShape::Disc );
					AreaLightElement->SetLightType( EDatasmithAreaLightType::Rect );

					break;

				case ECoronaLightShapes::Rectangle:
					AreaLightElement->SetLightShape( EDatasmithLightShape::Rectangle );
					AreaLightElement->SetLightType( EDatasmithAreaLightType::Rect );

					break;

				case ECoronaLightShapes::Sphere:
					AreaLightElement->SetLightShape( EDatasmithLightShape::Sphere );

					break;

				case ECoronaLightShapes::Cylinder:
					AreaLightElement->SetLightShape( EDatasmithLightShape::Cylinder );

					break;
				}
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("iesOn")) == 0 )
			{
				if ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 )
				{
					AreaLightElement->SetUseIes( true );
					AreaLightElement->SetUseIesBrightness( true );
				}
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("iesfile")) == 0 )
			{
				Converter.ApplyIesProfile(*GetActualPath( ParamBlock2->GetStr( ParamDefinition.ID, GetCOREInterface()->GetTime() ) ));
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("colorMode")) == 0 )
			{
				if ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) == 1 )
				{
					AreaLightElement->SetUseTemperature( true );
				}
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("blackbodyTemp")) == 0 )
			{
				AreaLightElement->SetTemperature( ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("intensityUnits")) == 0 )
			{
				IntensityUnitID = ParamDefinition.ID;
				switch ( (ECoronaIntensityUnits)ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) )
				{
				case ECoronaIntensityUnits::Candelas:
					AreaLightElement->SetIntensityUnits( EDatasmithLightUnits::Candelas );
					break;
				case ECoronaIntensityUnits::Lumen:
					AreaLightElement->SetIntensityUnits( EDatasmithLightUnits::Lumens );
					break;
				case ECoronaIntensityUnits::Lux:
					//We need to convert the light intensity after establishing the shape of the light.
					bLightIntensityIsInLux = true;
					break;
				case ECoronaIntensityUnits::Default:
					//We need to convert the light intensity after establishing the shape of the light.
					bLightIntensityIsInDefaultCoronaUnit = true;
					break;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("intensity")) == 0 )
			{
				IntensityBlockID = (short)j;
				IntensityID = ParamDefinition.ID;
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("width")) == 0 )
			{
				AreaLightElement->SetWidth( ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("height")) == 0 )
			{
				AreaLightElement->SetLength( ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("visibleDirectly")) == 0 )
			{
				bLightShapeVisible = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap")) == 0 )
			{
				Texmap* TextureMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if ( TextureMap )
				{
					ProcessLightTexture( AreaLightElement, TextureMap, DatasmithScene );
				}
			}
		}

		ParamBlock2->ReleaseDesc();
	}

	if ( AreaLightElement->GetUseTemperature() )
	{
		AreaLightElement->SetColor( FLinearColor::White );
	}

	if ( AreaLightElement->GetUseIes() )
	{

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		if (Converter.IsIesProfileValid())
		{
			AreaLightElement->SetLightType(EDatasmithAreaLightType::Point);
		}
		else
		{
			AreaLightElement->SetUseIes( false );
			DatasmithMaxDirectLink::LogWarning(FString::Printf(TEXT("IES light definition \"%s\" cannot be found for light \"%s\""), *FPaths::GetCleanFilename(Converter.GetIesProfile()), AreaLightElement->GetName()));
		}
	}

	if ( AreaLightElement->GetLightShape() == EDatasmithLightShape::Disc || AreaLightElement->GetLightShape() == EDatasmithLightShape::Sphere )
	{
		AreaLightElement->SetLength( AreaLightElement->GetWidth() );
	}

	if ((bLightIntensityIsInDefaultCoronaUnit || bLightIntensityIsInLux) && IntensityID != -1 && IntensityBlockID != -1 && IntensityUnitID != -1)
	{
		if ( bLightIntensityIsInDefaultCoronaUnit )
		{
			const FVector LightScale = AreaLightElement->GetScale();

			double Area = 0.0;
			const double LightWidthInMeters = AreaLightElement->GetWidth() * LightScale.X * 0.01;
			const double LightLengthInMeters = AreaLightElement->GetLength() * LightScale.Y * 0.01;
			const double LightHeightInMeters = AreaLightElement->GetWidth() * LightScale.Z * 0.01; // Only used for spheres which can become ellipsoids when scaled

			switch ( AreaLightElement->GetLightShape() )
			{
			case EDatasmithLightShape::Rectangle:
				Area = LightWidthInMeters * LightLengthInMeters;

				break;

			case EDatasmithLightShape::Disc:
				// Area of a disc = Pi * Radius * Radius
				Area = LightWidthInMeters * LightLengthInMeters * PI;

				break;

			case EDatasmithLightShape::Sphere:
				// Area of an ellipsoid = 4 * Pi ( ( ( width * length ) ^ 1.6075 + (width * height) ^ 1.6075 + (length * height) ^ 1.6075 ) / 3 ) ^ ( 1 / 1.6075 )
				Area = FMath::Pow( LightWidthInMeters * LightLengthInMeters, 1.6075 ) + FMath::Pow( LightWidthInMeters * LightHeightInMeters, 1.6075 ) + FMath::Pow( LightLengthInMeters * LightHeightInMeters, 1.6075 );
				Area /= 3.0;
				Area = FMath::Pow( Area, 1.0 / 1.6075 );
				Area *= 4.0 * PI;

				break;

			case EDatasmithLightShape::Cylinder:
				// Area of a cylinder = ( 2 * Pi * radius * height ) + ( 2 * Pi * radius^2 )
				Area = ( 2.0 * PI * LightWidthInMeters * LightLengthInMeters ) + ( 2.0 * PI * FMath::Square( LightWidthInMeters ) );
				break;
			}

			// Default Corona units: watts / steradian * square meters (w / sr * m^2)
			// w / sr * m^2 => candela / m^2
			// 683 w / sr * m^2 = 1 candela / m^2

			if ( !FMath::IsNearlyZero( Area ) )
			{
				AreaLightElement->SetIntensityUnits( EDatasmithLightUnits::Candelas );

				const double Intensity = AreaLightElement->GetIntensity();
				double IntensityInCandelasPerSqMeters = Intensity * 683.0;
				double IntensityInCandelas = IntensityInCandelasPerSqMeters * Area;

				AreaLightElement->SetIntensity( IntensityInCandelas );
			}
		}
		else
		{
			// Here we make use of the automatic unit conversion of the corona unit properties.
			IParamBlock2* ParamBlock2 = Light.GetParamBlockByID((short)IntensityBlockID);
			const float InitialIntensityValue = ParamBlock2->GetFloat(IntensityID, GetCOREInterface()->GetTime());

			switch (AreaLightElement->GetLightShape())
			{
			case EDatasmithLightShape::Disc:
			case EDatasmithLightShape::Rectangle:
				ParamBlock2->SetValue(IntensityUnitID, GetCOREInterface()->GetTime(), (int)ECoronaIntensityUnits::Lumen);
				AreaLightElement->SetIntensityUnits(EDatasmithLightUnits::Lumens);
				AreaLightElement->SetIntensity(ParamBlock2->GetFloat(IntensityID, GetCOREInterface()->GetTime()));
				break;

			case EDatasmithLightShape::Sphere:
			case EDatasmithLightShape::Cylinder:
				ParamBlock2->SetValue(IntensityUnitID, GetCOREInterface()->GetTime(), (int)ECoronaIntensityUnits::Candelas);
				AreaLightElement->SetIntensityUnits(EDatasmithLightUnits::Candelas);
				AreaLightElement->SetIntensity(ParamBlock2->GetFloat(IntensityID, GetCOREInterface()->GetTime()));
				break;
			}

			ParamBlock2->SetValue(IntensityUnitID, GetCOREInterface()->GetTime(), bLightIntensityIsInDefaultCoronaUnit ? (int)ECoronaIntensityUnits::Default : (int)ECoronaIntensityUnits::Lux);
			//Making sure there is no floating point calculation artifact by restoring the exact same value.
			ParamBlock2->SetValue(IntensityID, GetCOREInterface()->GetTime(), InitialIntensityValue);
		}
	}

	// For disc and sphere shapes, width is radius so set both length and width to 2 * radius
	if ( AreaLightElement->GetLightShape() == EDatasmithLightShape::Disc || AreaLightElement->GetLightShape() == EDatasmithLightShape::Sphere ||
		AreaLightElement->GetLightShape() == EDatasmithLightShape::Cylinder )
	{
		AreaLightElement->SetWidth( AreaLightElement->GetWidth() * 2.f );

		if ( AreaLightElement->GetLightShape() == EDatasmithLightShape::Disc || AreaLightElement->GetLightShape() == EDatasmithLightShape::Sphere )
		{
			AreaLightElement->SetLength( AreaLightElement->GetWidth() );
		}
	}

	if ( !bLightShapeVisible)
	{
		AreaLightElement->SetLightShape( EDatasmithLightShape::None );
	}

	return true;
}

bool FDatasmithMaxSceneExporter::ParsePhotometricLight(DatasmithMaxDirectLink::FLightNodeConverter& Converter, LightObject& Light, TSharedRef< IDatasmithPointLightElement > PointLightElement, TSharedRef< IDatasmithScene > DatasmithScene)
{
	LightscapeLight& PhotometricLight = static_cast< LightscapeLight& >( Light );

	// Photometric lights don't expose attenuation in EvalLightState
	IParamBlock2* ParamBlock2 = PhotometricLight.GetParamBlockByID(LightscapeLight::PB_GENERAL);
	bool bUseAtten = ParamBlock2->GetInt((short) LightscapeLight::PB_USE_FARATTENUATION) != 0;
	float Atten = ParamBlock2->GetFloat((short) LightscapeLight::PB_END_FARATTENUATION);

	if (bUseAtten)
	{
		PointLightElement->SetAttenuationRadius( Atten * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
	}

	LightscapeLight::IntensityType IntensityUnits = PhotometricLight.GetIntensityType();

	switch ( IntensityUnits )
	{
	case LightscapeLight::LUMENS:
		PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Lumens );
		break;
	case LightscapeLight::CANDELAS:
		PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Candelas );
		break;
	case LightscapeLight::LUX_AT:
		{
			PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Candelas ); // The call to Light.GetIntensity already converted the Lux to be at 1 meter so at that distance its the same units as Candelas
		}
		break;
	default:
		PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Unitless );
		break;
	}

	// Normalize the color if a component is bigger than 1
	FLinearColor LightColor = PointLightElement->GetColor();
	float MaxComponent = FMath::Max( LightColor.R, FMath::Max( LightColor.G, LightColor.B ) );

	if ( MaxComponent > 1.f )
	{
		LightColor.R /= MaxComponent;
		LightColor.G /= MaxComponent;
		LightColor.B /= MaxComponent;

		PointLightElement->SetColor( LightColor );
	}

	PointLightElement->SetUseTemperature( PhotometricLight.GetUseKelvin() != 0 );
	PointLightElement->SetTemperature( PhotometricLight.GetKelvin( GetCOREInterface()->GetTime() ) );

	if ( PointLightElement->GetUseTemperature() )
	{
		Point3 RGBFilter = PhotometricLight.GetRGBFilter( GetCOREInterface()->GetTime() );
		PointLightElement->SetColor( FLinearColor( RGBFilter.x, RGBFilter.y, RGBFilter.z ) );
	}

	bool bUseIes = PhotometricLight.GetDistribution() == LightscapeLight::WEB_DIST;

	PointLightElement->SetUseIes( bUseIes );
	if (bUseIes)
	{
		Converter.ApplyIesProfile(PhotometricLight.GetFullWebFileName());
	}
	PointLightElement->SetUseIesBrightness( false );

	FQuat IesRotateX = FQuat( FVector::RightVector, FMath::DegreesToRadians( -PhotometricLight.GetWebRotateX() ) );
	FQuat IesRotateY = FQuat( FVector::UpVector, FMath::DegreesToRadians( PhotometricLight.GetWebRotateY() ) );
	FQuat IesRotateZ = FQuat( FVector::ForwardVector, FMath::DegreesToRadians( PhotometricLight.GetWebRotateZ() ) );

	PointLightElement->SetIesRotation( IesRotateX * IesRotateY * IesRotateZ );

	bool bSphapeRenderingEnabled = true;
	bool bIsOneSided = false;
	bool bHasVolume = false;
	LightscapeLight::LightTypes LightType = (LightscapeLight::LightTypes)PhotometricLight.Type();

	if ( PointLightElement->IsA( EDatasmithElementType::AreaLight ) )
	{
		TSharedRef< IDatasmithAreaLightElement > AreaLightElement = StaticCastSharedRef< IDatasmithAreaLightElement >( PointLightElement );

		switch ( LightType )
		{
		case LightscapeLight::DISC_TYPE:
		case LightscapeLight::TARGET_DISC_TYPE:
			AreaLightElement->SetLightShape( EDatasmithLightShape::Disc );
			AreaLightElement->SetWidth( PhotometricLight.GetRadius( GetCOREInterface()->GetTime() ) * 2.f * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
			AreaLightElement->SetLength( PhotometricLight.GetRadius( GetCOREInterface()->GetTime() ) * 2.f * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
			bIsOneSided = true;

			break;
		case LightscapeLight::AREA_TYPE:
		case LightscapeLight::TARGET_AREA_TYPE:
			AreaLightElement->SetLightShape( EDatasmithLightShape::Rectangle );
			AreaLightElement->SetWidth( PhotometricLight.GetWidth( GetCOREInterface()->GetTime() ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
			AreaLightElement->SetLength( PhotometricLight.GetLength( GetCOREInterface()->GetTime() ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
			bIsOneSided = true;

			break;

		case LightscapeLight::LINEAR_TYPE:
		case LightscapeLight::TARGET_LINEAR_TYPE:
			AreaLightElement->SetLightShape( EDatasmithLightShape::Rectangle );
			AreaLightElement->SetWidth( 1.f ); // 1 cm
			AreaLightElement->SetLength( PhotometricLight.GetLength( GetCOREInterface()->GetTime() ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );

			break;

		case LightscapeLight::SPHERE_TYPE:
		case LightscapeLight::TARGET_SPHERE_TYPE:
			AreaLightElement->SetLightShape( EDatasmithLightShape::Sphere );
			AreaLightElement->SetWidth( PhotometricLight.GetRadius( GetCOREInterface()->GetTime() ) * 2.f * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
			AreaLightElement->SetLength( PhotometricLight.GetRadius( GetCOREInterface()->GetTime() ) * 2.f * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
			bHasVolume = true;
			
			break;

		case LightscapeLight::CYLINDER_TYPE:
		case LightscapeLight::TARGET_CYLINDER_TYPE:
			AreaLightElement->SetLightShape( EDatasmithLightShape::Cylinder );
			AreaLightElement->SetWidth( PhotometricLight.GetRadius( GetCOREInterface()->GetTime() ) * 2.f * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
			AreaLightElement->SetLength( PhotometricLight.GetLength( GetCOREInterface()->GetTime() ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
			bHasVolume = true;

			break;
		}

		for ( int32 i = 0; i < PhotometricLight.GetCustAttribContainer()->GetNumCustAttribs(); ++i )
		{
			LightscapeLight::AreaLightCustAttrib* AreaLightCustAttrib = LightscapeLight::GetAreaLightCustAttrib( PhotometricLight.GetCustAttribContainer()->GetCustAttrib( i ) );

			if ( AreaLightCustAttrib )
			{
				bSphapeRenderingEnabled = AreaLightCustAttrib->IsLightShapeRenderingEnabled( GetCOREInterface()->GetTime() ) != 0;
			}
		}

		switch ( PhotometricLight.GetDistribution() )
		{
		case LightscapeLight::SPOTLIGHT_DIST:
			AreaLightElement->SetLightType( EDatasmithAreaLightType::Spot );
			break;
		case LightscapeLight::ISOTROPIC_DIST:
			if (bIsOneSided)
			{
				AreaLightElement->SetLightType(EDatasmithAreaLightType::Rect);
			}
			else
			{
				AreaLightElement->SetLightType(EDatasmithAreaLightType::Point);
			}
			break;
		case LightscapeLight::DIFFUSE_DIST:
			if (bHasVolume)
			{
				AreaLightElement->SetLightType(EDatasmithAreaLightType::Point);
			}
			else
			{
				AreaLightElement->SetLightType(EDatasmithAreaLightType::Rect);
			}
			break;
		case LightscapeLight::WEB_DIST:
			AreaLightElement->SetLightType( EDatasmithAreaLightType::IES_DEPRECATED );
			break;
		}

		if ( !bSphapeRenderingEnabled )
		{
			AreaLightElement->SetLightShape( EDatasmithLightShape::None );
		}
	}
	else if ((LightType == LightscapeLight::POINT_TYPE || LightType == LightscapeLight::TARGET_POINT_TYPE)
		&& PointLightElement->IsA(EDatasmithElementType::SpotLight)
		&& PhotometricLight.GetDistribution() == LightscapeLight::DIFFUSE_DIST)
	{
		// In the case we have a Diffuse (hemispherical) point light, it is effectively behaving as a spotlight with a 90 degree outer angle
		TSharedRef< IDatasmithSpotLightElement > SpotLightElement = StaticCastSharedRef< IDatasmithSpotLightElement >(PointLightElement);
		SpotLightElement->SetOuterConeAngle(90.f);
		SpotLightElement->SetInnerConeAngle(80.f);
	}

	return true;
}

bool FDatasmithMaxSceneExporter::ParseVRayLight(LightObject& Light, TSharedRef< IDatasmithAreaLightElement > AreaLightElement, TSharedRef< IDatasmithScene > DatasmithScene)
{
	enum class EVRayLightType
	{
		Plane,
		Dome,
		Sphere,
		Mesh,
		Disc
	};

	IParamBlock2* ParamBlock2 = Light.GetParamBlockByID( (short)EVRayLightParamBlocks::Params );
	EVRayLightType VRayLightTypeValue = (EVRayLightType)ParamBlock2->GetInt( (short)EVrayLightsParams::Type );
	
	//Since the Undo buffer is off, we can simply use the already made logic to convert the units type to lumens
	const int UnitType = ParamBlock2->GetInt( (short)EVrayLightsParams::NormalizeColor );
	const int UnitlessType= 0;
	const int LumensType = 1;
	if (UnitType == LumensType)
	{
		AreaLightElement->SetIntensityUnits( EDatasmithLightUnits::Lumens );
	}
	

	float VRayLightSize0 = ParamBlock2->GetFloat( (short)EVrayLightsParams::Size0 ) * 2.f * (float)GetSystemUnitScale( UNITS_CENTIMETERS );
	float VRayLightSize1 = ParamBlock2->GetFloat( (short)EVrayLightsParams::Size1 ) * 2.f * (float)GetSystemUnitScale( UNITS_CENTIMETERS );
	AreaLightElement->SetWidth( VRayLightSize0 );

	const double CentimeterToMeter = 0.01;

	switch ( VRayLightTypeValue )
	{
	case EVRayLightType::Plane:
		AreaLightElement->SetLightShape( EDatasmithLightShape::Rectangle );
		AreaLightElement->SetLightType( EDatasmithAreaLightType::Rect );
		AreaLightElement->SetLength( VRayLightSize1 );

		AreaLightElement->SetOuterConeAngle(160.f);
		AreaLightElement->SetInnerConeAngle(150.f);

		if (UnitType == UnitlessType)
		{
			const double Area = VRayLightSize0 * CentimeterToMeter * VRayLightSize1 * CentimeterToMeter;
			AreaLightElement->SetIntensity( AreaLightElement->GetIntensity() * Area * GetLightPhysicalScale());
			AreaLightElement->SetIntensityUnits( EDatasmithLightUnits::Lumens );
		}

		break;

	case EVRayLightType::Disc:
		AreaLightElement->SetLightShape( EDatasmithLightShape::Disc );
		AreaLightElement->SetLightType(EDatasmithAreaLightType::Rect);
		AreaLightElement->SetLength( VRayLightSize0 );

		AreaLightElement->SetOuterConeAngle( 80.f );
		AreaLightElement->SetInnerConeAngle( 75.f );

		if (UnitType == UnitlessType)
		{
			// Area of a disc = Pi * Radius * Radius
			const double Area = CentimeterToMeter * VRayLightSize0 * CentimeterToMeter * VRayLightSize0 * PI / 4.0;
			AreaLightElement->SetIntensity(AreaLightElement->GetIntensity() * Area * GetLightPhysicalScale());
			AreaLightElement->SetIntensityUnits(EDatasmithLightUnits::Lumens);
		}

		break;

	case EVRayLightType::Sphere:
		AreaLightElement->SetLightShape( EDatasmithLightShape::Sphere );
		AreaLightElement->SetLength( VRayLightSize0 );

		if (UnitType == UnitlessType)
		{
			// Area of a sphere = 4 * Pi * Radius * Radius
			const double Area = CentimeterToMeter * VRayLightSize0 * CentimeterToMeter * VRayLightSize0 * PI;
			AreaLightElement->SetIntensity(AreaLightElement->GetIntensity() * Area * GetLightPhysicalScale());
			AreaLightElement->SetIntensityUnits(EDatasmithLightUnits::Lumens);
		}

		break;
	}

	if ( ParamBlock2->GetInt( (short)EVrayLightsParams::TexmapOn ) != 0 )
	{
		Texmap* LightTexture = ParamBlock2->GetTexmap( (short)EVrayLightsParams::Texmap );

		if ( LightTexture )
		{
			ProcessLightTexture( AreaLightElement, LightTexture, DatasmithScene );
		}
	}
	
	AreaLightElement->SetUseTemperature( ParamBlock2->GetInt( (short)EVrayLightsParams::ColorMode ) != 0 );
	AreaLightElement->SetTemperature( ParamBlock2->GetFloat( (short)EVrayLightsParams::Temperature ) );

	if ( AreaLightElement->GetUseTemperature() )
	{
		AreaLightElement->SetColor( FLinearColor::White );
	}

	bool bLightShapeVisible = ( ParamBlock2->GetInt( (short)EVrayLightsParams::Transparent ) == 0 );

	if ( !bLightShapeVisible )
	{
		AreaLightElement->SetLightShape( EDatasmithLightShape::None );
	}

	return true;
}

void FDatasmithMaxSceneExporter::ParseSun(INode* Node, TSharedRef<IDatasmithLightActorElement> LightElement)
{
	float Intensity = 10.0f;
	FLinearColor Color = FLinearColor::White;
	bool bUseTemperature = true;
	float Temperature = 5780.f;

	ObjectState ObjState = Node->EvalWorldState(0);
	LightObject& Light = *(LightObject*)ObjState.obj;
	Class_ID ClassID = Light.ClassID();

	if(ClassID == VRAYSUNCLASS)
	{
		IParamBlock2* ParamBlock2 = Light.GetParamBlockByID( (short)EVRayLightParamBlocks::Params );

		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];
				
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("intensity_multiplier")) == 0)
			{
				Intensity = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("filter_color")) == 0)
			{
				Color = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor((BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
				bUseTemperature = false;
			}
		}
	}
	else if(ClassID == CORONASUNCLASS || ClassID == CORONASUNCLASSB)
	{
		int32 CoronaSunColorMode = 0;
		float CoronaSunTemperature = 0;
		FLinearColor CoronaSunColor(ForceInitToZero);

		const int NumParamBlocks = Light.NumParamBlocks();

		for ( int j = 0; j < NumParamBlocks; j++ )
		{
			IParamBlock2* ParamBlock2 = Light.GetParamBlockByID( (short)j );
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("intensity")) == 0)
				{
					Intensity = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("colorMode")) == 0)
				{
					CoronaSunColorMode = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("blackbodyTemperature")) == 0)
				{
					CoronaSunTemperature = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("colorDirect")) == 0)
				{
					CoronaSunColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor((BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
				}
			}
		}
		switch (CoronaSunColorMode)
		{
		case 0: // Direct Input
			{
				bUseTemperature = false;
				Color = CoronaSunColor;
				break;
			}
		case 1: // Kelvin temp
			{
				bUseTemperature = true;
				Temperature = CoronaSunTemperature;
				break;
			}
		case 2: // Realistic
			{
				// Use defaults
				break;
			}
		}
	}

	LightElement->SetIntensity( Intensity );
	LightElement->SetUseIes( false );
	LightElement->SetUseTemperature( bUseTemperature );
	LightElement->SetTemperature( Temperature );
	LightElement->SetColor( Color );
}

bool FDatasmithMaxSceneExporter::ParseVRayLightPortal(LightObject& Light, TSharedRef< IDatasmithLightmassPortalElement > LightPortalElement, TSharedRef< IDatasmithScene > DatasmithScene)
{
	IParamBlock2* ParamBlock2 = Light.GetParamBlockByID( (short)EVRayLightParamBlocks::Params );

	float VRayLightSize0 = ParamBlock2->GetFloat( (short)EVrayLightsParams::Size0 ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS );
	float VRayLightSize1 = ParamBlock2->GetFloat( (short)EVrayLightsParams::Size1 ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS );
	
	FVector CurrentScale = LightPortalElement->GetScale();
	LightPortalElement->SetScale( 10.f, VRayLightSize0 * CurrentScale.X, VRayLightSize1 * CurrentScale.Y);

	return true;
}

bool FDatasmithMaxSceneExporter::ParseVRayLightIES(DatasmithMaxDirectLink::FLightNodeConverter& Converter, LightObject& Light, TSharedRef< IDatasmithPointLightElement > PointLightElement, TSharedRef< IDatasmithScene > DatasmithScene)
{
	PointLightElement->SetUseIes( true );
	PointLightElement->SetUseIesBrightness( false ); // Vray IES lights have their own intensity

	if ( PointLightElement->IsA( EDatasmithElementType::AreaLight ) )
	{
		TSharedRef< IDatasmithAreaLightElement > AreaLightElement = StaticCastSharedRef< IDatasmithAreaLightElement >( PointLightElement );

		AreaLightElement->SetLightType( EDatasmithAreaLightType::IES_DEPRECATED );
	}

	const int NumParamBlocks = Light.NumParamBlocks();

	float IesRotateXAngle = 0.f;
	float IesRotateYAngle = 0.f;
	float IesRotateZAngle = 0.f;

	for ( int j = 0; j < NumParamBlocks; j++ )
	{
		IParamBlock2* ParamBlock2 = Light.GetParamBlockByID( (short)j );
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("intensity_type")) == 0 )
			{
				if ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) == 0 )
				{
					PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Lumens );
				}
				else
				{
					PointLightElement->SetIntensityUnits( EDatasmithLightUnits::Candelas );
				}
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("intensity_value")) == 0 )
			{
				PointLightElement->SetIntensity( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("ies_file")) == 0 )
			{
				Converter.ApplyIesProfile( *GetActualPath( ParamBlock2->GetStr( ParamDefinition.ID, GetCOREInterface()->GetTime() ) ) );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("rotation_X")) == 0 )
			{
				IesRotateXAngle = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("rotation_Y")) == 0 )
			{
				IesRotateYAngle = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("rotation_Z")) == 0 )
			{
				IesRotateZAngle = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("color_mode")) == 0 )
			{
				PointLightElement->SetUseTemperature( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("color_temperature")) == 0 )
			{
				PointLightElement->SetTemperature( ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) );
			}
		}

		ParamBlock2->ReleaseDesc();
	}

	if ( PointLightElement->GetUseTemperature() )
	{
		PointLightElement->SetColor( FLinearColor::White );
	}

	FQuat IesRotateX = FQuat( FVector::RightVector, FMath::DegreesToRadians( -IesRotateXAngle ) );
	FQuat IesRotateY = FQuat( FVector::UpVector, FMath::DegreesToRadians( IesRotateYAngle ) );
	FQuat IesRotateZ = FQuat( FVector::ForwardVector, FMath::DegreesToRadians( IesRotateZAngle ) );

	PointLightElement->SetIesRotation( IesRotateX * IesRotateY * IesRotateZ );

	if ( PointLightElement->IsA( EDatasmithElementType::AreaLight ) )
	{
		bool bUseLightShape = false;
		EDatasmithLightShape LightShape = EDatasmithLightShape::Disc;

		TSharedRef< IDatasmithAreaLightElement > AreaLightElement = StaticCastSharedRef< IDatasmithAreaLightElement >( PointLightElement );

		for ( int j = 0; j < NumParamBlocks; j++ )
		{
			IParamBlock2* ParamBlock2 = Light.GetParamBlockByID( (short)j );
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("override_shape")) == 0 )
				{
					bUseLightShape = ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0;
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("shape")) == 0 )
				{
					EVRayIESLightShapes VRayLightShape = (EVRayIESLightShapes)ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() );

					switch ( VRayLightShape )
					{
					case EVRayIESLightShapes::Rectangle:
						LightShape = EDatasmithLightShape::Rectangle;
						break;
					case EVRayIESLightShapes::Circle:
						LightShape = EDatasmithLightShape::Disc;
						break;
					case EVRayIESLightShapes::Sphere:
						LightShape = EDatasmithLightShape::Sphere;
						break;
					case EVRayIESLightShapes::VerticalCylinder:
						LightShape = EDatasmithLightShape::Cylinder;
						break;
					}
				}

				// We assume that we're parsing the shape type before the shape dimensions
				if ( LightShape == EDatasmithLightShape::Rectangle )
				{
					if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("width")) == 0 )
					{
						AreaLightElement->SetWidth( ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
					}
					else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("length")) == 0 )
					{
						AreaLightElement->SetLength( ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
					}
				}
				else
				{
					if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("diameter")) == 0 )
					{
						AreaLightElement->SetWidth( ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
						AreaLightElement->SetLength( ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
					}
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		if ( bUseLightShape  )
		{
			AreaLightElement->SetLightShape( LightShape );
		}
	}

	return true;
}

bool FDatasmithMaxSceneExporter::ParseLightParameters(DatasmithMaxDirectLink::FLightNodeConverter& Converter, EMaxLightClass LightClass, LightObject& Light, TSharedRef< IDatasmithLightActorElement > LightElement, TSharedRef< IDatasmithScene > DatasmithScene)
{
	const int NumParamBlocks = Light.NumParamBlocks();

	for ( int j = 0; j < NumParamBlocks; j++ )
	{
		IParamBlock2* ParamBlock2 = Light.GetParamBlockByID( (short)j );
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			// Arnold Light transform
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("shapeType")) == 0 && LightClass == EMaxLightClass::ArnoldLight)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 2 && LightElement->IsA( EDatasmithElementType::SpotLight ) )
				{
					TSharedPtr< IDatasmithSpotLightElement > SpotLightElement = StaticCastSharedRef< IDatasmithSpotLightElement >( LightElement );
					SpotLightElement->SetOuterConeAngle(160);
					SpotLightElement->SetInnerConeAngle(150);
				}
				else if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 7)
				{
					LightElement->SetUseIes( true );
					LightElement->SetUseIesBrightness(true);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("exposure")) == 0 && LightClass == EMaxLightClass::ArnoldLight)
			{
				float ArnoldLightExposure = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()); // Not used yet
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("penumbra_angle")) == 0 && LightClass == EMaxLightClass::ArnoldLight)
			{
				float ArnoldPenumbraAngle = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()); // Not used yet
			}
			// arnoldlight use temp?
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("useKelvin")) == 0 && LightClass == EMaxLightClass::ArnoldLight)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 1)
				{
					LightElement->SetUseTemperature( true );
				}
			}
			// thealight use temp?
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("luz_color_scale")) == 0 && LightClass == EMaxLightClass::TheaLightOmni)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 2)
				{
					LightElement->SetUseTemperature( true );
				}
			}
			// kelvin Scale
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("kelvin")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("color_temperature")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("blackbodyTemp")) == 0 ||
				FCString::Stricmp(ParamDefinition.int_name, TEXT("luz_kelvin")) == 0)
			{
				LightElement->SetTemperature( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap")) == 0)
			{
				Texmap* TextureMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (TextureMap != nullptr)
				{
					ProcessLightTexture( LightElement, TextureMap, DatasmithScene );
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("intensity")) == 0 && LightClass == EMaxLightClass::ArnoldLight)
			{
				float Intensity = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				LightElement->SetIntensity( Intensity );
			}

			// using ies file
			if ( LightElement->GetUseIes() )
			{
				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("multiplier")) == 0)
				{
					if (LightClass != EMaxLightClass::PhotometricLight)
					{
						LightElement->SetIesBrightnessScale( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
					}
					else
					{
						LightElement->SetIesBrightnessScale( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) / 100.0f );
					}
				}

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("webfile")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("IesFile")) == 0 || 
					FCString::Stricmp(ParamDefinition.int_name, TEXT("ies_file")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("filename")) == 0)
				{
					Converter.ApplyIesProfile( *GetActualPath(ParamBlock2->GetStr(ParamDefinition.ID, GetCOREInterface()->GetTime())) );
				}
			}

			if (LightElement->IsA( EDatasmithElementType::AreaLight ) || LightElement->IsA( EDatasmithElementType::LightmassPortal ) )
			{
				TSharedRef< IDatasmithPointLightElement > PointLightElement = StaticCastSharedRef< IDatasmithPointLightElement >( LightElement );

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("size0")) == 0)
				{
					PointLightElement->SetSourceRadius( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("width")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("light_Width")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("quadX")) == 0)
				{
					PointLightElement->SetSourceRadius( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) / 2.0f * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("size1")) == 0)
				{
					PointLightElement->SetSourceLength( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) * (float)GetSystemUnitScale( UNITS_CENTIMETERS ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("height")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("length")) == 0 ||
					FCString::Stricmp(ParamDefinition.int_name, TEXT("light_length")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("quadY")) == 0)
				{
					//Arnold has height parameter but it is not the right one we are looking for quadY
					if (LightClass != EMaxLightClass::ArnoldLight || FCString::Stricmp(ParamDefinition.int_name, TEXT("height")) != 0)
					{
						PointLightElement->SetSourceLength(ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) / 2.0f * (float)GetSystemUnitScale( UNITS_CENTIMETERS ));
					}
				}
			}
		}

		ParamBlock2->ReleaseDesc();
	}

	if ( LightElement->GetUseTemperature() )
	{
		LightElement->SetColor( FLinearColor::White );
	}

	return true;
}

bool FDatasmithMaxSceneExporter::ProcessLightTexture(TSharedRef< IDatasmithLightActorElement > LightElement, Texmap* LightTexture, TSharedRef< IDatasmithScene > DatasmithScene)
{
	if ( !LightTexture )
	{
		return false;
	}

	if ( LightElement->IsA( EDatasmithElementType::EnvironmentLight ) )
	{
		FString ElementName = LightElement->GetName();
		ElementName += TEXT("Background");
		TSharedRef< IDatasmithEnvironmentElement > EnvironmentBackground = FDatasmithSceneFactory::CreateEnvironment(*ElementName);
		FDatasmithMaxMatWriter::DumpTexture(DatasmithScene, EnvironmentBackground->GetEnvironmentComp(), LightTexture, DATASMITH_EMITTEXNAME, DATASMITH_EMITCOLNAME, false, false);

		if (EnvironmentBackground->GetEnvironmentComp()->GetParamSurfacesCount() == 1 && EnvironmentBackground->GetEnvironmentComp()->GetParamSurfacesCount() == 1)
		{
			EnvironmentBackground->SetIsIlluminationMap(false);
			EnvironmentBackground->GetEnvironmentComp()->GetParamTextureSampler(0).Multiplier = LightElement->GetIntensity();
			DatasmithScene->AddActor(EnvironmentBackground);

			ElementName = LightElement->GetName();
			ElementName += TEXT("Lighting");

			TSharedRef< IDatasmithEnvironmentElement > EnvironmentLighting = FDatasmithSceneFactory::CreateEnvironment(*ElementName);
			FDatasmithMaxMatWriter::DumpTexture(DatasmithScene, EnvironmentLighting->GetEnvironmentComp(), LightTexture, DATASMITH_EMITTEXNAME, DATASMITH_EMITCOLNAME, false, false);

			EnvironmentLighting->SetIsIlluminationMap(true);
			EnvironmentLighting->GetEnvironmentComp()->GetParamTextureSampler(0).Multiplier = LightElement->GetIntensity();
			DatasmithScene->AddActor(EnvironmentLighting);

			FString CheckTex = EnvironmentBackground->GetEnvironmentComp()->GetParamTexture(0);
			CheckTex += FDatasmithMaxMatWriter::TextureSuffix;

			for (int TextureIndex = 0; TextureIndex < DatasmithScene->GetTexturesCount(); TextureIndex++)
			{
				if (DatasmithScene->GetTexture(TextureIndex)->GetName() == CheckTex)
				{
					DatasmithScene->GetTexture(TextureIndex)->SetAllowResize(false);
				}
			}
		}
	}
	else
	{
		TSharedRef< IDatasmithMaterialElement > MaterialElement = FDatasmithSceneFactory::CreateMaterial(LightElement->GetName());
		TSharedRef< IDatasmithShaderElement > MaterialShader = FDatasmithSceneFactory::CreateShader(LightElement->GetName());

		FDatasmithMaxMatWriter::DumpTexture(DatasmithScene, MaterialShader->GetEmitComp(), LightTexture, DATASMITH_EMITTEXNAME, DATASMITH_EMITCOLNAME, false, false);
		MaterialShader->SetEmitPower(1.0);
		MaterialShader->SetUseEmissiveForDynamicAreaLighting( true );
		MaterialShader->SetShaderUsage( EDatasmithShaderUsage::LightFunction );
		MaterialElement->AddShader(MaterialShader);

		DatasmithScene->AddMaterial(MaterialElement);
		LightElement->SetLightFunctionMaterial(MaterialElement->GetName());
	}

	return true;
}

bool FDatasmithMaxSceneExporter::ExportActor(TSharedRef< IDatasmithScene > DatasmithScene, INode* Node, const TCHAR* Name, float UnitMultiplier)
{
	ObjectState ObjState = Node->EvalWorldState(0);
	if (ObjState.obj == nullptr)
	{
		return false;
	}

	TSharedRef< IDatasmithActorElement > ActorElement = FDatasmithSceneFactory::CreateActor(Name);

	DatasmithScene->AddActor(ActorElement);

	return ParseActor(Node, ActorElement, UnitMultiplier, DatasmithScene);
}

bool FDatasmithMaxSceneExporter::ExportCameraActor(TSharedRef< IDatasmithScene > DatasmithScene, INode* Parent, INodeTab Instances, int InstanceIndex, const TCHAR* Name, float UnitMultiplier)
{
	// weird behavior of 3dsmax it returns the head assembly as Instance of the sun
	ObjectState ObjState = Instances[InstanceIndex]->EvalWorldState(0);
	if (ObjState.obj == nullptr || Parent == nullptr)
	{
		return false;
	}

	if (ObjState.obj->SuperClassID() != CAMERA_CLASS_ID)
	{
		return false;
	}

	TSharedRef< IDatasmithCameraActorElement > CameraActor = FDatasmithSceneFactory::CreateCameraActor(Name);

	if ( !FDatasmithMaxCameraExporter::ExportCamera( GetCOREInterface()->GetTime(), *Parent, CameraActor ) )
	{
		return false;
	}

	if (InstanceIndex < 0 || InstanceIndex >= Instances.Count())
	{
		return false;
	}

	ParseActor(Instances[InstanceIndex], CameraActor, UnitMultiplier, DatasmithScene);

	FQuat Rotation = CameraActor->GetRotation();
	Rotation *= FQuat(0.0, 0.707107, 0.0, 0.707107);
	Rotation *= FQuat(0.707107, 0.0, 0.0, 0.707107);

	CameraActor->SetRotation(Rotation);

	DatasmithScene->AddActor(CameraActor);
	return true;
}

void FDatasmithMaxSceneExporter::WriteEnvironment(TSharedRef< IDatasmithScene > DatasmithScene, bool bOnlySelection)
{
	if (bOnlySelection || GetCOREInterface()->GetUseEnvironmentMap() == false)
	{
		return;
	}
	Texmap* TextureMap = GetCOREInterface()->GetEnvironmentMap();
	if (TextureMap != nullptr)
	{
		TSharedPtr< IDatasmithEnvironmentElement > EnvironmentBackground = FDatasmithSceneFactory::CreateEnvironment(TEXT("EnvironmentBackground"));
		FDatasmithMaxMatWriter::DumpTexture(DatasmithScene, EnvironmentBackground->GetEnvironmentComp(), TextureMap, DATASMITH_EMITTEXNAME, DATASMITH_EMITCOLNAME, false, false);

		if (EnvironmentBackground->GetEnvironmentComp()->GetParamSurfacesCount() == 1 && EnvironmentBackground->GetEnvironmentComp()->GetParamSurfacesCount() == 1)
		{
			EnvironmentBackground->SetIsIlluminationMap(false);
			EnvironmentBackground->GetEnvironmentComp()->GetParamTextureSampler(0).Multiplier = 1.0f;
			DatasmithScene->AddActor(EnvironmentBackground);

			TSharedPtr< IDatasmithEnvironmentElement > EnvironmentLighting = FDatasmithSceneFactory::CreateEnvironment(TEXT("EnvironmentLighting"));
			FDatasmithMaxMatWriter::DumpTexture(DatasmithScene, EnvironmentLighting->GetEnvironmentComp(), TextureMap, DATASMITH_EMITTEXNAME, DATASMITH_EMITCOLNAME, false, false);
			EnvironmentLighting->SetIsIlluminationMap(true);
			EnvironmentLighting->GetEnvironmentComp()->GetParamTextureSampler(0).Multiplier = 1.0f;

			DatasmithScene->AddActor(EnvironmentLighting);

			FString CheckTex = EnvironmentBackground->GetEnvironmentComp()->GetParamTexture(0);
			CheckTex += FDatasmithMaxMatWriter::TextureSuffix;

			for (int TextureIndex = 0; TextureIndex < DatasmithScene->GetTexturesCount(); TextureIndex++)
			{
				if (DatasmithScene->GetTexture(TextureIndex)->GetName() == CheckTex)
				{
					DatasmithScene->GetTexture(TextureIndex)->SetAllowResize(false);
				}
			}
		}
	}
}

void FDatasmithMaxSceneExporter::ExportToneOperator(TSharedRef< IDatasmithScene > DatasmithScene)
{
	ToneOperatorInterface* ToneOpInterface = static_cast<ToneOperatorInterface*>( GetCOREInterface(TONE_OPERATOR_INTERFACE) );

	if ( !ToneOpInterface )
	{
		return;
	}

	ToneOperator* ToneOp = ToneOpInterface->GetToneOperator();

	if ( !ToneOp )
	{
		return;
	}

	TSharedRef< IDatasmithPostProcessVolumeElement > PostProcessVolume = FDatasmithSceneFactory::CreatePostProcessVolume( TEXT("Global Exposure") );
	PostProcessVolume->SetLabel( TEXT("Global Exposure") );

	if ( FDatasmithMaxCameraExporter::ExportToneOperator( *ToneOp, PostProcessVolume ) )
	{
		DatasmithScene->AddActor( PostProcessVolume );
	}
}
