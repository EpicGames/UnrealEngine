// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxSceneHelper.h"

#include "DatasmithMaxLogger.h"

#include "Windows/AllowWindowsPlatformTypes.h"

MAX_INCLUDES_START
#include "lslights.h"
#include "xref/iXrefObj.h"
MAX_INCLUDES_END

const TArray< FString, TInlineAllocator< 4 > > FDatasmithMaxSceneHelper::CollisionNodesPrefixes = { TEXT("UBX"), TEXT("UCX"), TEXT("UCP"), TEXT("USP") };

EMaxLightClass FDatasmithMaxSceneHelper::GetLightClass(INode* Node)
{
	EMaxLightClass LightClass = EMaxLightClass::Unknown;
	ObjectState ObjState = Node->EvalWorldState(0);

	if (ObjState.obj->SuperClassID() == LIGHT_CLASS_ID)
	{
		LightObject* Light = (LightObject*)ObjState.obj;
		if (Light != nullptr)
		{
			Class_ID ClassID = Light->ClassID();
			if (ClassID == THEAOMNICLASS)
			{
				LightClass = EMaxLightClass::TheaLightOmni;
			}
			else if (ClassID == THEASPOTCLASS)
			{
				LightClass = EMaxLightClass::TheaLightSpot;
			}
			else if (ClassID == THEAIESCLASS)
			{
				LightClass = EMaxLightClass::TheaLightIES;
			}
			else if (ClassID == THEAPLANECLASS)
			{
				LightClass = EMaxLightClass::TheaLightPlane;
			}
			else if (ClassID == REGULARSPOTACLASS || ClassID == REGULARSPOTBCLASS)
			{
				LightClass = EMaxLightClass::SpotLight;
			}
			else if (ClassID == REGULARDIRECTACLASS || ClassID == REGULARDIRECTBCLASS)
			{
				LightClass = EMaxLightClass::DirectLight;

				INode *ParentNode = Node->GetParentNode();
				if (ParentNode)
				{
					ObjectState ParentObjectState = ParentNode->EvalWorldState(0);
					if (ParentObjectState.obj && ParentObjectState.obj->ClassID() == DAYLIGHTASSEMBLYCLASS)
					{
						LightClass = EMaxLightClass::SunEquivalent;
					}
				}
			}
			else if (ClassID == REGULAROMNICLASS)
			{
				LightClass = EMaxLightClass::OmniLight;
			}
			else if (Light->IsSubClassOf(LIGHTSCAPE_LIGHT_CLASS))
			{
				LightClass = EMaxLightClass::PhotometricLight;
			}
			else if (ClassID == PHOTOPLANEACLASS || ClassID == PHOTOPLANEBCLASS)
			{
				LightClass = EMaxLightClass::PhotoplaneLight;
			}
			else if (ClassID == VRAYLIGHTIESCLASS)
			{
				LightClass = EMaxLightClass::VRayLightIES;
			}
			else if (ClassID == VRAYLIGHTCLASS)
			{
				LightClass = EMaxLightClass::VRayLight;
			}
			else if (ClassID == CORONALIGHTCLASS)
			{
				LightClass = EMaxLightClass::CoronaLight;
			}
			else if (ClassID == ARNOLDLIGHTCLASS)
			{
				LightClass = EMaxLightClass::ArnoldLight;
			}
			else if (ClassID == SUNLIGHTACLASS ||
				ClassID == SUNLIGHTBCLASS ||
				ClassID == VRAYSUNCLASS ||
				ClassID == SUNPOSITIONERCLASS ||
				ClassID == CORONASUNCLASS ||
				ClassID == CORONASUNCLASSB)
			{
				LightClass = EMaxLightClass::SunEquivalent;
			}
			else if (ClassID == SKYLIGHTCLASS ||
				ClassID == IESSKYLIGHTCLASS ||
				ClassID == MRSKYLIGHTCLASS)
			{
				LightClass = EMaxLightClass::SkyEquivalent;
			}
			else if (ClassID == MRPORTALLIGHTCLASS)
			{
				LightClass = EMaxLightClass::SkyPortal;
			}
		}
	}

	if (LightClass == EMaxLightClass::Unknown)
	{
		DatasmithMaxLogger::Get().AddUnsupportedLight(Node);
	}

	return LightClass;
}

bool FDatasmithMaxSceneHelper::CanBeTriMesh(Object* Obj)
{
	if (Obj == NULL)
	{
		return false;
	}

	int CanConvertToTri = Obj->CanConvertToType(Class_ID(TRIOBJ_CLASS_ID, 0));
	if (CanConvertToTri == 0)
	{
		return false;
	}

	TriObject* Tri = (TriObject*)Obj->ConvertToType(GetCOREInterface()->GetTime(), Class_ID(TRIOBJ_CLASS_ID, 0));
	if (Tri == NULL)
	{
		return false;
	}

	bool bResult = true;
	if (Tri->GetMesh().getNumFaces() == 0)
	{
		bResult = false;
	}

	if (Obj != Tri)
	{
		Tri->DeleteMe();
	}

	return bResult;
}

bool FDatasmithMaxSceneHelper::HasCollisionName(INode* Node)
{
	FString NodeName = Node->GetName();
	FString LeftString, RightString;

	if ( NodeName.Split( TEXT("_"), &LeftString, &RightString ) )
	{
		if ( CollisionNodesPrefixes.Find( LeftString ) != INDEX_NONE )
		{
			return true;
		}
	}
	return false;
}

#include "Windows/HideWindowsPlatformTypes.h"
