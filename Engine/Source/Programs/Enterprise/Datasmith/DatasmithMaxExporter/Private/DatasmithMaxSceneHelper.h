// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxExporterDefines.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"

#include "Templates/SharedPointer.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "max.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

class FMaxRendereableNode;
class FDatasmithMaxStaticMeshAttributes;

// Dummy function for calls to ProgressStart in the Max API
DWORD WINAPI DummyFunction(LPVOID Arg);

// This tell the exporter how to manage the instancing of a Geometry
enum class EMaxExporterInstanceMode
{
	NotInstanced,
	InstanceMaster,
	InstanceCopy,
	UnrealHISM
};

enum class EMaxTriMode
{
	TriNotChecked,
	TriEnabled,
	TriDisabled
};

enum class EMaxEntityType
{
	Geometry,
	Light,
	Camera,
	Dummy
};

enum class EMaxLightClass
{
	Unknown,
	TheaLightOmni,
	TheaLightSpot,
	TheaLightIES,
	TheaLightPlane,
	SpotLight,
	DirectLight,
	OmniLight,
	PhotometricLight,
	PhotoplaneLight,
	VRayLightIES,
	VRayLight,
	CoronaLight,
	ArnoldLight,
	SunEquivalent,
	SkyEquivalent,
	SkyPortal
};

class FDatasmithMaxSceneHelper
{
public:
	static EMaxLightClass GetLightClass(INode* InNode);
	static bool CanBeTriMesh(Object* Obj);
	static const TArray< FString, TInlineAllocator< 4 > > CollisionNodesPrefixes; // List of supported mesh prefixes for Unreal collision

	static bool HasCollisionName(INode* Node);
};
