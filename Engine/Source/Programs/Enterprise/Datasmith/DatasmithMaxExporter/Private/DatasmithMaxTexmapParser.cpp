// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxTexmapParser.h"

#include "DatasmithMaxExporterDefines.h"
#include "DatasmithMaxLogger.h"
#include "DatasmithMaxSceneExporter.h"
#include "DatasmithMaxWriter.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "bitmap.h"
	#include "iparamb2.h"
	#include "max.h"
	#include "mtl.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

namespace DatasmithMaxTexmapParser
{
	FCompositeTexmapParameters ParseCompositeTexmap( Texmap* InTexmap )
	{
		FCompositeTexmapParameters CompositeParameters;
		TimeValue CurrentTime = GetCOREInterface()->GetTime();

		const int NumParamBlocks = InTexmap->NumParamBlocks();

		for ( int j = 0; j < NumParamBlocks; j++ )
		{
			IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for ( int i = 0; i < ParamBlockDesc->count; i++ )
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapEnabled")) == 0)
				{
					for (int s = 0; s < ParamBlock2->Count(ParamDefinition.ID); s++)
					{
						FCompositeTexmapParameters::FLayer& Layer = CompositeParameters.Layers.AddDefaulted_GetRef();

						Layer.Map.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime, s ) != 0 );
					}
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		for ( int j = 0; j < NumParamBlocks; j++ )
		{
			IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for ( int i = 0; i < ParamBlockDesc->count; i++ )
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("opacity") ) == 0 )
				{
					for ( int s = 0; s < ParamBlock2->Count(ParamDefinition.ID); s++ )
					{
						FCompositeTexmapParameters::FLayer& Layer = CompositeParameters.Layers[s];
						Layer.Map.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime, s ) / 100.0f;
					}
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("mapList") ) == 0 )
				{
					for ( int s = 0; s < ParamBlock2->Count(ParamDefinition.ID); s++ )
					{
						FCompositeTexmapParameters::FLayer& Layer = CompositeParameters.Layers[s];
						Layer.Map.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime, s );
					}
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("Mask") ) == 0 )
				{
					for ( int s = 0; s < ParamBlock2->Count(ParamDefinition.ID); s++ )
					{
						FCompositeTexmapParameters::FLayer& Layer = CompositeParameters.Layers[s];

						Layer.Mask.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime, s );
						Layer.Mask.bEnabled = ( Layer.Mask.Map != nullptr );
						Layer.Mask.Weight = 1.f;
					}
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("blendmode") ) == 0 )
				{
					for ( int s = 0; s < ParamBlock2->Count(ParamDefinition.ID); s++ )
					{
						FCompositeTexmapParameters::FLayer& Layer = CompositeParameters.Layers[s];

						switch ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime, s ) )
						{
						case 1:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Average;
							break;
						case 2:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Add;
							break;
						case 3:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Sub;
							break;
						case 4:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Darken;
							break;
						case 5:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Mult;
							break;
						case 6:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Burn;
							break;
						case 7:
							Layer.CompositeMode = EDatasmithCompositeCompMode::LinearBurn;
							break;
						case 8:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Lighten;
							break;
						case 9:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Screen;
							break;
						case 10:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Dodge;
							break;
						case 11:
							Layer.CompositeMode = EDatasmithCompositeCompMode::LinearDodge;
							break;
						case 14:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Overlay;
							break;
						case 15:
							Layer.CompositeMode = EDatasmithCompositeCompMode::SoftLight;
							break;
						case 16:
							Layer.CompositeMode = EDatasmithCompositeCompMode::HardLight;
							break;
						case 17:
							Layer.CompositeMode = EDatasmithCompositeCompMode::PinLight;
							break;
						case 19:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Difference;
							break;
						case 20:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Exclusion;
							break;
						case 21:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Hue;
							break;
						case 22:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Saturation;
							break;
						case 23:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Color;
							break;
						case 24:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Value;
							break;
						default:
							Layer.CompositeMode = EDatasmithCompositeCompMode::Alpha;
							break;
						}
					}
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		return CompositeParameters;
	}

	FNormalMapParameters ParseNormalMap( Texmap* InTexmap )
	{
		FNormalMapParameters NormalMapParameters;

		for (int j = 0; j < InTexmap->NumParamBlocks(); j++)
		{
			IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];
				if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("flip_green") ) == 0 || FCString::Stricmp( ParamDefinition.int_name, TEXT("flipgreen") ) == 0 )
				{
					NormalMapParameters.bFlipGreen = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("flip_red") ) == 0 || FCString::Stricmp( ParamDefinition.int_name, TEXT("flipred") ) == 0 )
				{
					NormalMapParameters.bFlipRed = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("swap_red_and_green") ) == 0 || FCString::Stricmp( ParamDefinition.int_name, TEXT("swap_rg") ) == 0 )
				{
					NormalMapParameters.bSwapRedAndGreen = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("normal_map") ) == 0 )
				{
					NormalMapParameters.NormalMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime() );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("normal_map_on") ) == 0 || FCString::Stricmp( ParamDefinition.int_name, TEXT("map1on") ) == 0 )
				{
					NormalMapParameters.NormalMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("normal_map_multiplier") ) == 0 || FCString::Stricmp( ParamDefinition.int_name, TEXT("mult_spin") ) == 0 )
				{
					NormalMapParameters.NormalMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("bump_map") ) == 0 )
				{
					NormalMapParameters.BumpMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime() );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("bump_map_on") ) == 0 || FCString::Stricmp( ParamDefinition.int_name, TEXT("map2on") ) == 0 )
				{
					NormalMapParameters.BumpMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("bump_map_multiplier") ) == 0 || FCString::Stricmp( ParamDefinition.int_name, TEXT("bump_spin") ) == 0 )
				{
					NormalMapParameters.BumpMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() );
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		return NormalMapParameters;
	}

	FAutodeskBitmapParameters ParseAutodeskBitmap(Texmap* InTexmap)
	{
		FAutodeskBitmapParameters AutodeskBitmapParameters;

		TimeValue CurrentTime = GetCOREInterface()->GetTime();

		const int NumParamBlocks = InTexmap->NumParamBlocks();
		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Parameters_Source")) == 0)
				{
					AutodeskBitmapParameters.SourceFile = (ParamBlock2->GetBitmap(ParamDefinition.ID, CurrentTime));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Parameters_Brightness")) == 0)
				{
					AutodeskBitmapParameters.Brightness = (ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Parameters_Invert_Image")) == 0)
				{
					AutodeskBitmapParameters.bInvertImage = (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Position_X")) == 0)
				{
					AutodeskBitmapParameters.Position.X = (ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Position_Y")) == 0)
				{
					AutodeskBitmapParameters.Position.Y = (ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Position_Rotation")) == 0)
				{
					AutodeskBitmapParameters.Rotation = (ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Scale_Width")) == 0)
				{
					AutodeskBitmapParameters.Scale.X = (ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Scale_Height")) == 0)
				{
					AutodeskBitmapParameters.Scale.Y = (ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Repeat_Horizontal")) == 0)
				{
					AutodeskBitmapParameters.bRepeatHorizontal = (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Repeat_Vertical")) == 0)
				{
					AutodeskBitmapParameters.bRepeatVertical = (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Advanced_Parameters_Blur")) == 0)
				{
					AutodeskBitmapParameters.BlurValue = (ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Advanced_Parameters_Blur_Offset")) == 0)
				{
					AutodeskBitmapParameters.BlurOffset = (ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Advanced_Parameters_Filtering")) == 0)
				{
					AutodeskBitmapParameters.FilteringValue = (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Advanced_Parameters_Map_Channel")) == 0)
				{
					AutodeskBitmapParameters.MapChannel = (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime));
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		return AutodeskBitmapParameters;
	}

	FCoronaBitmapParameters ParseCoronaBitmap(Texmap* InTexmap)
	{
		FCoronaBitmapParameters CoronaBitmapParameters;

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		const int NumParamBlocks = InTexmap->NumParamBlocks();
		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("filename")) == 0)
				{
					CoronaBitmapParameters.Path = FDatasmithMaxSceneExporter::GetActualPath(ParamBlock2->GetStr(ParamDefinition.ID, GetCOREInterface()->GetTime()));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("uvwScale")) == 0)
				{
					Point3 Point = ParamBlock2->GetPoint3(ParamDefinition.ID,CurrentTime);
					CoronaBitmapParameters.TileU = Point.x;
					CoronaBitmapParameters.TileV = Point.y;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("uvwOffset")) == 0)
				{
					Point3 Point = ParamBlock2->GetPoint3(ParamDefinition.ID, CurrentTime);
					CoronaBitmapParameters.OffsetU = Point.x;
					CoronaBitmapParameters.OffsetV = Point.y;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("uvwAngle")) == 0)
				{
					Point3 Point = ParamBlock2->GetPoint3(ParamDefinition.ID, CurrentTime);
					CoronaBitmapParameters.RotW = Point.z;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("uvwChannel")) == 0)
				{
					CoronaBitmapParameters.UVCoordinate = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) - 1;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("tilingU")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) == 2)
					{
						CoronaBitmapParameters.MirrorU = 2;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("tilingV")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) == 2)
					{
						CoronaBitmapParameters.MirrorV = 2;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("gamma")) == 0)
				{
					CoronaBitmapParameters.Gamma = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		return CoronaBitmapParameters;
	}

	FCoronaColorParameters ParseCoronaColor(Texmap* InTexmap)
	{
		FCoronaColorParameters CoronaColorParameters;

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		for ( int ParamBlockIndex = 0; ParamBlockIndex < InTexmap->NumParamBlocks(); ++ParamBlockIndex )
		{
			IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)ParamBlockIndex);
			const ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for ( int ParamIndex = 0; ParamIndex < ParamBlockDesc->count; ++ParamIndex )
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[ParamIndex];

				if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("color")) == 0 )
				{
					CoronaColorParameters.RgbColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, CurrentTime) );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("ColorHdr")) == 0 )
				{
					Point3 ColorHdr = ParamBlock2->GetPoint3(ParamDefinition.ID, CurrentTime);
					CoronaColorParameters.ColorHdr = FVector{ ColorHdr.x, ColorHdr.y, ColorHdr.z };
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("Multiplier")) == 0 )
				{
					CoronaColorParameters.Multiplier = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("Temperature")) == 0 )
				{
					CoronaColorParameters.Temperature = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("Method")) == 0 )
				{
					CoronaColorParameters.Method = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime);
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("inputIsLinear")) == 0 )
				{
					CoronaColorParameters.bInputIsLinear = (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0);
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("HexColor")) == 0 )
				{
					CoronaColorParameters.HexColor = ParamBlock2->GetStr(ParamDefinition.ID, CurrentTime);
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		return CoronaColorParameters;
	}

	FColorCorrectionParameters ParseColorCorrection(Texmap* InTexmap)
	{
		const TimeValue CurrentTime = GetCOREInterface()->GetTime();
		FColorCorrectionParameters ColorCorrectionParameters;

		for (int j = 0; j < InTexmap->NumParamBlocks(); j++)
		{
			IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map")) == 0)
				{
					ColorCorrectionParameters.TextureSlot1 = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("color")) == 0)
				{
					ColorCorrectionParameters.Color1 = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, CurrentTime) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Tint")) == 0)
				{
					ColorCorrectionParameters.Tint = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, CurrentTime) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("HueShift")) == 0)
				{
					ColorCorrectionParameters.HueShift = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime) / 360.0f;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Saturation")) == 0)
				{
					ColorCorrectionParameters.Saturation = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime) / 100.0f;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("LiftRGB")) == 0)
				{
					ColorCorrectionParameters.LiftRGB = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Brightness")) == 0)
				{
					ColorCorrectionParameters.Brightness = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime) / 100.0f;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("GammaRGB")) == 0)
				{
					ColorCorrectionParameters.GammaRGB = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Contrast")) == 0)
				{
					ColorCorrectionParameters.Contrast = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime) / 100.0f;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("TintStrength")) == 0)
				{
					ColorCorrectionParameters.TintStrength = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime) / 100.0f;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("LightnessMode")) == 0)
				{
					ColorCorrectionParameters.bAdvancedLightnessMode = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("RewireR")) == 0)
				{
					ColorCorrectionParameters.RewireR = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("RewireG")) == 0)
				{
					ColorCorrectionParameters.RewireG = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("RewireB")) == 0)
				{
					ColorCorrectionParameters.RewireB = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bEnableR")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0)
					{
						ColorCorrectionParameters.bEnableR = true;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bEnableG")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0)
					{
						ColorCorrectionParameters.bEnableG = true;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bEnableB")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0)
					{
						ColorCorrectionParameters.bEnableB = true;
					}
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		if (!(ColorCorrectionParameters.RewireR == 0 && ColorCorrectionParameters.RewireG == 1 && ColorCorrectionParameters.RewireB == 2) &&
			!(ColorCorrectionParameters.RewireR == 8 && ColorCorrectionParameters.RewireG == 8 && ColorCorrectionParameters.RewireB == 8) &&
			!(ColorCorrectionParameters.RewireR == 4 && ColorCorrectionParameters.RewireG == 5 && ColorCorrectionParameters.RewireB == 6))
		{
			ColorCorrectionParameters.RewireR = 0;
			ColorCorrectionParameters.RewireG = 1;
			ColorCorrectionParameters.RewireB = 2;
			DatasmithMaxLogger::Get().AddGeneralError(TEXT("Color correct cannot use different settings per channel"));
		}

		if (ColorCorrectionParameters.bEnableR || ColorCorrectionParameters.bEnableG || ColorCorrectionParameters.bEnableB)
		{
			DatasmithMaxLogger::Get().AddGeneralError(TEXT("Color correct cannot use different settings per channel"));
		}

		return ColorCorrectionParameters;
	}
}