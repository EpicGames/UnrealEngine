// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DatasmithDefinitions.h"

#include "Containers/Array.h"

class Texmap;
class PBBitmap;

namespace DatasmithMaxTexmapParser
{
	struct FMapParameter
	{
		FMapParameter()
			: Map( nullptr )
			, bEnabled( true )
			, Weight( 1.f )
		{
		}

		bool IsMapPresentAndEnabled()
		{
			return Map && bEnabled;
		}

		Texmap* Map;
		bool bEnabled;
		float Weight;
	};

	struct FWeightedColorParameter
	{
		FWeightedColorParameter()
			: Value( FLinearColor::White )
			, Weight( 1.f )
		{
		}

		FLinearColor Value;
		float Weight;
	};

	struct FCompositeTexmapParameters
	{
		struct FLayer
		{
			FLayer()
				: CompositeMode( EDatasmithCompositeCompMode::Alpha )
			{
			}

			EDatasmithCompositeCompMode CompositeMode;

			FMapParameter Map;
			FMapParameter Mask;
		};

		TArray< FLayer > Layers;
	};

	FCompositeTexmapParameters ParseCompositeTexmap( Texmap* InTexmap );

	struct FNormalMapParameters
	{
		FMapParameter NormalMap;
		FMapParameter BumpMap;

		bool bFlipGreen = false;
		bool bFlipRed = false;
		bool bSwapRedAndGreen = false;
	};

	FNormalMapParameters ParseNormalMap( Texmap* InTexmap );

	struct FAutodeskBitmapParameters
	{
		PBBitmap* SourceFile = nullptr;
		float Brightness = 1;
		bool bInvertImage = false;
		FVector2D Position = FVector2D(0,0);
		float Rotation = 0;
		FVector2D Scale = FVector2D(1, 1);
		bool bRepeatHorizontal = true;
		bool bRepeatVertical = true;
		float BlurValue = 0;
		float BlurOffset = 0;
		float FilteringValue = 0;
		int MapChannel = 1;
	};

	FAutodeskBitmapParameters ParseAutodeskBitmap(Texmap* InTexmap);

	struct FCoronaBitmapParameters
	{
		FString Path;
		float TileU = 1.0f;
		float TileV = 1.0f;
		float OffsetU = 1.0f;
		float OffsetV = 1.0f;
		float RotW = 0.f;
		int UVCoordinate = 0;
		int MirrorU = 0;
		int MirrorV = 0;
		float Gamma = 1.0f;
	};

	FCoronaBitmapParameters ParseCoronaBitmap(Texmap* InTexmap);

	struct FCoronaColorParameters
	{
		FLinearColor RgbColor = FLinearColor::Black;
		FVector ColorHdr = FVector::ZeroVector;
		float Multiplier = 1.f;
		float Temperature = 6500.f;
		int Method = 0;
		FString HexColor;
		bool bInputIsLinear = false;
	};

	FCoronaColorParameters ParseCoronaColor(Texmap* InTexmap);

	struct FColorCorrectionParameters
	{
		Texmap* TextureSlot1 = nullptr;
		FLinearColor Color1;
		FLinearColor Tint;

		float HueShift = 0.f;
		float Saturation = 0.f;
		float LiftRGB = 0.f;
		float Brightness = 0.f;
		float GammaRGB = 0.f;
		float Contrast = 0.f;
		float TintStrength = 0.f;

		int RewireR = 0;
		int RewireG = 0;
		int RewireB = 0;
		bool bAdvancedLightnessMode = false;

		bool bEnableR = false;
		bool bEnableG = false;
		bool bEnableB = false;
	};

	FColorCorrectionParameters ParseColorCorrection(Texmap* InTexmap);
}