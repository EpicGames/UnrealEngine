// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxWriter.h"

#include "DatasmithExportOptions.h"
#include "DatasmithMaxSceneHelper.h"

FString FDatasmithMaxMatWriter::DumpBitmapThea(TSharedPtr<IDatasmithCompositeTexture>& CompTex, BitmapTex* InBitmapTex, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale)
{
	BitmapTex* Texture = (BitmapTex*)InBitmapTex->GetReference(0);
	return DumpBitmap(CompTex, Texture, Prefix, bForceInvert, bIsGrayscale);
}

void FDatasmithMaxMatWriter::ExportTheaSubmaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithShaderElement >& MaterialShader, Mtl* Material, EDSMaterialType MaterialType)
{
	int NumParamBlocks = Material->NumParamBlocks();

	bool bDiffuseTexEnable = false;
	bool bReflectanceTexEnable = false;
	bool bReflectance90TexEnable = false;
	bool bTranslucentTexEnable = false;
	bool bReflectanceEnable = false;
	bool bTransmittanceTexEnable = false;
	bool bAbsorptionTexEnable = false;
	bool bNormalEnable = false;

	float BumpAmount = 0.f;
	float Roughness = 0.1f;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("diffuse_tex")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) != NULL)
				{
					bDiffuseTexEnable = true;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectance_tex")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) != NULL)
				{
					bReflectanceEnable = true;
					bReflectanceTexEnable = true;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectance90_tex")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) != NULL)
				{
					bReflectance90TexEnable = true;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("translucent_tex")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) != NULL)
				{
					bTranslucentTexEnable = true;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("transmittance_tex")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) != NULL)
				{
					bTransmittanceTexEnable = true;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_absorption_tex")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) != NULL)
				{
					bAbsorptionTexEnable = true;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectance")) == 0)
			{
				BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (!(Color.r == 0 && Color.g == 0 && Color.b == 0))
				{
					bReflectanceEnable = true;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("normal")) == 0)
			{
				bNormalEnable = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump")) == 0)
			{
				BumpAmount = (float)ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Roughness")) == 0)
			{
				Roughness = (float)ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) / 100.0f;
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (MaterialType == EDSMaterialType::TheaBasic)
			{
				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("use_custom_curve")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("fresnel_curve_values")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trace_refl")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("absorption_val")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("IOR")) == 0)
				{
					MaterialShader->SetIOR( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
					MaterialShader->SetIORRefra( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("extinction")) == 0)
				{
					MaterialShader->SetIORk( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("sigma")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("anisotropy")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("micro_on")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("micro_height")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("micro_width")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("aniso_rot")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("diffuse_tex")) == 0 && bDiffuseTexEnable == true)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					DumpTexture(DatasmithScene, MaterialShader->GetDiffuseComp(), LocalTex, DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Diffuse")) == 0 && bDiffuseTexEnable == false)
				{
					BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());

					// compensation since thea color picker is not affected by colorGamma
					Color.r = pow(Color.r, FDatasmithExportOptions::ColorGamma);
					Color.g = pow(Color.g, FDatasmithExportOptions::ColorGamma);
					Color.b = pow(Color.b, FDatasmithExportOptions::ColorGamma);
					MaterialShader->GetDiffuseComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectance_tex")) == 0 && bReflectanceTexEnable == true)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					DumpTexture(DatasmithScene, MaterialShader->GetRefleComp(), LocalTex, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectance")) == 0 && bReflectanceTexEnable == false)
				{
					BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());

					// compensation since thea color picker is not affected by colorGamma
					Color.r = pow(Color.r, FDatasmithExportOptions::ColorGamma);
					Color.g = pow(Color.g, FDatasmithExportOptions::ColorGamma);
					Color.b = pow(Color.b, FDatasmithExportOptions::ColorGamma);
					if (Color.r != 0 || Color.g != 0 || Color.b != 0)
					{
						MaterialShader->GetRefleComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("translucent_tex")) == 0 && bTranslucentTexEnable == true)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Translucent")) == 0 && bTranslucentTexEnable == false)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectance90_tex")) == 0 && bReflectance90TexEnable == true && bReflectanceEnable == true)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectance90")) == 0 && bReflectance90TexEnable == false && bReflectanceEnable == true)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("absorption_col")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("sigma_tex")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughness_tex")) == 0)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					if (LocalTex)
					{
						if (Roughness == 1)
						{
							DumpTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, false, true);
						}
						else
						{
							DumpWeightedTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), Roughness, DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, false, true);
						}
					}
					else
					{
						// already readed roughnessvalue
						if (!bReflectanceEnable)
						{
							MaterialShader->GetRoughnessComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(0.75f, TEXT("Roughness")) );
						}
						else
						{
							MaterialShader->GetRoughnessComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(Roughness, TEXT("Roughness")) );
						}
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("anisotropy_tex")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("aniso_rot_tex")) == 0)
				{
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_tex")) == 0)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					if (BumpAmount > 0 && LocalTex != NULL)
					{
						MaterialShader->SetBumpAmount( sqrt(BumpAmount / 100.0f) );
						if (bNormalEnable)
						{
							DumpNormalTexture(DatasmithScene, MaterialShader->GetNormalComp(), LocalTex, DATASMITH_NORMALTEXNAME, DATASMITH_NORMALTEXNAME, false, false);
						}
						else
						{
							if (FDatasmithMaxMatHelper::GetTextureClass(LocalTex) == EDSBitmapType::NormalMap)
							{
								DumpTexture(DatasmithScene, MaterialShader->GetNormalComp(), LocalTex, DATASMITH_NORMALTEXNAME, DATASMITH_NORMALTEXNAME, false, false);
								DumpTexture(DatasmithScene, MaterialShader->GetBumpComp(), LocalTex, DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
							}
							else
							{
								DumpTexture(DatasmithScene, MaterialShader->GetBumpComp(), LocalTex, DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
							}
						}
					}
				}
			}
			// glossy
			else if (MaterialType == EDSMaterialType::TheaGlossy)
			{
				Texmap* GlossyTransTex = NULL;
				BMM_Color_fl GlossyTransColor = BMM_Color_fl(0, 0, 0, 0);
				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("transmittance_tex")) == 0 && bTransmittanceTexEnable == true)
				{
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					GlossyTransTex = LocalTex;
					DumpTexture(DatasmithScene, MaterialShader->GetTransComp(), LocalTex, DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, false, false);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("transmittance")) == 0 && bTransmittanceTexEnable == false)
				{
					BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());

					// compensation since thea color picker is not affected by colorGamma
					Color.r = pow(Color.r, FDatasmithExportOptions::ColorGamma);
					Color.g = pow(Color.g, FDatasmithExportOptions::ColorGamma);
					Color.b = pow(Color.b, FDatasmithExportOptions::ColorGamma);
					GlossyTransColor = Color;
					MaterialShader->GetTransComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughnessTr_on")) == 0)
				{
					// mBsdf.setBooleanParameter("Rough Tr.",(bool)ParamBlock2->GetInt(ParamDefinition.ID,GetCOREInterface()->GetTime()));
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughnessTr")) == 0)
				{
					// mBsdf.setRealParameter("Roughness Tr.",(float)ParamBlock2->GetInt(ParamDefinition.ID,GetCOREInterface()->GetTime())/100.0f);
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughnessTr_tex")) == 0)
				{
					/*Texmap* LocalTex =ParamBlock2->GetTexmap(ParamDefinition.ID,GetCOREInterface()->GetTime());
					addCustomTexture(LocalTex,mBsdf,"./Roughness Tr. Map/");*/
					continue;
				}

				if (GlossyTransTex == NULL && GlossyTransColor.r == 0 && GlossyTransColor.g == 0 && GlossyTransColor.b == 0)
				{
					MaterialShader->SetMetal(1.0);
				}
			}
			// COATING
			else if (MaterialType == EDSMaterialType::TheaCoating)
			{
				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_absorption_col")) == 0 && bAbsorptionTexEnable == false)
				{
					/*BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
					if (!(Color.r == 0 && Color.g == 0 && Color.b == 0))
					{
					TheaSDK::Texture mTex = (TheaSDK::Texture)mBsdf.addObject("./Absorption Map/", "Constant Texture", "");
					mTex.setRgbParameter("Color", TheaSDK::Rgb(Color.r, Color.g, Color.b));
					}*/
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_absorption_tex")) == 0 && bAbsorptionTexEnable == true)
				{
					/*Texmap* LocalTex =ParamBlock2->GetTexmap(ParamDefinition.ID,GetCOREInterface()->GetTime());
					addCustomTexture(LocalTex,mBsdf,"./Absorption Map/");*/
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("thickness_tex")) == 0)
				{
					/*Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					addCustomTexture(LocalTex, mBsdf, "./Thickness Map/");*/
					continue;
				}
			}
			// THIN FILM
			else if (MaterialType == EDSMaterialType::TheaFilm)
			{
				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("thin_bump")) == 0)
				{
					// mBsdf.setRealParameter("Bump Strength",(float)ParamBlock2->GetInt(ParamDefinition.ID,GetCOREInterface()->GetTime())/100.0f);
					continue;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("thin_normal")) == 0)
				{
					// mBsdf.setBooleanParameter("Normal Mapping",(bool)ParamBlock2->GetInt(ParamDefinition.ID,GetCOREInterface()->GetTime()));
					continue;
				}
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	if (MaterialType == EDSMaterialType::TheaFilm)
	{
		BMM_Color_fl Color(1.0f, 1.0f, 1.0f, 1.0f);
		MaterialShader->GetRefleComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
		MaterialShader->SetIORRefra(1.05);
	}
}

void FDatasmithMaxMatWriter::ExportTheaMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	TSharedPtr< IDatasmithShaderElement > MaterialShader = FDatasmithSceneFactory::CreateShader((TCHAR*)Material->GetName().data());

	int NumParamBlocks = Material->NumParamBlocks();
	Mtl* SubMaterial = NULL;
	bool bClippingEnabled;

	Mtl* LayerMtl[4] = {NULL, NULL, NULL, NULL};
	Mtl* StackedMtl[4] = {NULL, NULL, NULL, NULL};

	Texmap* LayerMtlMask[4] = {NULL, NULL, NULL, NULL};
	Texmap* StackedMtlMask[4] = {NULL, NULL, NULL, NULL};

	float LayerMtlWeight[4] = {0, 0, 0, 0};
	float StackedMtlWeight[4] = {0, 0, 0, 0};

	bool bClipEnabled = false;
	Texmap* ClipTex = NULL;
	bool bSoftClip = false;
	bool bClipByAmount = false;
	float ClipAmount = 1.0;

	bool bEmitEnabled = false;
	int EmitMode = 0;
	BMM_Color_fl EmitColor = 0;
	Texmap* EmitTex = 0;
	int EmitKelvin = 0;
	int EmitUnit = 0;
	float EmitPower = 0;

	// read layered (horizontal) and stacked (vertical) materials
	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clip_enabled")) == 0)
			{
				bClippingEnabled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("materialList")) == 0)
			{
				LayerMtl[0] = ParamBlock2->GetMtl(ParamDefinition.ID, GetCOREInterface()->GetTime(), 0);
				LayerMtl[1] = ParamBlock2->GetMtl(ParamDefinition.ID, GetCOREInterface()->GetTime(), 1);
				LayerMtl[2] = ParamBlock2->GetMtl(ParamDefinition.ID, GetCOREInterface()->GetTime(), 2);
				LayerMtl[3] = ParamBlock2->GetMtl(ParamDefinition.ID, GetCOREInterface()->GetTime(), 3);
				StackedMtl[0] = ParamBlock2->GetMtl(ParamDefinition.ID, GetCOREInterface()->GetTime(), 4);
				StackedMtl[1] = ParamBlock2->GetMtl(ParamDefinition.ID, GetCOREInterface()->GetTime(), 5);
				StackedMtl[2] = ParamBlock2->GetMtl(ParamDefinition.ID, GetCOREInterface()->GetTime(), 6);
				StackedMtl[3] = ParamBlock2->GetMtl(ParamDefinition.ID, GetCOREInterface()->GetTime(), 7);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("maskList")) == 0)
			{
				LayerMtlMask[0] = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 0);
				LayerMtlMask[1] = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 1);
				LayerMtlMask[2] = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 2);
				LayerMtlMask[3] = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 3);
				StackedMtlMask[0] = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 4);
				StackedMtlMask[1] = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 5);
				StackedMtlMask[2] = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 6);
				StackedMtlMask[3] = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), 7);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("amountList")) == 0)
			{
				LayerMtlWeight[0] = float(ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), 0)) / 100.0f;
				LayerMtlWeight[1] = float(ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), 1)) / 100.0f;
				LayerMtlWeight[2] = float(ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), 2)) / 100.0f;
				LayerMtlWeight[3] = float(ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), 3)) / 100.0f;
				StackedMtlWeight[0] = float(ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), 4)) / 100.0f;
				StackedMtlWeight[1] = float(ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), 5)) / 100.0f;
				StackedMtlWeight[2] = float(ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), 6)) / 100.0f;
				StackedMtlWeight[3] = float(ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), 7)) / 100.0f;
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clip_enabled")) == 0)
			{
				bClipEnabled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clip_tex")) == 0)
			{
				ClipTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("soft")) == 0)
			{
				bSoftClip = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("visibility")) == 0)
			{
				bClipByAmount = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("visibilityValue")) == 0)
			{
				ClipAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_enabled")) == 0)
			{
				bEmitEnabled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_color_scale")) == 0)
			{
				EmitMode = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("EmitColor")) == 0)
			{
				EmitColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_tex")) == 0)
			{
				EmitTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_kelvin")) == 0)
			{
				EmitKelvin = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("unit")) == 0)
			{
				EmitUnit = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("power")) == 0)
			{
				EmitPower = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	if (bClipEnabled)
	{
		if (ClipTex)
		{
			if (bSoftClip)
			{
				DumpTexture(DatasmithScene, MaterialShader->GetMaskComp(), ClipTex, DATASMITH_CLIPTEXNAME, DATASMITH_CLIPTEXNAME, false, true);
			}
			else
			{
				DumpTexture(DatasmithScene, MaterialShader->GetTransComp(), ClipTex, DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, false, false);
			}
		}
		else
		{
			if (bClipByAmount)
			{
				float ColorValue = 1.0f - ClipAmount;
				BMM_Color_fl Color(ColorValue, ColorValue, ColorValue, 1.0f);
				// compensation since thea color picker is not affected by colorGamma
				Color.r = pow(Color.r, FDatasmithExportOptions::ColorGamma);
				Color.g = pow(Color.g, FDatasmithExportOptions::ColorGamma);
				Color.b = pow(Color.b, FDatasmithExportOptions::ColorGamma);
				MaterialShader->GetTransComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
			}
		}
	}

	if (bEmitEnabled)
	{
		if (EmitMode == 0)
		{
			if (EmitTex)
			{
				DumpTexture(DatasmithScene, MaterialShader->GetEmitComp(), EmitTex, DATASMITH_EMITTEXNAME, DATASMITH_EMITCOLNAME, false, false);
			}
			else
			{
				// compensation since thea color picker is not affected by colorGamma
				EmitColor.r = pow(EmitColor.r, FDatasmithExportOptions::ColorGamma);
				EmitColor.g = pow(EmitColor.g, FDatasmithExportOptions::ColorGamma);
				EmitColor.b = pow(EmitColor.b, FDatasmithExportOptions::ColorGamma);
				MaterialShader->GetEmitComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(EmitColor));
			}
		}
		else
		{
			MaterialShader->SetEmitTemperature( (float)EmitKelvin );
		}
	}

	TArray< TSharedPtr< IDatasmithShaderElement > > AllShaders;
	AllShaders.Add(MaterialShader);
	// Export the first valid layered material
	for (int i = 3; i >= 0; i--)
	{
		if (LayerMtl[i] != NULL)
		{
			if (FDatasmithMaxMatHelper::GetMaterialClass(LayerMtl[i]) == EDSMaterialType::TheaBasic || FDatasmithMaxMatHelper::GetMaterialClass(LayerMtl[i]) == EDSMaterialType::TheaGlossy || FDatasmithMaxMatHelper::GetMaterialClass(LayerMtl[i]) == EDSMaterialType::TheaFilm)
			{
				SubMaterial = LayerMtl[i];
			}
		}
	}

	if (SubMaterial != NULL)
	{

		// DatasmithShaderElement MaterialShader((TCHAR*)Material->GetName().data());
		switch (FDatasmithMaxMatHelper::GetMaterialClass(SubMaterial))
		{
		case EDSMaterialType::TheaBasic:
			ExportTheaSubmaterial(DatasmithScene, AllShaders.Last(), SubMaterial, EDSMaterialType::TheaBasic);
			break;
		case EDSMaterialType::TheaGlossy:
			ExportTheaSubmaterial(DatasmithScene, AllShaders.Last(), SubMaterial, EDSMaterialType::TheaGlossy);
			break;
		case EDSMaterialType::TheaFilm:
			ExportTheaSubmaterial(DatasmithScene, AllShaders.Last(), SubMaterial, EDSMaterialType::TheaFilm);
			break;
		}
	}

	// Export the stacked material, coating is a special case

	int CoatingLayer = -1;
	for (int i = 0; i < 4; i++)
	{
		if (StackedMtl[i] != NULL)
		{
			if (FDatasmithMaxMatHelper::GetMaterialClass(StackedMtl[i]) == EDSMaterialType::TheaCoating)
			{
				CoatingLayer = i;
			}
			else
			{
				TSharedPtr< IDatasmithShaderElement > MatShaderStacked = FDatasmithSceneFactory::CreateShader((TCHAR*)StackedMtl[i]->GetName().data());
				AllShaders.Add(MatShaderStacked);

				switch (FDatasmithMaxMatHelper::GetMaterialClass(StackedMtl[i]))
				{
				case EDSMaterialType::TheaBasic:
					ExportTheaSubmaterial(DatasmithScene, AllShaders.Last(), StackedMtl[i], EDSMaterialType::TheaBasic);
					break;
				case EDSMaterialType::TheaGlossy:
					ExportTheaSubmaterial(DatasmithScene, AllShaders.Last(), StackedMtl[i], EDSMaterialType::TheaGlossy);
					break;
				case EDSMaterialType::TheaFilm:
					ExportTheaSubmaterial(DatasmithScene, AllShaders.Last(), StackedMtl[i], EDSMaterialType::TheaFilm);
					break;
				}

				AllShaders.Last()->SetBlendMode( EDatasmithBlendMode::Alpha );
				AllShaders.Last()->SetIsStackedLayer(true);

				if (StackedMtlMask[i])
				{
					DumpTexture(DatasmithScene, AllShaders.Last()->GetWeightComp(), StackedMtlMask[i], DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
				}
				else
				{
					AllShaders.Last()->GetWeightComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(StackedMtlWeight[i], TEXT("StackedMatWeight")) );
				}
			}
		}
	}

	// if reflection is only on the coating we put it on the base
	if (CoatingLayer != -1 && AllShaders.Num() == 1 && AllShaders[0]->GetRefleComp()->GetParamSurfacesCount() == 0)
	{
		ExportTheaSubmaterial(DatasmithScene, AllShaders[0], StackedMtl[CoatingLayer], EDSMaterialType::TheaCoating); // overwrite some parameters like reflection
	}
	else
	{
		if (CoatingLayer != -1)
		{
			TSharedPtr< IDatasmithShaderElement > MatShaderStacked = FDatasmithSceneFactory::CreateShader((TCHAR*)StackedMtl[CoatingLayer]->GetName().data());
			AllShaders.Add(MatShaderStacked);

			ExportTheaSubmaterial(DatasmithScene, AllShaders.Last(), StackedMtl[CoatingLayer], EDSMaterialType::TheaCoating);

			AllShaders.Last()->SetBlendMode( EDatasmithBlendMode::ClearCoat );
			AllShaders.Last()->SetIsStackedLayer(true);

			if (StackedMtlMask[CoatingLayer])
			{
				DumpTexture(DatasmithScene, AllShaders.Last()->GetWeightComp(), StackedMtlMask[CoatingLayer], DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
			}
			else
			{
				AllShaders.Last()->GetWeightComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(StackedMtlWeight[CoatingLayer], TEXT("StackedWeight")) );
			}
		}
	}

	for (int i = 1; i < AllShaders.Num(); i++)
	{
		AllShaders[i]->SetEmitComp( AllShaders[0]->GetEmitComp() );
		AllShaders[i]->SetEmitPower( AllShaders[0]->GetEmitPower() );
		AllShaders[i]->SetEmitTemperature( AllShaders[0]->GetEmitTemperature() );

		AllShaders[i]->SetMaskComp( AllShaders[0]->GetMaskComp() );
	}

	for (int i = 0; i < AllShaders.Num(); i++)
	{
		MaterialElement->AddShader(AllShaders[i]);
	}
}
