// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxWriter.h"
#include "DatasmithMaxSceneHelper.h"
#include "DatasmithMaxSceneExporter.h"

#include "Misc/Paths.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "gamma.h"
	#include "stdmat.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

FString FDatasmithMaxMatWriter::GetActualVRayBitmapName(BitmapTex* InBitmapTex)
{
	FString Path;

	int NumParamBlocks = InBitmapTex->NumParamBlocks();

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InBitmapTex->GetParamBlockByID((short)j);
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("HDRIMapName")) == 0)
			{
				Path = FDatasmithMaxSceneExporter::GetActualPath(ParamBlock2->GetStr(ParamDefinition.ID, GetCOREInterface()->GetTime()));
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	float Gamma = FDatasmithMaxMatHelper::GetVrayHdriGamma(InBitmapTex);
	FString OrigBase = FPaths::GetBaseFilename(Path) + FString("_") + FString::SanitizeFloat(Gamma).Replace(TEXT("."), TEXT("_"));
	FString Base = FDatasmithUtils::SanitizeObjectName( OrigBase + TextureSuffix );

	return Base;
}

float FDatasmithMaxMatHelper::GetVrayHdriGamma(BitmapTex* InBitmapTex)
{
	const int NumParamBlocks = InBitmapTex->NumParamBlocks();

	int ColorSpace = 0;
	float Gamma = -1;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InBitmapTex->GetParamBlockByID((short)j);
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("color_space")) == 0)
			{
				ColorSpace = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("gamma")) == 0)
			{
				Gamma = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	switch (ColorSpace)
	{
	case 0:
		Gamma = 1.0f;
		break;
	case 2:
		Gamma = 2.2f;
		break;
	case 3:
		Gamma = 0.0f;
		break;
	default:
		break;
	}

	return Gamma;
}

FString FDatasmithMaxMatWriter::DumpVrayHdri(TSharedPtr<IDatasmithCompositeTexture>& CompTex, BitmapTex* InBitmapTex, const TCHAR* Prefix, bool bForceInvert)
{
	float VrayMultiplier = 1.0f;

	int NumParamBlocks = InBitmapTex->NumParamBlocks();

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InBitmapTex->GetParamBlockByID((short)j);
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("renderMultiplier")) == 0)
			{
				VrayMultiplier = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	FString BitmapName = GetActualVRayBitmapName( InBitmapTex );

	FDatasmithTextureSampler TextureSampler;
	TextureSampler.Multiplier = VrayMultiplier;

	CompTex->AddSurface(*BitmapName, TextureSampler);

	return *BitmapName;
}

void FDatasmithMaxMatWriter::ExportVRayMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	TSharedPtr< IDatasmithShaderElement > MaterialShader = FDatasmithSceneFactory::CreateShader( (TCHAR*)Material->GetName().data() );

	int NumParamBlocks = Material->NumParamBlocks();

	bool bDiffuseTexEnable = true;
	bool bReflectanceTexEnable = true;
	bool bRefractTexEnable = true;
	bool bEmitTexEnable = true;
	bool bGlossyTexEnable = true;
	bool bGlossyHTexEnable = true;
	bool bBumpTexEnable = true;
	bool bOpacityTexEnable = true;
	bool bLockIor = true;
	bool bRefleFresnel = true;

	float BumpAmount = 0.f;

	BMM_Color_fl ColorDiffuse;
	BMM_Color_fl ColorReflection;
	BMM_Color_fl ColorRefraction;

	float DiffuseTexAmount = 0.0f;
	float reflectanceTexAmount = 0.0f;
	float RefractTexAmount = 0.0f;
	float GlossyTexAmount = 0.0f;
	float LevelGlossy = 1.0f;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_diffuse")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bDiffuseTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflection")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bReflectanceTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refraction")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bRefractTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_opacity")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bOpacityTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflectionGlossiness")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bGlossyTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_hilightGlossiness")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bGlossyHTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_bump")) == 0)
			{
				if (ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime()) == NULL)
				{
					bBumpTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_bump_multiplier")) == 0)
			{
				BumpAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_diffuse_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bDiffuseTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_diffuse_multiplier")) == 0)
			{
				DiffuseTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) / 100.0f;
				if (DiffuseTexAmount == 0)
				{
					bDiffuseTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflection_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bReflectanceTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflection_multiplier")) == 0)
			{
				reflectanceTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) / 100.0f;
				if (reflectanceTexAmount == 0)
				{
					bReflectanceTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refraction_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bRefractTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refraction_multiplier")) == 0)
			{
				RefractTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) / 100.0f;
				if (RefractTexAmount == 0)
				{
					bRefractTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflectionGlossiness_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bGlossyTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflectionGlossiness_multiplier")) == 0)
			{
				GlossyTexAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) / 100.0f;
				if (GlossyTexAmount == 0)
				{
					bGlossyTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection_glossiness")) == 0)
			{
				float Glossy = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				Glossy = 0.8f * Glossy + 0.2f * Glossy * Glossy; // vray has a more pronunciated curve on glossiness
				if (Glossy < LevelGlossy)
				{
					LevelGlossy = Glossy;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("hilight_glossiness")) == 0)
			{
				float Glossy = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				Glossy = 0.8f * Glossy + 0.2f * Glossy * Glossy; // vray has a more pronunciated curve on glossiness
				Glossy *= Glossy;
				if (Glossy < LevelGlossy)
				{
					LevelGlossy = Glossy;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_hilightGlossiness_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bGlossyHTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_bump_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bBumpTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_opacity_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bOpacityTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_self_illumination_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bEmitTexEnable = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection_lockIOR")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bLockIor = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection_fresnel")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bRefleFresnel = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("diffuse")) == 0)
			{
				ColorDiffuse = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection")) == 0)
			{
				ColorReflection = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refraction")) == 0)
			{
				ColorRefraction = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refraction_ior")) == 0)
			{
				MaterialShader->SetIORRefra( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
				if (bLockIor == true && bRefleFresnel == true)
				{
					MaterialShader->SetIOR( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection_ior")) == 0)
			{
				if (bRefleFresnel)
				{
					if (bLockIor == false)
					{
						MaterialShader->SetIOR( ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) );
					}
				}
				else
				{
					MaterialShader->SetIOR(0.0);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_diffuse")) == 0 && bDiffuseTexEnable == true)
			{
				Texmap* DiffuseTexMap = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (DiffuseTexAmount == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetDiffuseComp(), DiffuseTexMap, DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
				}
				else
				{
					DumpWeightedTexture(DatasmithScene, MaterialShader->GetDiffuseComp(), DiffuseTexMap, ColorDiffuse, DiffuseTexAmount, DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("diffuse")) == 0 && bDiffuseTexEnable == false)
			{
				MaterialShader->GetDiffuseComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(ColorDiffuse));
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflection")) == 0 && bReflectanceTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (reflectanceTexAmount == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetRefleComp(), LocalTex, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
				}
				else
				{
					DumpWeightedTexture(DatasmithScene, MaterialShader->GetRefleComp(), LocalTex, ColorReflection, reflectanceTexAmount, DATASMITH_REFLETEXNAME, DATASMITH_REFLECOLNAME, false, false);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection")) == 0 && bReflectanceTexEnable == false)
			{
				if (ColorReflection.r > 0 || ColorReflection.g > 0 || ColorReflection.b > 0)
				{
					MaterialShader->GetRefleComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(ColorReflection));
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflectionGlossiness")) == 0 && bGlossyTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (LevelGlossy == 1 && GlossyTexAmount == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, true, true);
				}
				else
				{
					DumpWeightedTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, BMM_Color_fl(1.0f - LevelGlossy, 1.0f - LevelGlossy, 1.0f - LevelGlossy, 1.0f - LevelGlossy), GlossyTexAmount,
						DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, true, true);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_hilightGlossiness")) == 0 && bGlossyHTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (LevelGlossy == 1 && GlossyTexAmount == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, true, false);
				}
				else
				{
					DumpWeightedTexture(DatasmithScene, MaterialShader->GetRoughnessComp(), LocalTex, BMM_Color_fl(1.0f - LevelGlossy, 1.0f - LevelGlossy, 1.0f - LevelGlossy, 1.0f - LevelGlossy), GlossyTexAmount,
						DATASMITH_ROUGHNESSTEXNAME, DATASMITH_ROUGHNESSVALUENAME, true, false);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection_glossiness")) == 0 && bGlossyTexEnable == false && bGlossyHTexEnable == false)
			{
				if (bReflectanceTexEnable == false && ColorReflection.r == 0 && ColorReflection.g == 0 && ColorReflection.b == 0)
				{
					MaterialShader->GetRoughnessComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(0.75f, TEXT("Roughness")) );
				}
				else
				{
					MaterialShader->GetRoughnessComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(1.0f - LevelGlossy, TEXT("Roughness")) );
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refraction")) == 0 && bRefractTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (RefractTexAmount == 1)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetTransComp(), LocalTex, DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, false, false);
				}
				else
				{
					DumpWeightedTexture(DatasmithScene, MaterialShader->GetTransComp(), LocalTex, ColorRefraction, RefractTexAmount, DATASMITH_TRANSPTEXNAME, DATASMITH_TRANSPCOLNAME, false, false);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refraction")) == 0 && bRefractTexEnable == false)
			{
				if (ColorRefraction.r > 0 || ColorRefraction.g > 0 || ColorRefraction.b > 0)
				{
					MaterialShader->GetTransComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(ColorRefraction));
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_opacity")) == 0 && bOpacityTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				DumpTexture(DatasmithScene, MaterialShader->GetMaskComp(), LocalTex, DATASMITH_CLIPTEXNAME, DATASMITH_CLIPTEXNAME, false, true);
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_self_illumination")) == 0 && bEmitTexEnable == true)
			{
				Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				if (LocalTex)
				{
					DumpTexture(DatasmithScene, MaterialShader->GetEmitComp(), LocalTex, DATASMITH_EMITTEXNAME, DATASMITH_EMITTEXNAME, false, false);
					MaterialShader->SetEmitPower( 100.0 );
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_bump")) == 0)
			{
				if (bBumpTexEnable == true && BumpAmount > 0)
				{
					MaterialShader->SetBumpAmount( BumpAmount / 100.0f );
					Texmap* LocalTex = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					if (FDatasmithMaxMatHelper::GetTextureClass(LocalTex) == EDSBitmapType::NormalMap)
					{
						DumpTexture(DatasmithScene, MaterialShader->GetNormalComp(), LocalTex, DATASMITH_NORMALTEXNAME, DATASMITH_NORMALTEXNAME, false, false);
						DumpTexture(DatasmithScene, MaterialShader->GetBumpComp(), LocalTex, DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
					}
					else
					{
						DumpTexture(DatasmithScene, MaterialShader->GetBumpComp(), LocalTex, DATASMITH_BUMPTEXNAME, DATASMITH_BUMPTEXNAME, false, true);
					}
				}
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	MaterialElement->AddShader(MaterialShader);
}

void FDatasmithMaxMatWriter::ExportVRayLightMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	TSharedPtr< IDatasmithShaderElement > MaterialShader = FDatasmithSceneFactory::CreateShader((TCHAR*)Material->GetName().data());

	int NumParamBlocks = Material->NumParamBlocks();

	bool bTexEnabled = true;
	bool bClipTexEnabled = true;
	Texmap* EmitTexture = NULL;
	Texmap* ClipTexture = NULL;
	BMM_Color_fl EmitColor;

	float Multiplier = 1.0f;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap")) == 0)
			{
				EmitTexture = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacity_texmap")) == 0)
			{
				ClipTexture = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bTexEnabled = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacity_texmap_on")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
				{
					bClipTexEnabled = false;
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Color")) == 0)
			{
				EmitColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("twoSided")) == 0)
			{
				// int twoSided = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Multiplier")) == 0)
			{
				Multiplier = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	if (bTexEnabled && EmitTexture != NULL)
	{
		DumpTexture(DatasmithScene, MaterialShader->GetEmitComp(), EmitTexture, DATASMITH_EMITTEXNAME, DATASMITH_EMITCOLNAME, false, false);
	}
	else
	{
		MaterialShader->GetEmitComp()->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(EmitColor));
	}

	if (bClipTexEnabled && ClipTexture != NULL)
	{
		DumpTexture(DatasmithScene, MaterialShader->GetMaskComp(), ClipTexture, DATASMITH_CLIPTEXNAME, DATASMITH_CLIPTEXNAME, false, true);
	}

	MaterialShader->SetLightOnly(true);
	MaterialShader->SetUseEmissiveForDynamicAreaLighting(true);
	MaterialShader->SetEmitPower(Multiplier);

	MaterialElement->AddShader(MaterialShader);
}

void FDatasmithMaxMatWriter::ExportVrayBlendMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material)
{
	FDatasmithMaxMatExport::WriteXMLMaterial(DatasmithScene, MaterialElement, Material->GetSubMtl(0));

	int NumParamBlocks = Material->NumParamBlocks();

	bool bCoatEnabled[9];
	Mtl* CoatMaterials[9];
	Texmap* Mask[9];
	float MixAmount[9];
	BMM_Color_fl MixColor[9];

	for (int i = 0; i < 9; i++)
	{
		bCoatEnabled[i] = true;
		CoatMaterials[i] = NULL;
		Mask[i] = NULL;
		MixAmount[i] = 0.5f;
	}

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coatMtl")) == 0)
			{
				for (int s = 0; s < 9; s++)
				{
					CoatMaterials[s] = ParamBlock2->GetMtl(ParamDefinition.ID, GetCOREInterface()->GetTime(), s);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_blend")) == 0)
			{
				for (int s = 0; s < 9; s++)
				{
					Mask[s] = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime(), s);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coatMtl_enable")) == 0)
			{
				for (int s = 0; s < 9; s++)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime(), s) == 0)
					{
						bCoatEnabled[s] = false;
					}
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_blend_multiplier")) == 0)
			{
				for (int s = 0; s < 9; s++)
				{
					MixAmount[s] = (float)ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime(), s);
				}
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Blend")) == 0)
			{
				for (int s = 0; s < 9; s++)
				{
					MixColor[s] = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime(), s);
				}
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	for (int s = 0; s < 9; s++)
	{
		if (CoatMaterials[s] != NULL && bCoatEnabled[s])
		{
			FDatasmithMaxMatExport::WriteXMLMaterial(DatasmithScene, MaterialElement, CoatMaterials[s]);

			TSharedPtr< IDatasmithShaderElement >& Shader = MaterialElement->GetShader( MaterialElement->GetShadersCount() - 1 );

			Shader->SetBlendMode(EDatasmithBlendMode::Alpha);
			Shader->SetIsStackedLayer(true);

			if (Mask[s])
			{
				DumpTexture(DatasmithScene, Shader->GetWeightComp(), Mask[s], DATASMITH_DIFFUSETEXNAME, DATASMITH_DIFFUSECOLNAME, false, false);
				if (abs((MixAmount[s] / 100.0f) - 1.0) > 0.00001)
				{
					Shader->GetWeightComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(MixAmount[s] / 100.0f, TEXT("MixAmount")) );
				}
			}
			else
			{
				MixColor[s].r *= MixAmount[s] / 100.0f;
				MixColor[s].g *= MixAmount[s] / 100.0f;
				MixColor[s].b *= MixAmount[s] / 100.0f;
				float Val = (MixColor[s].r + MixColor[s].g + MixColor[s].b) / 3.0f;
				Shader->GetWeightComp()->AddParamVal1( IDatasmithCompositeTexture::ParamVal(Val, TEXT("MixAmountByCol")) );
			}
		}
	}
}

FString FDatasmithMaxMatWriter::DumpVrayColor(TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert)
{
	FString ActualPrefix = FString(Prefix) + FString(TEXT("comp"));

	int NumParamBlocks = InTexmap->NumParamBlocks();

	int GammaCorrection = 1;
	BMM_Color_fl Color;
	float ColorGamma = 1.f;
	float GammaValue = 1.f;
	float RgbMultiplier = 1.f;

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Color")) == 0)
			{
				Color = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("RgbMultiplier")) == 0)
			{
				RgbMultiplier = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("ColorGamma")) == 0)
			{
				ColorGamma = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("GammaCorrection")) == 0)
			{
				GammaCorrection = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("GammaValue")) == 0)
			{
				GammaValue = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	switch (GammaCorrection)
	{
	case 1:
		GammaValue = 1.0f / GammaValue;
		break;
	case 2:
		if (gammaMgr.IsEnabled())
		{
			GammaValue = 1.0f / gammaMgr.GetDisplayGamma();
		}
		else
		{
			GammaValue = 1.0f;
		}
		break;
	default:
		GammaValue = 1.0f;
		break;
	}

	Color.r = pow(pow(Color.r * RgbMultiplier, ColorGamma), GammaValue);
	Color.g = pow(pow(Color.g * RgbMultiplier, ColorGamma), GammaValue);
	Color.b = pow(pow(Color.b * RgbMultiplier, ColorGamma), GammaValue);

	FString Result = TEXT("");

	CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));

	return Result;
}

