// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithMaxWriter.h"

#include "DatasmithMaxClassIDs.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxMaterialsToUEPbr.h"
#include "DatasmithExportOptions.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithMaxSceneHelper.h"
#include "DatasmithMaxLogger.h"
#include "Misc/Paths.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "bitmap.h"
	#include "xref/iXrefMaterial.h"
MAX_INCLUDES_END 
#include "Windows/HideWindowsPlatformTypes.h"

FString FDatasmithMaxMatWriter::TextureSuffix = TEXT("_Tex");
FString FDatasmithMaxMatWriter::TextureBakeFormat = TEXT(".tga");

bool FDatasmithMaxMatHelper::HasNonBakeableSubmap(Texmap* InTexmap)
{
	if (InTexmap == NULL)
	{
		return false;
	}

	// Skip any view dependent TexMap
	// Having a THIRDPARTYMULTITEXCLASS as an input for a color correction node crashes when calling RenderBitmap on the color correction texmap so skip it.
	if (InTexmap->ClassID() == VRAYDIRTCLASS || InTexmap->ClassID() == CORONAAOCLASS || InTexmap->ClassID() == FALLOFFCLASS || InTexmap->ClassID() == THIRDPARTYMULTITEXCLASS)
	{
		return true;
	}

	if (InTexmap->ClassID() == RBITMAPCLASS)
	{
		StdUVGen* UV = ((BitmapTex*)InTexmap)->GetUVGen();
		if (UV == NULL)
		{
			return true;
		}
		float U = UV->GetUScl(GetCOREInterface()->GetTime());
		float V = UV->GetVScl(GetCOREInterface()->GetTime());
		float IntegralPart;
		if (U > 10.f || V > 10.f || FMath::IsNearlyEqual(FMath::Modf(U, &IntegralPart), 0.f) == false || FMath::IsNearlyEqual(FMath::Modf(V, &IntegralPart), 0.f) == false)
		{
			return true;
		}
	}

	for (int SubMap = 0; SubMap < InTexmap->NumSubTexmaps(); SubMap++)
	{
		if (InTexmap->SubTexmapOn(SubMap) != 0 && HasNonBakeableSubmap(InTexmap->GetSubTexmap(SubMap)) == true)
		{
			return true;
		}
	}

	return false;
}

EDSBitmapType FDatasmithMaxMatHelper::GetTextureClass(Texmap* InTexMap)
{
	EDSBitmapType Type = EDSBitmapType::NotSupported;

	if (InTexMap != nullptr)
	{
		MSTR ClassName;
		InTexMap->GetClassName(ClassName);

		if (InTexMap->ClassID() == RBITMAPCLASS)
		{
			Type = EDSBitmapType::RegularBitmap;
		}
		//Somehow, there are multiple autodesk map classes using the same ClassID, we only support Autodesk Bitmap.
		else if (InTexMap->ClassID() == AUTODESKBITMAPCLASS && FCString::Stricmp(ClassName, TEXT("Autodesk Bitmap")) == 0)
		{
			Type = EDSBitmapType::AutodeskBitmap;
		}
		else if (InTexMap->ClassID() == THEABITMAPCLASS)
		{
			Type = EDSBitmapType::TheaBitmap;
		}
		else if (InTexMap->ClassID() == REGULARNORMALCLASS)
		{
			Type = EDSBitmapType::NormalMap;
		}
		else if (InTexMap->ClassID() == VRAYNORMALCLASS)
		{
			Type = EDSBitmapType::NormalMap;
		}
		else if (InTexMap->ClassID() == CORONANORMALCLASS)
		{
			Type = EDSBitmapType::NormalMap;
		}
		else if (InTexMap->ClassID() == COLORCORRECTCLASS)
		{
			if (FDatasmithMaxMatHelper::HasNonBakeableSubmap(InTexMap) == false)
			{
				Type = EDSBitmapType::BakeableMap;
			}
			else
			{
				Type = EDSBitmapType::ColorCorrector;
			}
		}
		else if (InTexMap->ClassID() == FALLOFFCLASS)
		{
			Type = EDSBitmapType::FallOff;
		}
		else if (InTexMap->ClassID() == MIXCLASS)
		{
			if (FDatasmithMaxMatHelper::HasNonBakeableSubmap(InTexMap) == false)
			{
				Type = EDSBitmapType::BakeableMap;
			}
			else
			{
				Type = EDSBitmapType::Mix;
			}
		}
		else if (InTexMap->ClassID() == NOISECLASS)
		{
			Type = EDSBitmapType::Noise;
		}
		else if (InTexMap->ClassID() == GRADIENTCLASS)
		{
			if (FDatasmithMaxMatHelper::HasNonBakeableSubmap(InTexMap) == false)
			{
				Type = EDSBitmapType::BakeableMap;
			}
			else
			{
				Type = EDSBitmapType::Gradient;
			}
		}
		else if (InTexMap->ClassID() == GRADIENTRAMPCLASS)
		{
			if (FDatasmithMaxMatHelper::HasNonBakeableSubmap(InTexMap) == false)
			{
				Type = EDSBitmapType::BakeableMap;
			}
		}
		else if (InTexMap->ClassID() == CHECKERCLASS)
		{
			if (FDatasmithMaxMatHelper::HasNonBakeableSubmap(InTexMap) == false)
			{
				Type = EDSBitmapType::BakeableMap;
			}
			else
			{
				Type = EDSBitmapType::Checker;
			}
		}
		else if (InTexMap->ClassID() == CELLULARCLASS)
		{
			Type = EDSBitmapType::Cellular;
		}
		else if (InTexMap->ClassID() == VRAYDIRTCLASS)
		{
			Type = EDSBitmapType::VRayDirt;
		}
		else if (InTexMap->ClassID() == COMPOSITETEXCLASS)
		{
			Type = EDSBitmapType::CompositeTex;
		}
		else if (InTexMap->ClassID() == CORONABITMAPCLASS)
		{
			Type = EDSBitmapType::CoronaBitmap;
		}
		else if (InTexMap->ClassID() == VRAYHDRICLASS)
		{
			Type = EDSBitmapType::VRayHRDI;
		}
		else if (InTexMap->ClassID() == CORONACOLORCLASS)
		{
			Type = EDSBitmapType::CoronaColor;
		}
		else if (InTexMap->ClassID() == CORONAMIXCLASS)
		{
			Type = EDSBitmapType::CoronaMix;
		}
		else if (InTexMap->ClassID() == CORONAMULTITEXCLASS)
		{
			Type = EDSBitmapType::CoronaMultiTex;
		}
		else if (InTexMap->ClassID() == CORONAAOCLASS)
		{
			Type = EDSBitmapType::CoronaAO;
		}
		else if (InTexMap->ClassID() == VRAYCOLORCLASS)
		{
			Type = EDSBitmapType::VRayColor;
		}
		else if (InTexMap->ClassID() == THIRDPARTYMULTITEXCLASS)
		{
			Type = EDSBitmapType::ThirdPartyMultiTex;
		}
		else if (InTexMap->ClassID() == CORONAPHYSICALSKYCLASS)
		{
			Type = EDSBitmapType::PhysicalSky;
		}
		else if (InTexMap->ClassID() == VRAYPHYSICALSKYCLASS)
		{
			Type = EDSBitmapType::PhysicalSky;
		}
		else if (InTexMap->ClassID() == MRPHYSICALSKYCLASS)
		{
			Type = EDSBitmapType::PhysicalSky;
		}
		else if (InTexMap->ClassID() == MRPHYSICALSKYBCLASS)
		{
			Type = EDSBitmapType::PhysicalSky;
		}
		else if (InTexMap->ClassID() == TILESMAPCLASS)
		{
			if (FDatasmithMaxMatHelper::HasNonBakeableSubmap(InTexMap) == false)
			{
				Type = EDSBitmapType::BakeableMap;
			}
		}
	}

	return Type;
}

EDSMaterialType FDatasmithMaxMatHelper::GetMaterialClass(Mtl* Material)
{
	EDSMaterialType Type = EDSMaterialType::NotSupported;
	if (Material != nullptr)
	{
		if (Material->ClassID() == THEAMATOLDCLASS)
		{
			Type = EDSMaterialType::TheaMaterialDeprecated;
		}
		else if (Material->ClassID() == THEARANDOMCLASS)
		{
			Type = EDSMaterialType::TheaRandom;
		}
		else if (Material->ClassID() == THEAMATERIALCLASS)
		{
			Type = EDSMaterialType::TheaMaterial;
		}
		else if (Material->ClassID() == THEABASICCLASS)
		{
			Type = EDSMaterialType::TheaBasic;
		}
		else if (Material->ClassID() == THEAGLOSSYCLASS)
		{
			Type = EDSMaterialType::TheaGlossy;
		}
		else if (Material->ClassID() == THEASSSCLASS)
		{
			Type = EDSMaterialType::TheaSSS;
		}
		else if (Material->ClassID() == THEAFILMCLASS)
		{
			Type = EDSMaterialType::TheaFilm;
		}
		else if (Material->ClassID() == THEACOATINGCLASS)
		{
			Type = EDSMaterialType::TheaCoating;
		}
		else if (Material->ClassID() == MULTIMAT_CLASS_ID)
		{
			Type = EDSMaterialType::MultiMat;
		}
		else if (Material->ClassID() == STANDARDMATCLASS)
		{
			Type = EDSMaterialType::StandardMat;
		}
		else if (Material->ClassID() == SHELLCLASS)
		{
			Type = EDSMaterialType::ShellMat;
		}
		else if (Material->ClassID() == BLENDMATCLASS)
		{
			Type = EDSMaterialType::BlendMat;
		}
		else if (Material->ClassID() == XREFMATCLASS)
		{
			Type = EDSMaterialType::XRefMat;
		}
		else if (Material->ClassID() == ARCHDESIGNMATCLASS)
		{
			Type = EDSMaterialType::ArchDesignMat;
		}
		else if (Material->ClassID() == VRAYMATCLASS)
		{
			Type = EDSMaterialType::VRayMat;
		}
		else if (Material->ClassID() == VRAYBLENDMATCLASS)
		{
			Type = EDSMaterialType::VRayBlendMat;
		}
		else if (Material->ClassID() == VRAYLIGHTMATCLASS)
		{
			Type = EDSMaterialType::VRayLightMat;
		}
		else if (Material->ClassID() == VRAYFASTSSSCLASS)
		{
			Type = EDSMaterialType::VRayFastSSSMat;
		}
		else if (Material->ClassID() == CORONAMATCLASS)
		{
			Type = EDSMaterialType::CoronaMat;
		}
		else if (Material->ClassID() == CORONALAYERMATCLASS)
		{
			Type = EDSMaterialType::CoronaLayerMat;
		}
		else if (Material->ClassID() == CORONALIGHTMATCLASS)
		{
			Type = EDSMaterialType::CoronaLightMat;
		}
		else if (Material->ClassID() == PHYSICALMATCLASS)
		{
			Type = EDSMaterialType::PhysicalMat;
		}
	}

	return Type;
}

Mtl* FDatasmithMaxMatHelper::GetRenderedXRefMaterial(Mtl* Material)
{
	const TimeValue CurrentTime = GetCOREInterface()->GetTime();
	const int NumParamBlocks = Material->NumParamBlocks();

	bool bSourceMaterialOverridden = false;
	Mtl* MaterialOverride = nullptr;
	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID((short)j);
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			// Diffuse
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("enableOverride")) == 0)
			{
				bSourceMaterialOverridden = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
			}
			else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("overrideMaterial")) == 0)
			{
				MaterialOverride = ParamBlock2->GetMtl(ParamDefinition.ID, CurrentTime);
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	Mtl* RenderedMaterial = nullptr;
	if (bSourceMaterialOverridden)
	{
		RenderedMaterial = MaterialOverride;
	}
	else
	{
		IXRefMaterial* XRefMaterial = static_cast<IXRefMaterial*>(Material);
		RenderedMaterial = XRefMaterial->GetSourceMaterial(true);
	}

	return RenderedMaterial;
}

FLinearColor FDatasmithMaxMatHelper::MaxColorToFLinearColor(BMM_Color_fl Color, float Multiplier /*= 1.0f*/)
{
	Color.r = FMath::Pow(Color.r, 1.0f / FDatasmithExportOptions::ColorGamma);
	Color.g = FMath::Pow(Color.g, 1.0f / FDatasmithExportOptions::ColorGamma);
	Color.b = FMath::Pow(Color.b, 1.0f / FDatasmithExportOptions::ColorGamma);
	float Red = Color.r * Multiplier;
	float Green = Color.g * Multiplier;
	float Blue = Color.b * Multiplier;

	return FLinearColor(Red, Green, Blue);
}

FLinearColor FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(BMM_Color_fl Color, float Multiplier /*= 1.0f*/)
{
	return FLinearColor( Color.r, Color.g, Color.b );
}

float FDatasmithMaxMatHelper::GetBitmapGamma(BitmapTex* InBitmapTex)
{
	if (InBitmapTex)
	{
		if (Bitmap* ActualBitmap = InBitmapTex->GetBitmap(GetCOREInterface()->GetTime()))
		{
			return ActualBitmap->Gamma();
		}
	}
	return 2.2f;
}

float FDatasmithMaxMatHelper::GetBitmapGamma(BitmapInfo* InBitmap)
{
	if (InBitmap)
	{
		return InBitmap->Gamma();
	}
	return 2.2f;
}

bool FDatasmithMaxMatHelper::IsSRGB(BitmapTex& InBitmapTex)
{
	if (Bitmap* ActualBitmap = InBitmapTex.GetBitmap(GetCOREInterface()->GetTime()))
	{
		return IsSRGB(*ActualBitmap);
	}
	return true;
}

bool FDatasmithMaxMatHelper::IsSRGB(Bitmap& InBitmap)
{
	if (InBitmap.IsHighDynamicRange() )
	{
		return false;
	}

#ifdef MAX_RELEASE_R19 
	BitmapInfo BitmapInformation = InBitmap.GetBitmapInfo();

	switch ( BitmapInformation.Type() )
	{
	// UE considers that all bit depth >= 16 are linear
	case BMM_GRAY_16:
	case BMM_FLOAT_RGBA_32:
	case BMM_FLOAT_GRAY_32:
	case BMM_REALPIX_32:
	case BMM_TRUE_48:
	case BMM_TRUE_64:
		return false; // Linear
		break;
	default:
		return true; // sRGB
		break;
	}
#endif // MAX_RELEASE_R19

	return true;
}

bool FDatasmithMaxMatExport::UseFirstSubMapOnly(EDSMaterialType MaterialType, Mtl* Material)
{
	switch (MaterialType)
	{
	case EDSMaterialType::TheaMaterial:
	case EDSMaterialType::MultiMat:
	case EDSMaterialType::StandardMat:
	case EDSMaterialType::ArchDesignMat:
	case EDSMaterialType::VRayMat:
	case EDSMaterialType::VRayLightMat:
	case EDSMaterialType::CoronaMat:
	case EDSMaterialType::CoronaLightMat:
	case EDSMaterialType::BlendMat:
	case EDSMaterialType::XRefMat:
	case EDSMaterialType::VRayBlendMat:
	case EDSMaterialType::CoronaLayerMat:
		return false;
	default:
		if (Material->NumSubMtls() < 1)
		{
			return false;
		}
		else
		{
			return true;
		}
	}
}

TSharedPtr< IDatasmithBaseMaterialElement > FDatasmithMaxMatExport::ExportUniqueMaterial(TSharedRef< IDatasmithScene > DatasmithScene, Mtl* Material, const TCHAR* AssetsPath)
{
	if (!Material)
	{
		return {};
	}

	FString MaterialName(FDatasmithMaxMaterialsToUEPbrManager::GetDatasmithMaterialName(Material));

	for (int i = 0; i < DatasmithScene->GetMaterialsCount(); i++)
	{
		if (FString(DatasmithScene->GetMaterial(i)->GetName()) == MaterialName)
		{
			return DatasmithScene->GetMaterial(i);
		}
	}

	if ( FDatasmithMaxMaterialsToUEPbr* MaterialConverter = FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter( Material ) )
	{
		TSharedPtr< IDatasmithBaseMaterialElement > UEPbrMaterial;

		MaterialConverter->Convert( DatasmithScene, UEPbrMaterial, Material, AssetsPath );

		FDatasmithMaxMaterialsToUEPbrManager::AddDatasmithMaterial(DatasmithScene, Material, UEPbrMaterial);

		return UEPbrMaterial;
	}
	else
	{
		TSharedRef< IDatasmithMaterialElement > MaterialElement = FDatasmithSceneFactory::CreateMaterial( *MaterialName );
		
		FDatasmithMaxMaterialsToUEPbrManager::AddDatasmithMaterial(DatasmithScene, Material, MaterialElement);

		WriteXMLMaterial( DatasmithScene, MaterialElement, Material );

		return MaterialElement;
	}
}

void FDatasmithMaxMatExport::WriteXMLMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement > MaterialElement, Mtl* Material)
{
	if (Material == nullptr)
	{
		return;
	}

	EDSMaterialType MaterialType = FDatasmithMaxMatHelper::GetMaterialClass(Material);

	switch (FDatasmithMaxMatHelper::GetMaterialClass(Material))
	{
	case EDSMaterialType::StandardMat:
		FDatasmithMaxMatWriter::ExportStandardMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::TheaMaterial:
		FDatasmithMaxMatWriter::ExportTheaMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::VRayMat:
		FDatasmithMaxMatWriter::ExportVRayMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::VRayLightMat:
		FDatasmithMaxMatWriter::ExportVRayLightMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::ArchDesignMat:
		FDatasmithMaxMatWriter::ExportArchDesignMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::CoronaMat:
		FDatasmithMaxMatWriter::ExportCoronaMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::CoronaLightMat:
		FDatasmithMaxMatWriter::ExportCoronaLightMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::BlendMat:
		FDatasmithMaxMatWriter::ExportBlendMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::XRefMat:
		FDatasmithMaxMatExport::WriteXMLMaterial(DatasmithScene, MaterialElement, FDatasmithMaxMatHelper::GetRenderedXRefMaterial(Material));
		break;

	case EDSMaterialType::VRayBlendMat:
		FDatasmithMaxMatWriter::ExportVrayBlendMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::CoronaLayerMat:
		FDatasmithMaxMatWriter::ExportCoronaBlendMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::PhysicalMat:
		FDatasmithMaxMatWriter::ExportPhysicalMaterial(DatasmithScene, MaterialElement, Material);
		break;

	case EDSMaterialType::TheaRandom:
		return; // already added!
		break;

	default:
		if (UseFirstSubMapOnly(MaterialType, Material) == false)
		{
			DatasmithMaxLogger::Get().AddUnsupportedMat(Material);
		}
		else
		{
			DatasmithMaxLogger::Get().AddPartialSupportedMat(Material);
			FDatasmithMaxMatExport::WriteXMLMaterial(DatasmithScene, MaterialElement, Material->GetSubMtl(0));
		}
		break;
	}
}

FString FDatasmithMaxMatWriter::DumpTexture(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, const TCHAR* ColorPrefix, bool bForceInvert, bool bIsGrayscale)
{
	if (InTexmap == NULL)
	{
		return TEXT("");
	}

	// if it is not a regular one it should be nested.
	if (CompTex->GetMode() != EDatasmithCompMode::Regular)
	{
		TSharedPtr<IDatasmithCompositeTexture> ChildCompTex = FDatasmithSceneFactory::CreateCompositeTexture();
		FString Result = DumpTexture(DatasmithScene, ChildCompTex, InTexmap, Prefix,ColorPrefix, bForceInvert, bIsGrayscale);
		CompTex->AddSurface(ChildCompTex);
		return Result;
	}

	EDSBitmapType TextureClass = FDatasmithMaxMatHelper::GetTextureClass(InTexmap);
	if (FString(Prefix) == DATASMITH_BUMPTEXNAME && TextureClass != EDSBitmapType::NormalMap)
	{
		DatasmithMaxLogger::Get().AddTextureError(TEXT("Bump maps are automatically converted to normal maps, as Unreal Engine does not handle bump maps directly."));
	}

	switch( TextureClass )
	{
		case EDSBitmapType::RegularBitmap:
		{
			return DumpBitmap(CompTex, (BitmapTex*)InTexmap, Prefix, bForceInvert, bIsGrayscale);
		}
		case EDSBitmapType::AutodeskBitmap:
		{
			return DumpAutodeskBitmap(CompTex, InTexmap, Prefix, bForceInvert, bIsGrayscale);
		}
		case EDSBitmapType::TheaBitmap:
		{
			return DumpBitmapThea(CompTex, (BitmapTex*)InTexmap, Prefix, bForceInvert, bIsGrayscale);
		}
		case EDSBitmapType::CoronaBitmap:
		{
			return DumpBitmapCorona(CompTex, (BitmapTex*)InTexmap, Prefix, bForceInvert, bIsGrayscale);
		}
		case EDSBitmapType::VRayHRDI:
		{
			return DumpVrayHdri(CompTex, (BitmapTex*)InTexmap, Prefix, bForceInvert);
		}
		case EDSBitmapType::ColorCorrector:
		{
			if(FDatasmithMaxMatHelper::HasNonBakeableSubmap(InTexmap) == false)
			{
				return DumpBakeable(CompTex, (BitmapTex*)InTexmap, Prefix, bForceInvert, bIsGrayscale);
			}
			else
			{
				return DumpColorCorrect(DatasmithScene, CompTex, InTexmap, Prefix, bForceInvert, bIsGrayscale);
			}
		}
		case EDSBitmapType::Gradient:
		{
			DatasmithMaxLogger::Get().AddPartialSupportedMap(InTexmap);
			if (InTexmap->GetSubTexmap(0) != NULL)
			{
				return DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(0), Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
			}
			else
			{
				return DumpColorOfTexmap(CompTex, InTexmap, ColorPrefix, TEXT("color1"));
			}
		}
		case EDSBitmapType::GradientRamp:
		{
			DatasmithMaxLogger::Get().AddPartialSupportedMap(InTexmap);
			if (InTexmap->GetSubTexmap(0) != NULL)
			{
				return DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(0), Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
			}
			else
			{
				return DumpColorOfTexmap(CompTex, InTexmap, ColorPrefix, TEXT("color1"));
			}
		}
		case EDSBitmapType::Checker:
		{
			DatasmithMaxLogger::Get().AddPartialSupportedMap(InTexmap);
			if (InTexmap->GetSubTexmap(0) != NULL)
			{
				return DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(0), Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
			}
			else
			{
				return DumpColorOfTexmap(CompTex, InTexmap, ColorPrefix, TEXT("color1"));
			}
		}
		case EDSBitmapType::Cellular:
		{
			DatasmithMaxLogger::Get().AddPartialSupportedMap(InTexmap);
			if (InTexmap->GetSubTexmap(0) != NULL)
			{
				return DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(0), Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
			}
			else
			{
				return DumpColorOfTexmap(CompTex, InTexmap, ColorPrefix, TEXT("cellColor"));
			}
		}
		case EDSBitmapType::FallOff:
		{
			return DumpFalloff(DatasmithScene, CompTex, InTexmap, Prefix, bForceInvert, bIsGrayscale);
		}
		case EDSBitmapType::Mix:
		{
			return DumpMix(DatasmithScene, CompTex, InTexmap, Prefix, bForceInvert, bIsGrayscale);
		}
		case EDSBitmapType::CompositeTex:
		{
			return DumpCompositetex(DatasmithScene, CompTex, InTexmap, Prefix, bForceInvert, bIsGrayscale);
		}
		case EDSBitmapType::NormalMap:
		{
			return DumpNormalTexture(DatasmithScene, CompTex, InTexmap, Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
		}
		case EDSBitmapType::Noise:
		{
			DatasmithMaxLogger::Get().AddPartialSupportedMap(InTexmap);
			if (InTexmap->GetSubTexmap(0) != NULL)
			{
				return DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(0), Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
			}
			else
			{
				return DumpColorOfTexmap(CompTex, InTexmap, ColorPrefix, TEXT("color1"));
			}
		}
		case EDSBitmapType::VRayDirt:
		{
			DatasmithMaxLogger::Get().AddTextureError(TEXT("V-Ray dirt texture is not supported and only the unoccluded texture/color will be used"));

			if (InTexmap->GetSubTexmap(0) != NULL)
			{
				return DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(0), Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
			}
			else
			{
				return DumpColorOfTexmap(CompTex, InTexmap, ColorPrefix, TEXT("unoccluded_color"));
			}
		}
		case EDSBitmapType::CoronaAO:
		{
			DatasmithMaxLogger::Get().AddGeneralError(TEXT("CoronaAO texture is not supported and only the unoccluded texture/color will be used"));

			if (InTexmap->GetSubTexmap(1) != NULL)
			{
				return DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(1), Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
			}
			else
			{
				return DumpColorOfTexmap(CompTex, InTexmap, ColorPrefix, TEXT("colorUnoccluded"));
			}
		}
		case EDSBitmapType::VRayColor:
		{
			return DumpVrayColor(CompTex, InTexmap, Prefix, bForceInvert);
		}
		case EDSBitmapType::CoronaColor:
		{
			return DumpCoronaColor(CompTex, InTexmap, Prefix, bForceInvert);
		}
		case EDSBitmapType::CoronaMix:
		{
			return DumpCoronaMix(DatasmithScene, CompTex, InTexmap, Prefix, bForceInvert, bIsGrayscale);
		}
		case EDSBitmapType::CoronaMultiTex:
		{
			DatasmithMaxLogger::Get().AddPartialSupportedMap(InTexmap);
			return DumpCoronaMultitex(DatasmithScene, CompTex, InTexmap, Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
		}
		case EDSBitmapType::PhysicalSky:
		{
			DatasmithScene->SetUsePhysicalSky(true);
			return TEXT("");
		}
		case EDSBitmapType::ThirdPartyMultiTex:
		{
			DatasmithMaxLogger::Get().AddPartialSupportedMap(InTexmap);
			if (InTexmap->GetSubTexmap(0) != NULL)
			{
				return DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(0), Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
			}
			else
			{
				return DumpColorOfTexmap(CompTex, InTexmap, ColorPrefix, TEXT("sub_color"));
			}
		}
		case EDSBitmapType::BakeableMap:
		{
			if (FDatasmithMaxMatHelper::HasNonBakeableSubmap(InTexmap) == false)
			{
				return DumpBakeable(CompTex, (BitmapTex*)InTexmap, Prefix, bForceInvert, bIsGrayscale);
			}
		}
		default:
		{

			if (InTexmap->NumSubTexmaps() > 0)
			{
				DatasmithMaxLogger::Get().AddUnsupportedMap(InTexmap);
				// just for try!
				if (InTexmap->GetSubTexmap(0) != NULL)
				{
					return DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(0), Prefix, ColorPrefix, bForceInvert, bIsGrayscale);
				}
				else
				{
					return DumpColorOfTexmap(CompTex, InTexmap, ColorPrefix, TEXT("color1"));
				}
			}
			else
			{
				DatasmithMaxLogger::Get().AddUnsupportedMap(InTexmap);
				return DumpColorOfTexmap(CompTex, InTexmap, ColorPrefix, TEXT("color1"));
			}
		}
	}
}

FString FDatasmithMaxMatWriter::DumpNormalTexture(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, const TCHAR* ColorPrefix, bool bForceInvert, bool bIsGrayscale)
{
	int NumParamBlocks = InTexmap->NumParamBlocks();

	bool bInvertGreen = false;
	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];
			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("flip_green")) == 0 || FCString::Stricmp(ParamDefinition.int_name, TEXT("flipgreen")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 1)
				{
					bInvertGreen = true;
				}
				continue;
			}

			if (FCString::Stricmp(ParamDefinition.int_name, TEXT("complement_ddl")) == 0)
			{
				if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 4)
				{
					bInvertGreen = false;
				}
				else
				{
					bInvertGreen = true;
				}
				continue;
			}
		}
	}

	EDSBitmapType TextureClass = FDatasmithMaxMatHelper::GetTextureClass(InTexmap);
	if (TextureClass == EDSBitmapType::NormalMap)
	{
		if (FString(Prefix) != DATASMITH_BUMPTEXNAME)
		{
			FString DumpName = DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(0), Prefix, ColorPrefix, bForceInvert || bInvertGreen, bIsGrayscale);

			TSharedPtr< IDatasmithTextureElement > TextureElement;
			for (int TextureIndex = 0; TextureIndex < DatasmithScene->GetTexturesCount(); TextureIndex++)
			{
				if (DumpName == DatasmithScene->GetTexture(TextureIndex)->GetName())
				{
					TextureElement = DatasmithScene->GetTexture(TextureIndex);
				}
			}
		
			if (TextureElement)
			{
				// Only process the TextureElement once (we assume that the "Other" TextureMode means that it hasn't been processed yet)
				if ( TextureElement->GetTextureMode() == EDatasmithTextureMode::Other )
				{
					if ( bInvertGreen )
					{
						TextureElement->SetTextureMode( EDatasmithTextureMode::NormalGreenInv );
					}
					else
					{
						TextureElement->SetTextureMode( EDatasmithTextureMode::Normal );
					}

					float RgbCurve = TextureElement->GetRGBCurve();
					// if RgbCurve <= 0 it has been set to default mode so we can forget about that
					// this happens mostly when no advanced gamma correction is enabled in 3dsmax preferences
					if (RgbCurve > 0.f)
					{
						if (FDatasmithMaxMatWriter::GetCoronaFixNormal(InTexmap) == true)
						{
							// corona has a checkbox to work in automatic mode
							RgbCurve = -1.f;
						}
						else
						{
							// Bitmaps are by default using Gamma/2.2f RGB curve but the reference for normalmaps
							// should be Gamma/1.0f 
							// if the user has set the proper value (1.0) the result will be 2.2/2.2 = 1.0
							RgbCurve *= 2.2f;
						}
						TextureElement->SetRGBCurve(RgbCurve);

						if (FMath::IsNearlyEqual(RgbCurve, 1.0f, 0.001f) == false && RgbCurve > 0.0f)
						{
							FString Error = FString("Potential wrong input gamma on normalmap: ") + FPaths::GetCleanFilename(TextureElement->GetFile());
							DatasmithMaxLogger::Get().AddTextureError(*Error);
						}
					}
				}
			}
			return DumpName;
		}
		else
		{
			return DumpTexture(DatasmithScene, CompTex, InTexmap->GetSubTexmap(1), TEXT(""), TEXT(""), bForceInvert, bIsGrayscale);
		}
	}

	if (TextureClass == EDSBitmapType::TheaBitmap)
	{
		return DumpBitmapThea(CompTex, (BitmapTex*)InTexmap, Prefix, bForceInvert || bInvertGreen, bIsGrayscale);
	}

	return TEXT("");
}

void FDatasmithMaxMatWriter::DumpWeightedTexture(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, BMM_Color_fl Color, float Weight, const TCHAR* Prefix, const TCHAR* ColorPrefix, bool bForceInvert, bool bIsGrayscale)
{
	FString ActualPrefix = FString(Prefix) + FString(TEXT("comp"));
	CompTex->SetMode(EDatasmithCompMode::Mix);
	DumpTexture(DatasmithScene, CompTex, InTexmap, DATASMITH_TEXTURENAME, DATASMITH_COLORNAME, bForceInvert, bIsGrayscale);

	CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
	CompTex->AddParamVal1(IDatasmithCompositeTexture::ParamVal(1.0f-(float)Weight, TEXT("Weight")));
}
void FDatasmithMaxMatWriter::DumpWeightedColor(TSharedPtr<IDatasmithCompositeTexture>& CompTex, BMM_Color_fl ColorA, BMM_Color_fl ColorB, float Weight, const TCHAR* Prefix)
{
	if (Weight == 1)
	{
		CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(ColorA));
		return;
	}
	if (Weight == 0)
	{
		CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(ColorB));
		return;
	}
	FString ActualPrefix = FString(Prefix) + FString(TEXT("comp"));
	CompTex->SetMode(EDatasmithCompMode::Mix);

	CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(ColorA));
	CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(ColorB));
	CompTex->AddParamVal1(IDatasmithCompositeTexture::ParamVal(1.0f-(float)Weight, TEXT("Weight")));
}

FString FDatasmithMaxMatWriter::DumpColorOfTexmap(TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* ColorPrefix, const TCHAR* Property)
{
	int NumParamBlocks = InTexmap->NumParamBlocks();

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		if (ParamBlock2 == NULL)
		{
			return TEXT("");
		}

		// The the descriptor to 'decode'
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		if (ParamBlockDesc == NULL)
		{
			return TEXT("");
		}

		// Loop through all the defined parameters therein
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if (FCString::Stricmp(ParamDefinition.int_name, Property) == 0)
			{
				BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
				CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
				return TEXT("");
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	// not found using 'property', choose first color
	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];
			BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());

			if (Color.r != 0 || Color.g != 0 || Color.b != 0)
			{
				CompTex->AddSurface(FDatasmithMaxMatHelper::MaxColorToFLinearColor(Color));
				return TEXT("");
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	return TEXT("");
}
