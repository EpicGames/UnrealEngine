// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaxMaterialsToUEPbr/DatasmithMaxCoronaMaterialsToUEPbr.h"

#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithMaxHelper.h"
#include "DatasmithMaxWriter.h"
#include "DatasmithSceneFactory.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxTexmapToUEPbr.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxCoronaTexmapToUEPbr.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "max.h"
	#include "mtl.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

namespace DatasmithMaxCoronaMaterialsToUEPbrImpl
{
	struct FMaxCoronaMaterial
	{
		// Diffuse
		DatasmithMaxTexmapParser::FWeightedColorParameter Diffuse;
		DatasmithMaxTexmapParser::FMapParameter DiffuseMap;
		float DiffuseLevel = 1.f;

		// Reflection
		DatasmithMaxTexmapParser::FWeightedColorParameter Reflection;
		DatasmithMaxTexmapParser::FMapParameter ReflectionMap;
		float ReflectionLevel = 0.f;

		float ReflectionGlossiness = 0.f;
		DatasmithMaxTexmapParser::FMapParameter ReflectionGlossinessMap;

		// Reflection IOR
		float ReflectionIOR;
		DatasmithMaxTexmapParser::FMapParameter ReflectionIORMap;

		// Refraction
		DatasmithMaxTexmapParser::FWeightedColorParameter Refraction;
		DatasmithMaxTexmapParser::FMapParameter RefractionMap;
		float RefractionLevel = 0.f;

		// Opacity
		DatasmithMaxTexmapParser::FWeightedColorParameter Opacity;
		DatasmithMaxTexmapParser::FMapParameter OpacityMap;
		float OpacityLevel = 0.f;

		// Bump
		DatasmithMaxTexmapParser::FMapParameter BumpMap;
	};

	FMaxCoronaMaterial ParseCoronaMaterialProperties( Mtl& Material )
	{
		FMaxCoronaMaterial CoronaMaterialProperties;

		const int NumParamBlocks = Material.NumParamBlocks();

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				// Diffuse
				if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("colorDiffuse")) == 0 )
				{
					CoronaMaterialProperties.Diffuse.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, CurrentTime ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapDiffuse")) == 0)
				{
					CoronaMaterialProperties.DiffuseMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("texmapOnDiffuse") ) == 0 )
				{
					CoronaMaterialProperties.DiffuseMap.bEnabled = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapAmountDiffuse")) == 0)
				{
					CoronaMaterialProperties.DiffuseMap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("levelDiffuse")) == 0)
				{
					CoronaMaterialProperties.DiffuseLevel = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}

				// Reflection
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("colorReflect")) == 0 )
				{
					CoronaMaterialProperties.Reflection.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, CurrentTime ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapReflect")) == 0)
				{
					CoronaMaterialProperties.ReflectionMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnReflect")) == 0)
				{
					CoronaMaterialProperties.ReflectionMap.bEnabled = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapamountReflect")) == 0)
				{
					CoronaMaterialProperties.ReflectionMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("levelReflect")) == 0)
				{
					CoronaMaterialProperties.ReflectionLevel = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}

				// Reflection Glossiness
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("reflectGlossiness") ) == 0 )
				{
					CoronaMaterialProperties.ReflectionGlossiness = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapReflectGlossiness")) == 0)
				{
					CoronaMaterialProperties.ReflectionGlossinessMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnReflectGlossiness")) == 0)
				{
					CoronaMaterialProperties.ReflectionGlossinessMap.bEnabled = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapamountReflectGlossiness")) == 0)
				{
					CoronaMaterialProperties.ReflectionGlossinessMap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
				}

				// Reflection IOR
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("fresnelIor")) == 0 )
				{
					CoronaMaterialProperties.ReflectionIOR = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapFresnelIor")) == 0 )
				{
					CoronaMaterialProperties.ReflectionIORMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnFresnelIor")) == 0 )
				{
					CoronaMaterialProperties.ReflectionIORMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("mapamountFresnelIor")) == 0 )
				{
					CoronaMaterialProperties.ReflectionIORMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}

				// Refraction
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("colorRefract")) == 0 )
				{
					CoronaMaterialProperties.Refraction.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, CurrentTime ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapRefract")) == 0)
				{
					CoronaMaterialProperties.RefractionMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOnRefract")) == 0)
				{
					CoronaMaterialProperties.RefractionMap.bEnabled = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapamountRefract")) == 0)
				{
					CoronaMaterialProperties.RefractionMap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("levelRefract")) == 0)
				{
					CoronaMaterialProperties.RefractionLevel = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}

				// Opacity
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("ColorOpacity")) == 0)
				{
				CoronaMaterialProperties.Opacity.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, CurrentTime ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOpacity")) == 0)
				{
					CoronaMaterialProperties.OpacityMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("texmapOnOpacity") ) == 0 )
				{
					CoronaMaterialProperties.OpacityMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("mapamountOpacity")) == 0 )
				{
					CoronaMaterialProperties.OpacityMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("levelOpacity")) == 0)
				{
					CoronaMaterialProperties.OpacityLevel = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}

				// Bump
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapBump")) == 0 )
				{
					CoronaMaterialProperties.BumpMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapamountBump")) == 0)
				{
					CoronaMaterialProperties.BumpMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("texmapOnBump") ) == 0 )
				{
					CoronaMaterialProperties.BumpMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		return CoronaMaterialProperties;
	}

	struct FMaxCoronaBlendMaterial
	{
		struct FCoronaCoatMaterialProperties
		{
			Mtl* Material = nullptr;
			float Amount = 1.f;

			DatasmithMaxTexmapParser::FMapParameter Mask;
		};

		Mtl* BaseMaterial = nullptr;
		static const int32 MaximumNumberOfCoat = 10;
		FCoronaCoatMaterialProperties CoatedMaterials[MaximumNumberOfCoat];
	};

	FMaxCoronaBlendMaterial ParseCoronaBlendMaterialProperties(Mtl& Material)
	{
		FMaxCoronaBlendMaterial CoronaBlendMaterialProperties;
		FMaxCoronaBlendMaterial::FCoronaCoatMaterialProperties* CoatedMaterials = CoronaBlendMaterialProperties.CoatedMaterials;

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();
		const int NumParamBlocks = Material.NumParamBlocks();

		for (int ParemBlockIndex = 0; ParemBlockIndex < NumParamBlocks; ++ParemBlockIndex)
		{
			IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)ParemBlockIndex);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int DescIndex = 0; DescIndex < ParamBlockDesc->count; ++DescIndex)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[DescIndex];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseMtl")) == 0)
				{
					CoronaBlendMaterialProperties.BaseMaterial = ParamBlock2->GetMtl(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("layers")) == 0)
				{
					for (int CoatIndex = 0; CoatIndex < FMaxCoronaBlendMaterial::MaximumNumberOfCoat; ++CoatIndex)
					{
						CoatedMaterials[CoatIndex].Material = ParamBlock2->GetMtl(ParamDefinition.ID, CurrentTime, CoatIndex);
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("amounts")) == 0)
				{
					for (int CoatIndex = 0; CoatIndex < 9; ++CoatIndex)
					{
						CoatedMaterials[CoatIndex].Amount = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime, CoatIndex);
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mixmaps")) == 0)
				{
					for (int CoatIndex = 0; CoatIndex < FMaxCoronaBlendMaterial::MaximumNumberOfCoat; ++CoatIndex)
					{
						CoatedMaterials[CoatIndex].Mask.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime, CoatIndex);
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("maskAmounts")) == 0)
				{
					for (int CoatIndex = 0; CoatIndex < 9; ++CoatIndex)
					{
						CoatedMaterials[CoatIndex].Mask.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime, CoatIndex);
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("masksOn")) == 0)
				{
					for (int CoatIndex = 0; CoatIndex < 9; ++CoatIndex)
					{
						CoatedMaterials[CoatIndex].Mask.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime, CoatIndex ) != 0 );
					}
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		return CoronaBlendMaterialProperties;
	}

	struct FCoronaLightMaterial
	{
		DatasmithMaxTexmapParser::FMapParameter EmitTexture;
		DatasmithMaxTexmapParser::FMapParameter ClipTexture;
		BMM_Color_fl EmitColor;
		float Multiplier = 1.0f;


		void Parse(Mtl& Material)
		{

			int NumParamBlocks = Material.NumParamBlocks();

			for (int j = 0; j < NumParamBlocks; j++)
			{
				IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)j);
				// The the descriptor to 'decode'
				ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
				// Loop through all the defined parameters therein
				for (int i = 0; i < ParamBlockDesc->count; i++)
				{
					const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

					if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap")) == 0)
					{
						EmitTexture.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacityTexmap")) == 0)
					{
						ClipTexture.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapOn")) == 0)
					{
						if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
						{
							EmitTexture.bEnabled = false;
						}
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacityTexmapOn")) == 0)
					{
						if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
						{
							ClipTexture.bEnabled = false;
						}
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("color")) == 0)
					{
						EmitColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("twosidedEmission")) == 0)
					{
						// int twoSided = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("intensity")) == 0)
					{
						Multiplier = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
				}
				ParamBlock2->ReleaseDesc();
			}
		}
	};

	struct FCoronaPhysicalMaterial
	{
		FLinearColor BaseColor;
		float BaseLevel;
		DatasmithMaxTexmapParser::FMapParameter BaseTexmap;
		int MetalnessMode;
		FLinearColor OpacityColor;
		float OpacityLevel;
		DatasmithMaxTexmapParser::FMapParameter OpacityTexmap;
		bool bOpacityCutout;
		float BaseRoughness;
		DatasmithMaxTexmapParser::FMapParameter BaseRoughnessTexmap;
		float BaseIor;
		DatasmithMaxTexmapParser::FMapParameter BaseIorTexmap;
		float RefractionAmount;
		DatasmithMaxTexmapParser::FMapParameter RefractionAmountTexmap;
		bool bUseThinMode;
		float ClearcoatAmount;
		DatasmithMaxTexmapParser::FMapParameter ClearcoatAmountTexmap;
		float ClearcoatIor;
		DatasmithMaxTexmapParser::FMapParameter ClearcoatIorTexmap;
		float ClearcoatRoughness;
		DatasmithMaxTexmapParser::FMapParameter ClearcoatRoughnessTexmap;
		FLinearColor VolumetricAbsorptionColor;
		DatasmithMaxTexmapParser::FMapParameter VolumetricAbsorptionTexmap;
		FLinearColor VolumetricScatteringColor;
		DatasmithMaxTexmapParser::FMapParameter VolumetricScatteringTexmap;
		float AttenuationDistance;
		float ScatterDirectionality;
		bool bScatterSingleBounce;
		FLinearColor SelfIllumColor;
		float SelfIllumLevel;
		DatasmithMaxTexmapParser::FMapParameter SelfIllumTexmap;
		DatasmithMaxTexmapParser::FMapParameter BaseBumpTexmap;
		float TranslucencyFraction;
		DatasmithMaxTexmapParser::FMapParameter TranslucencyFractionTexmap;
		FLinearColor ThinAbsorptionColor;
		DatasmithMaxTexmapParser::FMapParameter ThinAbsorptionTexmap;
		DatasmithMaxTexmapParser::FMapParameter ClearcoatBumpTexmap;
		DatasmithMaxTexmapParser::FMapParameter MetalnessTexmap;
		int RoughnessMode;
		FLinearColor TranslucencyColor;
		DatasmithMaxTexmapParser::FMapParameter TranslucencyColorTexmap;
		int IorMode;

		void Parse(Mtl& Material)
		{

			const TimeValue CurrentTime = GetCOREInterface()->GetTime();
			int NumParamBlocks = Material.NumParamBlocks();

			for (int j = 0; j < NumParamBlocks; j++)
			{
				IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)j);
				// The the descriptor to 'decode'
				ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
				// Loop through all the defined parameters therein
				for (int i = 0; i < ParamBlockDesc->count; i++)
				{
					const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

					if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseColor")) == 0)
					{
						BaseColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseLevel")) == 0)
					{
						BaseLevel = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseTexmap")) == 0)
					{
						BaseTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseTexmapOn")) == 0)
					{
						BaseTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseMapAmount")) == 0)
					{
						BaseTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("metalnessMode")) == 0)
					{
						MetalnessMode = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacityColor")) == 0)
					{
						OpacityColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacityLevel")) == 0)
					{
						OpacityLevel = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacityTexmap")) == 0)
					{
						OpacityTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacityTexmapOn")) == 0)
					{
						OpacityTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacityMapAmount")) == 0)
					{
						OpacityTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacityCutout")) == 0)
					{
						bOpacityCutout = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseRoughness")) == 0)
					{
						BaseRoughness = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseRoughnessTexmap")) == 0)
					{
						BaseRoughnessTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseRoughnessTexmapOn")) == 0)
					{
						BaseRoughnessTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseRoughnessMapAmount")) == 0)
					{
						BaseRoughnessTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseIor")) == 0)
					{
						BaseIor = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseIorTexmap")) == 0)
					{
						BaseIorTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseIorTexmapOn")) == 0)
					{
						BaseIorTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseIorMapAmount")) == 0)
					{
						BaseIorTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refractionAmount")) == 0)
					{
						RefractionAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refractionAmountTexmap")) == 0)
					{
						RefractionAmountTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refractionAmountTexmapOn")) == 0)
					{
						RefractionAmountTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refractionAmountMapAmount")) == 0)
					{
						RefractionAmountTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("useThinMode")) == 0)
					{
						bUseThinMode = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatAmount")) == 0)
					{
						ClearcoatAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatAmountTexmap")) == 0)
					{
						ClearcoatAmountTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatAmountTexmapOn")) == 0)
					{
						ClearcoatAmountTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatAmountMapAmount")) == 0)
					{
						ClearcoatAmountTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatIor")) == 0)
					{
						ClearcoatIor = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatIorTexmap")) == 0)
					{
						ClearcoatIorTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatIorTexmapOn")) == 0)
					{
						ClearcoatIorTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatIorMapAmount")) == 0)
					{
						ClearcoatIorTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatRoughness")) == 0)
					{
						ClearcoatRoughness = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatRoughnessTexmap")) == 0)
					{
						ClearcoatRoughnessTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatRoughnessTexmapOn")) == 0)
					{
						ClearcoatRoughnessTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatRoughnessMapAmount")) == 0)
					{
						ClearcoatRoughnessTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("volumetricAbsorptionColor")) == 0)
					{
						VolumetricAbsorptionColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("volumetricAbsorptionTexmap")) == 0)
					{
						VolumetricAbsorptionTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("volumetricAbsorptionTexmapOn")) == 0)
					{
						VolumetricAbsorptionTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("volumetricAbsorptionMapAmount")) == 0)
					{
						VolumetricAbsorptionTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("volumetricScatteringColor")) == 0)
					{
						VolumetricScatteringColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("volumetricScatteringTexmap")) == 0)
					{
						VolumetricScatteringTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("volumetricScatteringTexmapOn")) == 0)
					{
						VolumetricScatteringTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("volumetricScatteringMapAmount")) == 0)
					{
						VolumetricScatteringTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("attenuationDistance")) == 0)
					{
						AttenuationDistance = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime()) * (float)GetSystemUnitScale(UNITS_CENTIMETERS);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("scatterDirectionality")) == 0)
					{
						ScatterDirectionality = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("scatterSingleBounce")) == 0)
					{
						bScatterSingleBounce = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("selfIllumColor")) == 0)
					{
						SelfIllumColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("selfIllumLevel")) == 0)
					{
						SelfIllumLevel = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("selfIllumTexmap")) == 0)
					{
						SelfIllumTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("selfIllumTexmapOn")) == 0)
					{
						SelfIllumTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("selfIllumMapAmount")) == 0)
					{
						SelfIllumTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseBumpTexmap")) == 0)
					{
						BaseBumpTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseBumpTexmapOn")) == 0)
					{
						BaseBumpTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseBumpMapAmount")) == 0)
					{
						BaseBumpTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("translucencyFraction")) == 0)
					{
						TranslucencyFraction = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("translucencyFractionTexmap")) == 0)
					{
						TranslucencyFractionTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("translucencyFractionTexmapOn")) == 0)
					{
						TranslucencyFractionTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("translucencyFractionMapAmount")) == 0)
					{
						TranslucencyFractionTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("thinAbsorptionColor")) == 0)
					{
						ThinAbsorptionColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("thinAbsorptionTexmap")) == 0)
					{
						ThinAbsorptionTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("thinAbsorptionTexmapOn")) == 0)
					{
						ThinAbsorptionTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("thinAbsorptionMapAmount")) == 0)
					{
						ThinAbsorptionTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatBumpTexmap")) == 0)
					{
						ClearcoatBumpTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatBumpTexmapOn")) == 0)
					{
						ClearcoatBumpTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("clearcoatBumpMapAmount")) == 0)
					{
						ClearcoatBumpTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("metalnessTexmap")) == 0)
					{
						MetalnessTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("metalnessTexmapOn")) == 0)
					{
						MetalnessTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("metalnessMapAmount")) == 0)
					{
						MetalnessTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughnessMode")) == 0)
					{
						RoughnessMode = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("translucencyColor")) == 0)
					{
						TranslucencyColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("translucencyColorTexmap")) == 0)
					{
						TranslucencyColorTexmap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("translucencyColorTexmapOn")) == 0)
					{
						TranslucencyColorTexmap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0;
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("translucencyColorMapAmount")) == 0)
					{
						TranslucencyColorTexmap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime);
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("iorMode")) == 0)
					{
						IorMode = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
				}
				ParamBlock2->ReleaseDesc();
			}
		}
	};

  
}

FDatasmithMaxCoronaMaterialsToUEPbr::FDatasmithMaxCoronaMaterialsToUEPbr()
{
	TexmapConverters.Add( new FDatasmithMaxCoronaAOToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxCoronaColorToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxCoronalNormalToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxCoronalBitmapToUEPbr() );
}

bool FDatasmithMaxCoronaMaterialsToUEPbr::IsSupported( Mtl* Material )
{
	return true;
}

void FDatasmithMaxCoronaMaterialsToUEPbr::Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath )
{
	if ( !Material )
	{
		return;
	}
	
	TSharedRef< IDatasmithUEPbrMaterialElement > PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial( GetMaterialName(Material) );
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	DatasmithMaxCoronaMaterialsToUEPbrImpl::FMaxCoronaMaterial CoronaMaterialProperties = DatasmithMaxCoronaMaterialsToUEPbrImpl::ParseCoronaMaterialProperties( *Material );

	ConvertState.DefaultTextureMode = EDatasmithTextureMode::Diffuse; // Both Diffuse and Reflection are considered diffuse maps

	// Diffuse
	IDatasmithMaterialExpression* DiffuseExpression = nullptr;
	{
		DiffuseExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, CoronaMaterialProperties.DiffuseMap, TEXT("Diffuse Color"),
			CoronaMaterialProperties.Diffuse.Value, TOptional< float >() );
	}

	if ( DiffuseExpression )
	{
		DiffuseExpression->SetName( TEXT("Diffuse") );

		IDatasmithMaterialExpressionGeneric* MultiplyDiffuseLevelExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		MultiplyDiffuseLevelExpression->SetExpressionName( TEXT("Multiply") );

		IDatasmithMaterialExpressionScalar* DiffuseLevelExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		DiffuseLevelExpression->SetName( TEXT("Diffuse Level") );
		DiffuseLevelExpression->GetScalar() = CoronaMaterialProperties.DiffuseLevel;

		DiffuseExpression->ConnectExpression( *MultiplyDiffuseLevelExpression->GetInput(0) );
		DiffuseLevelExpression->ConnectExpression( *MultiplyDiffuseLevelExpression->GetInput(1) );

		DiffuseExpression = MultiplyDiffuseLevelExpression;
	}

	// Reflection
	IDatasmithMaterialExpression* ReflectionExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, CoronaMaterialProperties.ReflectionMap, TEXT("Reflection Color"), CoronaMaterialProperties.Reflection.Value, TOptional< float >() );

	if ( ReflectionExpression )
	{
		ReflectionExpression->SetName( TEXT("Reflection") );

		IDatasmithMaterialExpressionGeneric* MultiplyReflectionLevelExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		MultiplyReflectionLevelExpression->SetExpressionName( TEXT("Multiply") );

		IDatasmithMaterialExpressionScalar* ReflectionLevelExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		ReflectionLevelExpression->SetName( TEXT("Reflection Level") );
		ReflectionLevelExpression->GetScalar() = CoronaMaterialProperties.ReflectionLevel;

		ReflectionExpression->ConnectExpression( *MultiplyReflectionLevelExpression->GetInput(0) );
		ReflectionLevelExpression->ConnectExpression( *MultiplyReflectionLevelExpression->GetInput(1) );

		ReflectionExpression = MultiplyReflectionLevelExpression;
	}

	IDatasmithMaterialExpressionGeneric* ReflectionIntensityExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
	ReflectionIntensityExpression->SetExpressionName( TEXT("Desaturation") );

	ReflectionExpression->ConnectExpression( *ReflectionIntensityExpression->GetInput(0) );

	// Glossiness
	IDatasmithMaterialExpression* GlossinessExpression = nullptr;
	{
		TGuardValue< bool > SetIsMonoChannel( ConvertState.bIsMonoChannel, true );

		ConvertState.DefaultTextureMode = EDatasmithTextureMode::Specular;

		GlossinessExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, CoronaMaterialProperties.ReflectionGlossinessMap, TEXT("Reflection Glossiness"), TOptional< FLinearColor >(), CoronaMaterialProperties.ReflectionGlossiness );

		if ( GlossinessExpression )
		{
			GlossinessExpression->SetName( TEXT("Reflection Glossiness") );
		}
	}

	// Bump
	{
		ConvertState.DefaultTextureMode = EDatasmithTextureMode::Bump; // Will change to normal if we pass through a normal map texmap
		ConvertState.bCanBake = false; // Current baking fails to produce proper normal maps

		IDatasmithMaterialExpression* BumpExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, CoronaMaterialProperties.BumpMap, TEXT("Bump Map"), TOptional< FLinearColor >(), TOptional< float >() );

		if ( BumpExpression )
		{
			BumpExpression->ConnectExpression( PbrMaterialElement->GetNormal() );
		}

		if ( BumpExpression )
		{
			BumpExpression->SetName( TEXT("Bump Map") );
		}

		ConvertState.bCanBake = true;
	}

	ConvertState.DefaultTextureMode = EDatasmithTextureMode::Specular; // At this point, all maps are considered specular maps

	// Opacity
	IDatasmithMaterialExpression* OpacityExpression = nullptr;
	{
		IDatasmithMaterialExpression* OpacityTexmapExpression = ConvertTexmap( CoronaMaterialProperties.OpacityMap );

		if (OpacityTexmapExpression)
		{
			OpacityExpression = &Desaturate(Multiply(*OpacityTexmapExpression, Scalar(CoronaMaterialProperties.OpacityLevel)));
		}
		else
		{
			FLinearColor OpacityColor = CoronaMaterialProperties.Opacity.Value * CoronaMaterialProperties.OpacityLevel;

			// Don't consider almost full opacity as transparency to prevent creating opacity expression(which would lead to making non-opaque shader)
			if (!OpacityColor.Equals(FLinearColor::White)) 
			{
				OpacityExpression = &Desaturate(Color(OpacityColor));
			}
		}
	}

	// Refraction
	CoronaMaterialProperties.Refraction.Weight *= CoronaMaterialProperties.RefractionLevel;
	CoronaMaterialProperties.Refraction.Value *= CoronaMaterialProperties.Refraction.Weight;
	CoronaMaterialProperties.RefractionMap.Weight *= CoronaMaterialProperties.RefractionLevel;

	IDatasmithMaterialExpression* RefractionExpression = nullptr;
	{
		TOptional< FLinearColor > OptionalRefractionColor;

		if ( !CoronaMaterialProperties.Refraction.Value.IsAlmostBlack() )
		{
			OptionalRefractionColor = CoronaMaterialProperties.Refraction.Value;
		}

		RefractionExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, CoronaMaterialProperties.RefractionMap, TEXT("Refraction"), OptionalRefractionColor, TOptional< float >() );
	}

	// UE Roughness
	{
		IDatasmithMaterialExpressionGeneric* MultiplyGlossiness = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		MultiplyGlossiness->SetExpressionName( TEXT("Multiply") );

		GlossinessExpression->ConnectExpression( *MultiplyGlossiness->GetInput(0) );
		GlossinessExpression->ConnectExpression( *MultiplyGlossiness->GetInput(1) );

		IDatasmithMaterialExpression* RoughnessOutput = MultiplyGlossiness;

		IDatasmithMaterialExpressionGeneric* OneMinusRougnessExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		OneMinusRougnessExpression->SetExpressionName( TEXT("OneMinus") );

		MultiplyGlossiness->ConnectExpression( *OneMinusRougnessExpression->GetInput(0) );

		RoughnessOutput = OneMinusRougnessExpression;

		IDatasmithMaterialExpressionGeneric* PowRoughnessExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		PowRoughnessExpression->SetExpressionName( TEXT("Power") );

		TSharedRef< IDatasmithKeyValueProperty > PowRoughnessValue = FDatasmithSceneFactory::CreateKeyValueProperty( TEXT("ConstExponent") );
		PowRoughnessValue->SetPropertyType( EDatasmithKeyValuePropertyType::Float );
		PowRoughnessValue->SetValue( *LexToString( 1.5f ) );

		PowRoughnessExpression->AddProperty( PowRoughnessValue );

		RoughnessOutput->ConnectExpression( *PowRoughnessExpression->GetInput(0) );
		PowRoughnessExpression->ConnectExpression( PbrMaterialElement->GetRoughness() );
	}

	IDatasmithMaterialExpressionGeneric* ReflectionFresnelExpression = nullptr;

	IDatasmithMaterialExpressionGeneric* IORFactor = nullptr;

	{
		DiffuseExpression->ConnectExpression( PbrMaterialElement->GetBaseColor() );

		IDatasmithMaterialExpressionGeneric* DiffuseLerpExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		DiffuseLerpExpression->SetExpressionName( TEXT("LinearInterpolate") );

		DiffuseLerpExpression->ConnectExpression( PbrMaterialElement->GetBaseColor() );

		IDatasmithMaterialExpression* ReflectionIOR = nullptr;

		{
			TGuardValue< bool > SetIsMonoChannel( ConvertState.bIsMonoChannel, true );
			ReflectionIOR = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, CoronaMaterialProperties.ReflectionIORMap, TEXT("Fresnel IOR"), TOptional< FLinearColor >(), CoronaMaterialProperties.ReflectionIOR );
		}

		ReflectionIOR->SetName( TEXT("Fresnel IOR") );

		IDatasmithMaterialExpressionScalar* MinusOneFresnelIOR = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		MinusOneFresnelIOR->GetScalar() = -1.f;

		IDatasmithMaterialExpressionGeneric* AddAdjustFresnelIOR = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		AddAdjustFresnelIOR->SetExpressionName( TEXT("Add") );

		ReflectionIOR->ConnectExpression( *AddAdjustFresnelIOR->GetInput(0) );
		MinusOneFresnelIOR->ConnectExpression( *AddAdjustFresnelIOR->GetInput(1) );

		IORFactor = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		IORFactor->SetExpressionName( TEXT("Multiply") );

		IDatasmithMaterialExpressionScalar* ScaleIORScalar = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		ScaleIORScalar->GetScalar() = 0.02f;

		AddAdjustFresnelIOR->ConnectExpression( *IORFactor->GetInput(0) );
		ScaleIORScalar->ConnectExpression( *IORFactor->GetInput(1) );

		IDatasmithMaterialExpressionGeneric* BaseColorIORPow = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		BaseColorIORPow->SetExpressionName( TEXT("Power") );

		IDatasmithMaterialExpressionScalar* BaseColorIORPowScalar = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		BaseColorIORPowScalar->GetScalar() = 0.5f;

		IORFactor->ConnectExpression( *BaseColorIORPow->GetInput(0) );
		BaseColorIORPowScalar->ConnectExpression( *BaseColorIORPow->GetInput(1) );

		IDatasmithMaterialExpressionGeneric* DiffuseIORLerpExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		DiffuseIORLerpExpression->SetExpressionName( TEXT("LinearInterpolate") );

		DiffuseExpression->ConnectExpression( *DiffuseIORLerpExpression->GetInput(0) );
		ReflectionExpression->ConnectExpression( *DiffuseIORLerpExpression->GetInput(1) );
		BaseColorIORPow->ConnectExpression( *DiffuseIORLerpExpression->GetInput(2) );

		DiffuseExpression->ConnectExpression( *DiffuseLerpExpression->GetInput(0) );
		DiffuseIORLerpExpression->ConnectExpression( *DiffuseLerpExpression->GetInput(1) );
		ReflectionIntensityExpression->ConnectExpression( *DiffuseLerpExpression->GetInput(2) );
	}

	// UE Metallic
	IDatasmithMaterialExpression* MetallicExpression = nullptr;
	{
		IDatasmithMaterialExpressionGeneric* MetallicIORPow = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		MetallicIORPow->SetExpressionName( TEXT("Power") );

		IDatasmithMaterialExpressionScalar* MetallicIORPowScalar = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		MetallicIORPowScalar->GetScalar() = 0.2f;

		IORFactor->ConnectExpression( *MetallicIORPow->GetInput(0) );
		MetallicIORPowScalar->ConnectExpression( *MetallicIORPow->GetInput(1) );

		IDatasmithMaterialExpressionGeneric* MultiplyIORExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		MultiplyIORExpression->SetExpressionName( TEXT("Multiply") );

		ReflectionIntensityExpression->ConnectExpression( *MultiplyIORExpression->GetInput(0) );
		MetallicIORPow->ConnectExpression( *MultiplyIORExpression->GetInput(1) );

		MetallicExpression = MultiplyIORExpression;
	}

	if ( MetallicExpression )
	{
		MetallicExpression->ConnectExpression( PbrMaterialElement->GetMetallic() );
	}

	// UE Specular
	if ( MetallicExpression )
	{
		MetallicExpression->ConnectExpression( PbrMaterialElement->GetSpecular() );
	}

	// UE Opacity & Refraction
	if ( !FMath::IsNearlyZero( CoronaMaterialProperties.RefractionLevel ) && ( OpacityExpression || RefractionExpression ) )
	{
		IDatasmithMaterialExpression* UEOpacityExpression = nullptr;

		if ( RefractionExpression )
		{
			IDatasmithMaterialExpressionGeneric* RefractionIntensity = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
			RefractionIntensity->SetExpressionName( TEXT("Desaturation") );

			RefractionExpression->ConnectExpression( *RefractionIntensity->GetInput(0) );

			IDatasmithMaterialExpressionGeneric* OneMinusRefraction = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
			OneMinusRefraction->SetExpressionName( TEXT("OneMinus") );

			RefractionIntensity->ConnectExpression( *OneMinusRefraction->GetInput(0) );

			if ( OpacityExpression )
			{
				IDatasmithMaterialExpressionGeneric* LerpOpacityRefraction = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				LerpOpacityRefraction->SetExpressionName( TEXT("LinearInterpolate") );

				OpacityExpression->ConnectExpression( *LerpOpacityRefraction->GetInput(0) );
				OneMinusRefraction->ConnectExpression( *LerpOpacityRefraction->GetInput(1) );
				OpacityExpression->ConnectExpression( *LerpOpacityRefraction->GetInput(2) );

				UEOpacityExpression = LerpOpacityRefraction;
			}
			else
			{
				UEOpacityExpression = OneMinusRefraction;
			}
		}
		else
		{
			UEOpacityExpression = OpacityExpression;
		}

		if ( UEOpacityExpression )
		{
			UEOpacityExpression->ConnectExpression( PbrMaterialElement->GetOpacity() );
			PbrMaterialElement->SetShadingModel( EDatasmithShadingModel::ThinTranslucent );

			IDatasmithMaterialExpressionGeneric* ThinTranslucencyMaterialOutput = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
			ThinTranslucencyMaterialOutput->SetExpressionName( TEXT("ThinTranslucentMaterialOutput") );

			// Transmittance color
			IDatasmithMaterialExpressionColor* TransmittanceExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
			TransmittanceExpression->GetColor() = FLinearColor::White;
			TransmittanceExpression->ConnectExpression( *ThinTranslucencyMaterialOutput->GetInput(0) );
		}
	}
	else
	{
		if (OpacityExpression)
		{
			Connect(PbrMaterialElement->GetOpacity(), *OpacityExpression);
		}
	}

	MaterialElement = PbrMaterialElement;
}

bool FDatasmithMaxCoronaBlendMaterialToUEPbr::IsSupported( Mtl* Material )
{
	using namespace DatasmithMaxCoronaMaterialsToUEPbrImpl;

	if (!Material)
	{
		return false;
	}

	FMaxCoronaBlendMaterial CoronaBlendMaterialProperties = ParseCoronaBlendMaterialProperties(*Material);
	bool bAllMaterialsSupported = true;

	if (CoronaBlendMaterialProperties.BaseMaterial)
	{
		FDatasmithMaxMaterialsToUEPbr* MaterialConverter = FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter(CoronaBlendMaterialProperties.BaseMaterial);
		bAllMaterialsSupported &= MaterialConverter != nullptr;
	}
	else
	{
		return false;
	}

	for (int CoatIndex = 0; bAllMaterialsSupported && CoatIndex < FMaxCoronaBlendMaterial::MaximumNumberOfCoat; ++CoatIndex)
	{
		const FMaxCoronaBlendMaterial::FCoronaCoatMaterialProperties& CoatedMaterial = CoronaBlendMaterialProperties.CoatedMaterials[CoatIndex];

		if (CoatedMaterial.Material != nullptr && CoatedMaterial.Mask.bEnabled)
		{
			//Only support if all the blended materials are UEPbr materials.
			FDatasmithMaxMaterialsToUEPbr* MaterialConverter = FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter(CoatedMaterial.Material);
			bAllMaterialsSupported &= MaterialConverter != nullptr;
		}
	}

	return bAllMaterialsSupported;
}

void FDatasmithMaxCoronaBlendMaterialToUEPbr::Convert( TSharedRef<IDatasmithScene> DatasmithScene, TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement, Mtl* Material, const TCHAR* AssetsPath )
{
	using namespace DatasmithMaxCoronaMaterialsToUEPbrImpl;

	TSharedRef< IDatasmithUEPbrMaterialElement > PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial(GetMaterialName(Material));
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	FMaxCoronaBlendMaterial CoronaBlendMaterialProperties = ParseCoronaBlendMaterialProperties( *Material );

	//Exporting the base material.
	IDatasmithMaterialExpressionFunctionCall* BaseMaterialFunctionCall = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();

	if (TSharedPtr<IDatasmithBaseMaterialElement> ExportedMaterial = FDatasmithMaxMatExport::ExportUniqueMaterial(DatasmithScene, CoronaBlendMaterialProperties.BaseMaterial, AssetsPath))
	{
		BaseMaterialFunctionCall->SetFunctionPathName(ExportedMaterial->GetName());
	}

	//Exporting the blended materials.
	IDatasmithMaterialExpression* PreviousExpression = BaseMaterialFunctionCall;
	for (int CoatIndex = 0; CoatIndex < FMaxCoronaBlendMaterial::MaximumNumberOfCoat; ++CoatIndex)
	{
		const FMaxCoronaBlendMaterial::FCoronaCoatMaterialProperties& CoatedMaterial = CoronaBlendMaterialProperties.CoatedMaterials[CoatIndex];

		if (CoatedMaterial.Material != nullptr)
		{
			IDatasmithMaterialExpressionFunctionCall* BlendFunctionCall = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
			BlendFunctionCall->SetFunctionPathName(TEXT("/Engine/Functions/MaterialLayerFunctions/MatLayerBlend_Standard.MatLayerBlend_Standard"));
			PreviousExpression->ConnectExpression(*BlendFunctionCall->GetInput(0));
			PreviousExpression = BlendFunctionCall;

			IDatasmithMaterialExpressionFunctionCall* LayerMaterialFunctionCall = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
			TSharedPtr<IDatasmithBaseMaterialElement> LayerMaterial = FDatasmithMaxMatExport::ExportUniqueMaterial(DatasmithScene, CoatedMaterial.Material, AssetsPath);

			if ( !LayerMaterial )
			{
				continue;
			}

			LayerMaterialFunctionCall->SetFunctionPathName(LayerMaterial->GetName());
			LayerMaterialFunctionCall->ConnectExpression(*BlendFunctionCall->GetInput(1));

			IDatasmithMaterialExpressionScalar* AmountExpression = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
			AmountExpression->SetName( TEXT("Layer Amount") );
			AmountExpression->GetScalar() = CoatedMaterial.Amount;

			IDatasmithMaterialExpression* MaskExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue(this, CoatedMaterial.Mask, TEXT("MixAmount"),
				FLinearColor::White, TOptional< float >());

			IDatasmithMaterialExpressionGeneric* AlphaExpression = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			AlphaExpression->SetExpressionName(TEXT("Multiply"));

			//AlphaExpression is nullptr only when there is no mask and the mask weight is ~100% so we add scalar 0 instead.
			if (!MaskExpression)
			{
				IDatasmithMaterialExpressionScalar* WeightExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
				WeightExpression->GetScalar() = 0.f;
				MaskExpression = WeightExpression;
			}

			AmountExpression->ConnectExpression(*AlphaExpression->GetInput(0));
			MaskExpression->ConnectExpression(*AlphaExpression->GetInput(1));

			AlphaExpression->ConnectExpression(*BlendFunctionCall->GetInput(2));
		}
	}

	PbrMaterialElement->SetUseMaterialAttributes(true);
	PreviousExpression->ConnectExpression(PbrMaterialElement->GetMaterialAttributes());
	MaterialElement = PbrMaterialElement;
}

bool FDatasmithMaxCoronaLightMaterialToUEPbr::IsSupported(Mtl* Material)
{
	return true;
}

void FDatasmithMaxCoronaLightMaterialToUEPbr::Convert(TSharedRef<IDatasmithScene> DatasmithScene,
	TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement, Mtl* Material, const TCHAR* AssetsPath)
{
	if ( !Material )
	{
		return;
	}

	TSharedRef<IDatasmithUEPbrMaterialElement> PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial(GetMaterialName(Material));
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	DatasmithMaxCoronaMaterialsToUEPbrImpl::FCoronaLightMaterial MaterialProperties;
	MaterialProperties.Parse(*Material);

	Connect(PbrMaterialElement->GetEmissiveColor(),
		COMPOSE_OR_DEFAULT2(nullptr, Multiply,
			TextureOrColor(TEXT("Emissive Color"), MaterialProperties.EmitTexture, FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(MaterialProperties.EmitColor)),
			&Scalar(MaterialProperties.Multiplier * 2.0f))
	);

	if (IDatasmithMaterialExpression* OpacictyExpression = ConvertTexmap(MaterialProperties.ClipTexture))
	{
		PbrMaterialElement->SetBlendMode(/*EBlendMode::BLEND_Masked*/1); // Set blend mode when there's opacity mask
		Connect(PbrMaterialElement->GetOpacity(), OpacictyExpression);
	}

	PbrMaterialElement->SetShadingModel(EDatasmithShadingModel::Unlit);

	MaterialElement = PbrMaterialElement;
}

bool FDatasmithMaxCoronaPhysicalMaterialToUEPbr::IsSupported(Mtl* Material)
{
	return true;
}

void FDatasmithMaxCoronaPhysicalMaterialToUEPbr::Convert(TSharedRef<IDatasmithScene> DatasmithScene,
	TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement, Mtl* Material, const TCHAR* AssetsPath)
{
	if ( !Material )
	{
		return;
	}

	TSharedRef< IDatasmithUEPbrMaterialElement > PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial( GetMaterialName(Material) );
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	DatasmithMaxCoronaMaterialsToUEPbrImpl::FCoronaPhysicalMaterial MaterialProperties;
	MaterialProperties.Parse(*Material);

	// todo: move out?
	const TCHAR* PARAM_NAME_BASELEVEL = TEXT("Base Level");
	const TCHAR* PARAM_NAME_BASECOLOR = TEXT("Base Color");
	const TCHAR* PARAM_NAME_BASECOLORMAP_WEIGHT = TEXT("Base Color Texture Amount");

	const TCHAR* PARAM_NAME_EMISSIVELEVEL = TEXT("Emissive Level");
	const TCHAR* PARAM_NAME_EMISSIVECOLOR = TEXT("Emissive Color");
	const TCHAR* PARAM_NAME_EMISSIVECOLORMAP_WEIGHT = TEXT("Emissive Color Texture Amount");

	const TCHAR* PARAM_NAME_OPACITY_LEVEL = TEXT("Opacity Level");
	const TCHAR* PARAM_NAME_OPACITY_MAP_WEIGHT = TEXT("Opacity Texture Amount");

	const TCHAR* PARAM_NAME_ROUGHNESS = TEXT("Roughness");
	const TCHAR* PARAM_NAME_ROUGHNESSMAP_WEIGHT = TEXT("Roughness Map Amount");

	const TCHAR* PARAM_NAME_BASEIOR = TEXT("Base IOR");
	const TCHAR* PARAM_NAME_BASEIORMAP_WEIGHT = TEXT("Base IOR Map Weight");

	const TCHAR* PARAM_NAME_REFRACTIONAMOUNT = TEXT("Refraction Amount");
	const TCHAR* PARAM_NAME_REFRACTIONMAP_WEIGHT = TEXT("Refraction Map Weight");

	const TCHAR* PARAM_NAME_BUMPMAP = TEXT("Bump Map");

	const TCHAR* PARAM_NAME_ABSORPTIONDISTANCE = TEXT("Absorption Distance");



	IDatasmithMaterialExpression* BaseTexmapExpression = BlendTexmapWithColor(MaterialProperties.BaseTexmap, MaterialProperties.BaseColor, PARAM_NAME_BASECOLOR, PARAM_NAME_BASECOLORMAP_WEIGHT);
	IDatasmithMaterialExpression& DiffuseColorExpression = BaseTexmapExpression
		                                                       ? Multiply(*BaseTexmapExpression, Scalar(MaterialProperties.BaseLevel, PARAM_NAME_BASELEVEL))
		                                                       : Color(ScaleRGB(MaterialProperties.BaseColor, MaterialProperties.BaseLevel), PARAM_NAME_BASECOLOR);

	// Opacity Color
	IDatasmithMaterialExpression* OpacityTexmapExpression = BlendTexmapWithColor(MaterialProperties.OpacityTexmap, MaterialProperties.OpacityColor, nullptr, PARAM_NAME_OPACITY_MAP_WEIGHT);
	IDatasmithMaterialExpression* OpacityColorExpression = nullptr;
	if (OpacityTexmapExpression)
	{
		OpacityColorExpression = &Multiply(*OpacityTexmapExpression, Scalar(MaterialProperties.OpacityLevel, PARAM_NAME_OPACITY_LEVEL));
	}
	else
	{
		FLinearColor OpacityColor = MaterialProperties.OpacityColor * MaterialProperties.OpacityLevel;
		// Don't consider almost full opacity as transparency to prevent creating opacity expression(which would lead to making non-opaque shader)
		if (!OpacityColor.Equals(FLinearColor::White)) 
		{
			OpacityColorExpression = &Color(OpacityColor);
		}
	}

	// Emissive
	if (!FMath::IsNearlyZero(MaterialProperties.SelfIllumLevel))
	{
		IDatasmithMaterialExpression* SelfIllumTexmapExpression = BlendTexmapWithColor(MaterialProperties.SelfIllumTexmap, MaterialProperties.SelfIllumColor, PARAM_NAME_EMISSIVECOLOR, PARAM_NAME_EMISSIVECOLORMAP_WEIGHT);
		Connect(PbrMaterialElement->GetEmissiveColor(), SelfIllumTexmapExpression
															? Multiply(*SelfIllumTexmapExpression,
																Scalar(MaterialProperties.SelfIllumLevel, PARAM_NAME_EMISSIVELEVEL))
															: Color( ScaleRGB(MaterialProperties.SelfIllumColor, MaterialProperties.SelfIllumLevel), PARAM_NAME_EMISSIVECOLOR));
	}

	// Refraction
	IDatasmithMaterialExpression* RefractionAmountMapExpression = BlendTexmapWithScalar(MaterialProperties.RefractionAmountTexmap, MaterialProperties.RefractionAmount, PARAM_NAME_REFRACTIONAMOUNT, PARAM_NAME_REFRACTIONMAP_WEIGHT);
	bool bHasRefraction = MaterialProperties.RefractionAmount > 0.0f || RefractionAmountMapExpression;

	// Roughness
	IDatasmithMaterialExpression* RoughnessTexmapExpression = BlendTexmapWithScalar(MaterialProperties.BaseRoughnessTexmap, MaterialProperties.BaseRoughness, PARAM_NAME_ROUGHNESS, PARAM_NAME_ROUGHNESSMAP_WEIGHT);
	IDatasmithMaterialExpression& RoughnessExpression = RoughnessTexmapExpression ? *RoughnessTexmapExpression
		                                                                          : Scalar(MaterialProperties.BaseRoughness, PARAM_NAME_ROUGHNESS);
	bool bRoughnessAsGlossiness = MaterialProperties.RoughnessMode == 1;
	Connect(PbrMaterialElement->GetRoughness(), bRoughnessAsGlossiness ? OneMinus(RoughnessExpression) : RoughnessExpression);

	// Specular
	// Specular in Unreal is F0 divided by 0.08 (i.e. F0=S*0.08 calculated in Unreal, where S is UE PBR Specular input )
	// and F0 = (ior-1)^2/(ior+1)^2
	// baseIor in 'normal' Corona mode(iorMode=0) is actual index of refraction, but in 'Disney' mode (iorMode=1) baseIor means the same "F0/0.08" as in Unreal
	// then, in Disney mode, (actual)Ior is (sqrt(Specular*0.08) + 1)/(1 - sqrt(Specular*0.08)
	IDatasmithMaterialExpression* BaseIorMapExpression = BlendTexmapWithScalar(MaterialProperties.BaseIorTexmap, MaterialProperties.BaseIor, PARAM_NAME_BASEIOR, PARAM_NAME_BASEIORMAP_WEIGHT);

	// Compute Ior (even for Disney mode) we'll need it for refraction anyway
	float Ior = (MaterialProperties.IorMode == 0)
		? MaterialProperties.BaseIor
		: ((MaterialProperties.BaseIor < 1.0f ) ? (FMath::Sqrt(MaterialProperties.BaseIor*0.08) + 1) / (1 - FMath::Sqrt(MaterialProperties.BaseIor*0.08)) : 1.8f);

	IDatasmithMaterialExpression& SpecularExpression = (MaterialProperties.IorMode == 0)
		? (BaseIorMapExpression
			? Divide(
				Power(Subtract(*BaseIorMapExpression, 1.0f), 2.0f), 
				Multiply(Power(Add(*BaseIorMapExpression, 1.0f), 2.0f), 0.08f))
			: Scalar((Ior > 1.0f ? FMath::Square(Ior-1)/FMath::Square(Ior+1) : 0.0f) / 0.08f))
		: (BaseIorMapExpression ? *BaseIorMapExpression : Scalar(MaterialProperties.BaseIor));

	IDatasmithMaterialExpression* BaseColorExpression = &DiffuseColorExpression;

	// Build material based on specific configuration
	if ((MaterialProperties.MetalnessMode == 1) && !MaterialProperties.MetalnessTexmap.IsMapPresentAndEnabled()) // Pure metal
	{
		Connect(PbrMaterialElement->GetMetallic(), &Scalar(1.0f));

		if (OpacityColorExpression)
		{
			Connect(PbrMaterialElement->GetOpacity(), Desaturate(*OpacityColorExpression));
			BaseColorExpression = &Multiply(DiffuseColorExpression, *OpacityColorExpression); // Scale base color by opacity color to imitate colored opacity
		}
	}
	else if (OpacityColorExpression || bHasRefraction)  // Transparent non-metal
	{
		if (MaterialProperties.bUseThinMode)
		{
			// useThinMode is used for 'leaves' which means that opacity controls where a leaf pixel is visible

			PbrMaterialElement->SetShadingModel(EDatasmithShadingModel::ThinTranslucent);
			IDatasmithMaterialExpressionGeneric* ThinTranslucencyMaterialOutput = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			ThinTranslucencyMaterialOutput->SetExpressionName(TEXT("ThinTranslucentMaterialOutput"));

			IDatasmithMaterialExpression* TranslucencyExpression = nullptr;

			// Translucency
			if (MaterialProperties.TranslucencyFraction > 0.0f || MaterialProperties.TranslucencyFractionTexmap.IsMapPresentAndEnabled())
			{
				IDatasmithMaterialExpression* TranslucencyColorTexmapExpression = BlendTexmapWithColor(MaterialProperties.TranslucencyColorTexmap, MaterialProperties.TranslucencyColor);
				IDatasmithMaterialExpression& TranslucencyColorExpression = TranslucencyColorTexmapExpression ? *TranslucencyColorTexmapExpression : Color(MaterialProperties.TranslucencyColor);
				IDatasmithMaterialExpression* TranslucencyFractionTexmap = ConvertTexmap(MaterialProperties.TranslucencyFractionTexmap);

				// todo: TranslucencyFractionTexmap
				TranslucencyExpression = &TranslucencyColorExpression;
			}

			// Thin Absorption
			if (bHasRefraction) // Only when there's some refraction thin absorption is affecting
			{
				IDatasmithMaterialExpression* ThinAbsorptionTexmapExpression = BlendTexmapWithColor(MaterialProperties.ThinAbsorptionTexmap, MaterialProperties.ThinAbsorptionColor);
				IDatasmithMaterialExpression& ThinAbsorptionColorExpression = ThinAbsorptionTexmapExpression ? *ThinAbsorptionTexmapExpression : Color(MaterialProperties.ThinAbsorptionColor);

				IDatasmithMaterialExpression& RefractionAmountExpression = RefractionAmountMapExpression ? *RefractionAmountMapExpression : Scalar(MaterialProperties.RefractionAmount);

				TranslucencyExpression = &(TranslucencyExpression ? Lerp(*TranslucencyExpression, ThinAbsorptionColorExpression, RefractionAmountExpression) : Multiply(ThinAbsorptionColorExpression, RefractionAmountExpression));
			}

			IDatasmithMaterialExpression* OpacityExpression = OpacityColorExpression;

			// Refraction (factor into opacity)
			if (bHasRefraction)
			{
				IDatasmithMaterialExpression& RefractionAmountExpression = RefractionAmountMapExpression ? *RefractionAmountMapExpression : Scalar(MaterialProperties.RefractionAmount);

				IDatasmithMaterialExpression& InvertedRefraction = OneMinus(RefractionAmountExpression); // Invert refaction to consider it Opacity factor for ThinMode (i.e. full refraction is Zero opacity)
				OpacityExpression = &(OpacityColorExpression ? Multiply(*OpacityColorExpression, InvertedRefraction) : InvertedRefraction);
			}

			OpacityExpression = COMPOSE_OR_NULL(Desaturate, OpacityExpression);
			Connect(PbrMaterialElement->GetOpacity(), OpacityExpression);

			// Multiply specular by opacity to override default 0.5 and disable specular on fully transparent pixels(e.g. 'holes' in green leaves)
			Connect(PbrMaterialElement->GetSpecular(), OpacityColorExpression ? Multiply(SpecularExpression, *OpacityColorExpression) : SpecularExpression);

			// Need to turn transmittance into White color in fully transparent areas to disable absorption
			// Actually fully transparent  - not transparent because of full refraction but with zero source Corona Opacity 
			Connect(*ThinTranslucencyMaterialOutput->GetInput(0), (TranslucencyExpression && OpacityColorExpression) 
				? &Lerp(Color(FLinearColor::White), *TranslucencyExpression, *OpacityColorExpression)
				: TranslucencyExpression);

			PbrMaterialElement->SetTwoSided(true); // Set two-sided for thin materials(usually used for thin leaves)
		}
		else if (bHasRefraction) // Refractive glass
		{
			IDatasmithMaterialExpression& InvertedRefraction = OneMinus(RefractionAmountMapExpression ? *RefractionAmountMapExpression : Scalar(MaterialProperties.RefractionAmount));

			IDatasmithMaterialExpression& OpacityExpression = Desaturate(OpacityColorExpression ? Multiply(*OpacityColorExpression, InvertedRefraction) : InvertedRefraction);

			IDatasmithMaterialExpression& RefractionAmountExpression = RefractionAmountMapExpression ? *RefractionAmountMapExpression : Scalar(MaterialProperties.RefractionAmount);

			IDatasmithMaterialExpression& Multiply09 = Multiply(Scalar(0.9f), RefractionAmountExpression);
			IDatasmithMaterialExpression& FresnelOpacity = OneMinus(CalcIORComplex(Ior, 0, Multiply(Scalar(0.5f), RefractionAmountExpression), Multiply09));

			Connect(PbrMaterialElement->GetOpacity(), Desaturate(Multiply(FresnelOpacity, OpacityExpression)));

			IDatasmithMaterialExpression& BaseIorExpression = Scalar(Ior, PARAM_NAME_BASEIOR);

			Connect(PbrMaterialElement->GetRefraction(), PathTracingQualitySwitch(Lerp(Scalar(1.0f), BaseIorExpression, Fresnel()), BaseIorExpression));
			Connect(PbrMaterialElement->GetSpecular(), OpacityColorExpression ? Multiply(SpecularExpression, *OpacityColorExpression) : SpecularExpression);

			BaseColorExpression = &Multiply(Scalar(0.6f), Power(Multiply09, Scalar(5.0f)));
			if (OpacityColorExpression)
			{
				BaseColorExpression = &Lerp(*BaseColorExpression, DiffuseColorExpression, *OpacityColorExpression);
			}

			Connect(PbrMaterialElement->GetMetallic(), ConvertTexmap(MaterialProperties.MetalnessTexmap));

			// Compute volumetric properties for PathTracer
			IDatasmithMaterialExpression* TransmittanceExpression = nullptr;
			bool bVolumetricAbsorptionEnabled = MaterialProperties.AttenuationDistance > 0.0f;
			if (bVolumetricAbsorptionEnabled && (!MaterialProperties.VolumetricAbsorptionColor.Equals(FLinearColor::White) || MaterialProperties.VolumetricAbsorptionTexmap.IsMapPresentAndEnabled()))
			{
				IDatasmithMaterialExpression* VolumetricAbsorptionTexmapExpression = BlendTexmapWithColor(MaterialProperties.VolumetricAbsorptionTexmap, MaterialProperties.VolumetricAbsorptionColor);
				IDatasmithMaterialExpression& VolumetricAbsorptionColorExpression = VolumetricAbsorptionTexmapExpression ? *VolumetricAbsorptionTexmapExpression : Color(MaterialProperties.VolumetricAbsorptionColor);

				TransmittanceExpression = &Power(VolumetricAbsorptionColorExpression, Divide(Scalar(100.0f), Scalar(MaterialProperties.AttenuationDistance, PARAM_NAME_ABSORPTIONDISTANCE)));
			}

			if (TransmittanceExpression)
			{
				IDatasmithMaterialExpressionGeneric* AbsorptionMediumMaterialOutput = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
				AbsorptionMediumMaterialOutput->SetExpressionName(TEXT("AbsorptionMediumMaterialOutput"));

				// Need to turn transmittance into White color in fully transparent areas to disable absorption
				Connect(*AbsorptionMediumMaterialOutput->GetInput(0),
					Lerp(Color(FLinearColor::White), 
						*TransmittanceExpression, 
						OpacityColorExpression ? *OpacityColorExpression : Scalar(1.0f)));
			}
		}
		else
		{
			Connect(PbrMaterialElement->GetMetallic(), ConvertTexmap(MaterialProperties.MetalnessTexmap));
			Connect(PbrMaterialElement->GetSpecular(), Multiply(*OpacityColorExpression, SpecularExpression));
			Connect(PbrMaterialElement->GetOpacity(), COMPOSE_OR_NULL(Desaturate, OpacityColorExpression));
		}
	}
	else // Any other Opaque
	{
		Connect(PbrMaterialElement->GetMetallic(), ConvertTexmap(MaterialProperties.MetalnessTexmap));
		Connect(PbrMaterialElement->GetSpecular(), SpecularExpression);
	}
	Connect(PbrMaterialElement->GetBaseColor(), BaseColorExpression);


	// Clear Coat
	bool bHasClearCoat = MaterialProperties.ClearcoatAmount > 0 || MaterialProperties.ClearcoatAmountTexmap.IsMapPresentAndEnabled();
	if (bHasClearCoat)
	{
		IDatasmithMaterialExpression* ClearCoatExpression = TextureOrScalar(TEXT("Clear Coat Amount"), MaterialProperties.ClearcoatAmountTexmap, MaterialProperties.ClearcoatAmount);

		if (ClearCoatExpression)
		{
			IDatasmithMaterialExpression& ClearCoatAmountByFresnelExpression = MaterialProperties.ClearcoatIorTexmap.IsMapPresentAndEnabled()
				? CalcIORComplex(*TextureOrScalar(TEXT("Clear Coat Ior"), MaterialProperties.ClearcoatIorTexmap, MaterialProperties.ClearcoatIor), Scalar(0.f), *ClearCoatExpression, Multiply(*ClearCoatExpression, Scalar(0.1f)))
				: CalcIORComplex(MaterialProperties.ClearcoatIor, 0, *ClearCoatExpression, Multiply(*ClearCoatExpression, Scalar(0.1f)));

			// Don't use Ior in Path Tracer(doesn't work well). May implement when Path Tracer implements CLear Coat Ior input
			Connect(PbrMaterialElement->GetClearCoat(), PathTracingQualitySwitch(ClearCoatAmountByFresnelExpression, *ClearCoatExpression));
		}
		
		Connect(PbrMaterialElement->GetClearCoatRoughness(), TextureOrScalar(TEXT("Clear Coat Roughness"), MaterialProperties.ClearcoatRoughnessTexmap, MaterialProperties.ClearcoatRoughness));
		PbrMaterialElement->SetShadingModel(EDatasmithShadingModel::ClearCoat);
	}

	// Convert all bump/normal maps
	ConvertState.DefaultTextureMode = EDatasmithTextureMode::Bump; // Will change to normal if we pass through a normal map texmap
	ConvertState.bCanBake = false; // Current baking fails to produce proper normal maps

	if (MaterialProperties.BaseBumpTexmap.IsMapPresentAndEnabled())
	{
		IDatasmithMaterialExpression* BumpExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue(this, MaterialProperties.BaseBumpTexmap, PARAM_NAME_BUMPMAP, TOptional<FLinearColor>(), TOptional<float>());
	
		if ( BumpExpression )
		{
			BumpExpression->ConnectExpression(PbrMaterialElement->GetNormal());
			BumpExpression->SetName(PARAM_NAME_BUMPMAP);
		}
	}

	if (bHasClearCoat)
	{
		IDatasmithMaterialExpression* ClearCoatBumpExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue(this, MaterialProperties.ClearcoatBumpTexmap, PARAM_NAME_BUMPMAP, TOptional<FLinearColor>(), TOptional<float>());

		if (ClearCoatBumpExpression)
		{
			// Clear Coat uses additional output
			IDatasmithMaterialExpressionGeneric* MultiplyDiffuseLevelExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
			MultiplyDiffuseLevelExpression->SetExpressionName( TEXT("ClearCoatNormalCustomOutput") );
			Connect(*MultiplyDiffuseLevelExpression->GetInput(0), ClearCoatBumpExpression);
		}
	}
	
	MaterialElement = PbrMaterialElement;
}
