// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithMaxMaterialsToUEPbr.h"

class FDatasmithMaxCoronaMaterialsToUEPbr : public FDatasmithMaxMaterialsToUEPbrExpressions
{
public:
	FDatasmithMaxCoronaMaterialsToUEPbr();

	virtual bool IsSupported( Mtl* Material ) override;
	virtual void Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath ) override;
};

class FDatasmithMaxCoronaBlendMaterialToUEPbr : public FDatasmithMaxCoronaMaterialsToUEPbr 
{
public:
	virtual bool IsSupported( Mtl* Material ) override;
	virtual void Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath ) override;
};

class FDatasmithMaxCoronaLightMaterialToUEPbr : public FDatasmithMaxCoronaMaterialsToUEPbr
{
public:
	virtual bool IsSupported( Mtl* Material ) override;
	virtual void Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath ) override;
};

class FDatasmithMaxCoronaPhysicalMaterialToUEPbr : public FDatasmithMaxCoronaMaterialsToUEPbr
{
public:
	virtual bool IsSupported( Mtl* Material ) override;
	virtual void Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath ) override;
};
