// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaxMaterialsToUEPbr/DatasmithMaxCoronaTexmapToUEPbr.h"

#include "DatasmithMaterialElements.h"
#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxSceneExporter.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithMaxWriter.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxTexmapToUEPbr.h"

#include "DatasmithMaterialsUtils.h"
#include "DatasmithSceneFactory.h"

#include "Misc/Paths.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "bitmap.h"
	#include "gamma.h"
	#include "maxtypes.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"


namespace DatasmithMaxCoronaTexmapToUEPbrImpl
{
	struct FMaxCoronaAOParameters
	{
		DatasmithMaxTexmapParser::FMapParameter UnoccludedMap;

		FLinearColor UnoccludedColor;
	};

	FMaxCoronaAOParameters ParseCoronaAOProperties( Texmap& InTexmap )
	{
		FMaxCoronaAOParameters CoronaAOParameters;

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		const int NumParamBlocks = InTexmap.NumParamBlocks();
		for ( int j = 0; j < NumParamBlocks; j++ )
		{
			IParamBlock2* ParamBlock2 = InTexmap.GetParamBlockByID( (short)j );
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("colorUnoccluded") ) == 0 )
				{
					CoronaAOParameters.UnoccludedColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, CurrentTime ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapUnoccluded")) == 0)
				{
					CoronaAOParameters.UnoccludedMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmapUnoccludedOn")) == 0)
				{
					CoronaAOParameters.UnoccludedMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		return CoronaAOParameters;
	}
}

bool FDatasmithMaxCoronaAOToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	return InTexmap ? (bool)( InTexmap->ClassID() == CORONAAOCLASS ) : false;
}

IDatasmithMaterialExpression* FDatasmithMaxCoronaAOToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	if ( !InTexmap )
	{
		return nullptr;
	}

	DatasmithMaxCoronaTexmapToUEPbrImpl::FMaxCoronaAOParameters CoronaAOParameters = DatasmithMaxCoronaTexmapToUEPbrImpl::ParseCoronaAOProperties( *InTexmap );

	return FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( MaxMaterialToUEPbr, CoronaAOParameters.UnoccludedMap, TEXT("Corona AO Unoccluded Color"), CoronaAOParameters.UnoccludedColor, TOptional< float >() );
}

bool FDatasmithMaxCoronaColorToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	return InTexmap ? (bool)( InTexmap->ClassID() == CORONACOLORCLASS ) : false;
}

IDatasmithMaterialExpression* FDatasmithMaxCoronaColorToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	if ( !InTexmap )
	{
		return nullptr;
	}

	DatasmithMaxTexmapParser::FCoronaColorParameters ColorParameters = DatasmithMaxTexmapParser::ParseCoronaColor( InTexmap );
	
	FLinearColor CoronaColor;

	switch ( ColorParameters.Method )
	{
	case 1:
		CoronaColor.R = ColorParameters.ColorHdr.X;
		CoronaColor.G = ColorParameters.ColorHdr.Y;
		CoronaColor.B = ColorParameters.ColorHdr.Z;
		break;
	case 2:
		CoronaColor = DatasmithMaterialsUtils::TemperatureToColor( ColorParameters.Temperature );
		ColorParameters.bInputIsLinear = true;
		break;
	case 3:
		if ( ColorParameters.HexColor.Len() == 7 )
		{
			FString Red = TEXT("0x") + ColorParameters.HexColor.Mid(1, 2);
			FString Green = TEXT("0x") + ColorParameters.HexColor.Mid(3, 2);
			FString Blue = TEXT("0x") + ColorParameters.HexColor.Mid(5, 2);

			CoronaColor.R = FCString::Strtoi(*Red, nullptr, 16) / 255.0f;
			CoronaColor.G = FCString::Strtoi(*Green, nullptr, 16) / 255.0f;
			CoronaColor.B = FCString::Strtoi(*Blue, nullptr, 16) / 255.0f;
			break;
		}
	default:
		CoronaColor = ColorParameters.RgbColor;
		break;
	}

	CoronaColor.R *= ColorParameters.Multiplier;
	CoronaColor.G *= ColorParameters.Multiplier;
	CoronaColor.B *= ColorParameters.Multiplier;

	if ( !ColorParameters.bInputIsLinear )
	{
		const bool bConvertToSRGB = false; // CoronaColor is already in SRGB
		CoronaColor = FLinearColor::FromSRGBColor( CoronaColor.ToFColor( bConvertToSRGB ) );
	}

	IDatasmithMaterialExpressionColor* ColorExpression = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
	ColorExpression->SetName( TEXT("Corona Color") );
	ColorExpression->GetColor() = CoronaColor;

	return ColorExpression;
}

bool FDatasmithMaxCoronalNormalToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	return ( InTexmap && InTexmap->ClassID() == CORONANORMALCLASS );
}

DatasmithMaxTexmapParser::FNormalMapParameters FDatasmithMaxCoronalNormalToUEPbr::ParseMap( Texmap* InTexmap )
{
	DatasmithMaxTexmapParser::FNormalMapParameters NormalMapParameters;

	const TimeValue CurrentTime = GetCOREInterface()->GetTime();

	for (int j = 0; j < InTexmap->NumParamBlocks(); j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for (int i = 0; i < ParamBlockDesc->count; i++)
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];
			if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("flipGreen") ) == 0 )
			{
				NormalMapParameters.bFlipGreen = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
			}
			else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("flipRed") ) == 0 )
			{
				NormalMapParameters.bFlipRed = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
			}
			else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("swapRedGreen") ) == 0 )
			{
				NormalMapParameters.bSwapRedAndGreen = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
			}
			else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("normalMap") ) == 0 )
			{
				NormalMapParameters.NormalMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime );
			}
			else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("multiplier") ) == 0 )
			{
				NormalMapParameters.NormalMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
			}
			else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("additionalBump") ) == 0 )
			{
				NormalMapParameters.BumpMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime );
			}
			else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("additionalBumpOn") ) == 0 )
			{
				NormalMapParameters.BumpMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
			}
			else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("additionalBumpStrength") ) == 0 )
			{
				NormalMapParameters.BumpMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
			}
		}

		ParamBlock2->ReleaseDesc();
	}

	return NormalMapParameters;
}

IDatasmithMaterialExpression* FDatasmithMaxCoronalNormalToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	TGuardValue< bool > ForceNormalMapsToLinearGuard( MaxMaterialToUEPbr->ConvertState.bTreatNormalMapsAsLinear, FDatasmithMaxMatWriter::GetCoronaFixNormal( InTexmap ) );
	return Super::Convert( MaxMaterialToUEPbr, InTexmap );
}

bool FDatasmithMaxCoronalBitmapToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	return ( InTexmap && InTexmap->ClassID() == CORONABITMAPCLASS );
}

float GetCoronaTexmapGamma(BitmapTex* InBitmapTex);

class FCoronaBitmapToTextureElementConverter: public DatasmithMaxDirectLink::ITexmapToTextureElementConverter
{
public:
	virtual TSharedPtr<IDatasmithTextureElement> Convert(DatasmithMaxDirectLink::FMaterialsCollectionTracker& MaterialsTracker, const FString& ActualBitmapName) override
	{
		FString Path = TEXT("");

		int NumParamBlocks = Tex->NumParamBlocks();
		
		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = Tex->GetParamBlockByID((short)j);
			// The the descriptor to 'decode'
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
			// Loop through all the defined parameters therein
			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("filename")) == 0)
				{
					Path = FDatasmithMaxSceneExporter::GetActualPath(ParamBlock2->GetStr(ParamDefinition.ID, GetCOREInterface()->GetTime()));
					continue;
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		if (Path.IsEmpty())
		{
			return {};
		}

		float Gamma = GetCoronaTexmapGamma(Tex);

		TSharedPtr< IDatasmithTextureElement > TextureElement = FDatasmithSceneFactory::CreateTexture(*ActualBitmapName);
		if (gammaMgr.IsEnabled())
		{
			TextureElement->SetRGBCurve(Gamma / 2.2f);
		}
		TextureElement->SetFile(*Path);

		return TextureElement;
	}
	BitmapTex* Tex;

	bool bIsSRGB;
};



IDatasmithMaterialExpression* FDatasmithMaxCoronalBitmapToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	DatasmithMaxTexmapParser::FCoronaBitmapParameters CoronaBitmapParameters = DatasmithMaxTexmapParser::ParseCoronaBitmap( InTexmap );

	FString ActualBitmapName = FDatasmithUtils::SanitizeObjectName(FPaths::GetBaseFilename(CoronaBitmapParameters.Path) + TEXT("_") +
		FString::SanitizeFloat(CoronaBitmapParameters.Gamma) + FDatasmithMaxMatWriter::TextureSuffix);

	const bool bIsSRGB = false;
	const bool bUseAlphaAsMono = false;

	TSharedRef<FCoronaBitmapToTextureElementConverter> Converter = MakeShared<FCoronaBitmapToTextureElementConverter>();
	Converter->Tex = (BitmapTex*)InTexmap;
	Converter->bIsSRGB = bIsSRGB;

	MaxMaterialToUEPbr->AddTexmap(InTexmap, ActualBitmapName, Converter);

	return FDatasmithMaxTexmapToUEPbrUtils::ConvertBitMap(MaxMaterialToUEPbr, InTexmap, ActualBitmapName, bUseAlphaAsMono, bIsSRGB);
}

