// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithMaxMaterialsToUEPbr.h"
#include "DatasmithMaxTexmapToUEPbr.h"

class FDatasmithMaxCoronaAOToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxCoronaColorToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxCoronalNormalToUEPbr : public FDatasmithMaxNormalToUEPbr
{
	using Super = FDatasmithMaxNormalToUEPbr;

public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	DatasmithMaxTexmapParser::FNormalMapParameters ParseMap( Texmap* InTexmap ) override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxCoronalBitmapToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

