// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaxMaterialsToUEPbr/DatasmithMaxMaterialsToUEPbr.h"

#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxLogger.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithMaxWriter.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxTexmapToUEPbr.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxCoronaMaterialsToUEPbr.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxScanlineMaterialsToUEPbr.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxVrayMaterialsToUEPbr.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxPhysicalMaterialToUEPbr.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxMentalMaterialToUEPbr.h"


DatasmithMaxDirectLink::FMaterialConversionContext* FDatasmithMaxMaterialsToUEPbrManager::Context = nullptr;

FDatasmithMaxMaterialsToUEPbr* FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter( Mtl* Material )
{
	if (Material == nullptr)
	{
		return nullptr;
	}

	FDatasmithMaxMaterialsToUEPbr* MaterialConverter = nullptr;
	const Class_ID MaterialClassID = Material->ClassID();

	if ( MaterialClassID == VRAYMATCLASS  )
	{
		static FDatasmithMaxVRayMaterialsToUEPbr VrayConverter = FDatasmithMaxVRayMaterialsToUEPbr();
		MaterialConverter = &VrayConverter;
	}
	else if ( MaterialClassID == VRAY2SIDEDMATCLASS )
	{
		static FDatasmithMaxVRay2SidedMaterialsToUEPbr Vray2SidedConverter = FDatasmithMaxVRay2SidedMaterialsToUEPbr();
		MaterialConverter = &Vray2SidedConverter;
	}
	else if ( MaterialClassID == VRAYWRAPPERMATCLASS || MaterialClassID == VRAYOVERRIDEMATCLASS )
	{
		static FDatasmithMaxVRayWrapperMaterialsToUEPbr VrayWrapperConverter = FDatasmithMaxVRayWrapperMaterialsToUEPbr();
		MaterialConverter = &VrayWrapperConverter;
	}
	else if (MaterialClassID == VRAYBLENDMATCLASS)
	{
		static FDatasmithMaxVRayBlendMaterialToUEPbr VrayBlendConverter = FDatasmithMaxVRayBlendMaterialToUEPbr();
		MaterialConverter = &VrayBlendConverter;
	}
	else if ( MaterialClassID == VRAYLIGHTMATCLASS )
	{
		static FDatasmithMaxVRayLightMaterialToUEPbr CoronaConverter = FDatasmithMaxVRayLightMaterialToUEPbr();
		MaterialConverter = &CoronaConverter;
	}
	
	else if ( MaterialClassID == STANDARDMATCLASS )
	{
		static FDatasmithMaxScanlineMaterialsToUEPbr ScanlineConverter = FDatasmithMaxScanlineMaterialsToUEPbr();
		MaterialConverter = &ScanlineConverter;
	}
	else if ( MaterialClassID == BLENDMATCLASS )
	{
		static FDatasmithMaxBlendMaterialsToUEPbr BlendConverter = FDatasmithMaxBlendMaterialsToUEPbr();
		MaterialConverter = &BlendConverter;
	}
	else if ( MaterialClassID == CORONAMATCLASS )
	{
		static FDatasmithMaxCoronaMaterialsToUEPbr CoronaConverter = FDatasmithMaxCoronaMaterialsToUEPbr();
		MaterialConverter = &CoronaConverter;
	}
	else if ( MaterialClassID == CORONALAYERMATCLASS )
	{
		static FDatasmithMaxCoronaBlendMaterialToUEPbr CoronaConverter = FDatasmithMaxCoronaBlendMaterialToUEPbr();
		MaterialConverter = &CoronaConverter;
	}
	else if ( MaterialClassID == CORONALIGHTMATCLASS )
	{
		static FDatasmithMaxCoronaLightMaterialToUEPbr CoronaConverter = FDatasmithMaxCoronaLightMaterialToUEPbr();
		MaterialConverter = &CoronaConverter;
	}
	else if ( MaterialClassID == CORONAPHYSICALMTLCLASS )
	{
		static FDatasmithMaxCoronaPhysicalMaterialToUEPbr CoronaConverter = FDatasmithMaxCoronaPhysicalMaterialToUEPbr();
		MaterialConverter = &CoronaConverter;
	}
	else if (MaterialClassID == PHYSICALMATCLASS)
	{
		static FDatasmithMaxPhysicalMaterialToUEPbr PhysicalConverter = FDatasmithMaxPhysicalMaterialToUEPbr();
		MaterialConverter = &PhysicalConverter;
	}
	else if (MaterialClassID == ARCHDESIGNMATCLASS)
	{
		static FDatasmithMaxMentalMaterialToUEPbr PhysicalConverter = FDatasmithMaxMentalMaterialToUEPbr();
		MaterialConverter = &PhysicalConverter;
	}
	else if (MaterialClassID == XREFMATCLASS)
	{
		MaterialConverter = GetMaterialConverter(FDatasmithMaxMatHelper::GetRenderedXRefMaterial(Material));
	}

	if ( MaterialConverter && MaterialConverter->IsSupported( Material ) )
	{
		ensure(Context);
		MaterialConverter->Context = Context;
		return MaterialConverter;
	}
	return nullptr;
}

void FDatasmithMaxMaterialsToUEPbrManager::AddDatasmithMaterial(TSharedRef<IDatasmithScene> DatasmithScene, Mtl* Material, TSharedPtr<IDatasmithBaseMaterialElement> DatasmithMaterial)
{
	if (!DatasmithMaterial)
	{
		return;
	}

	if (!Context)
	{
		return;
	}

	Context->MaterialsCollectionTracker.AddDatasmithMaterialForUsedMaterial(DatasmithScene, Material, DatasmithMaterial);

}

const TCHAR* FDatasmithMaxMaterialsToUEPbrManager::GetDatasmithMaterialName(Mtl* Material)
{
	if (Context)
	{
		return Context->MaterialsCollectionTracker.GetMaterialName(Material);
	}
	return Material->GetName().data();
}

FDatasmithMaxMaterialsToUEPbr::FDatasmithMaxMaterialsToUEPbr()
{
	TexmapConverters.Add( new FDatasmithMaxBitmapToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxAutodeskBitmapToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxNormalToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxRGBMultiplyToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxRGBTintToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxBakeableToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxMixToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxTextureOutputToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxFalloffToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxNoiseToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxCompositeToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxColorCorrectionToUEPbr() );

	// Pass-through converters
	TexmapConverters.Add( new FDatasmithMaxPassthroughToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxCellularToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxThirdPartyMultiTexmapToUEPbr() );
}

bool FDatasmithMaxMaterialsToUEPbr::IsTexmapSupported( Texmap* InTexmap ) const
{
	if ( !InTexmap )
	{
		return true;
	}

	Class_ID TexmapClassID = InTexmap->ClassID(); // TODO: Remove

	bool bIsTexmapSupported = false;

	for ( const IDatasmithMaxTexmapToUEPbr& TexmapConverter : TexmapConverters )
	{
		if ( TexmapConverter.IsSupported( this, InTexmap ) )
		{
			bIsTexmapSupported = true;
			break;
		}
	}

	if ( bIsTexmapSupported )
	{
		for ( int32 SubTexmapIndex = 0; SubTexmapIndex < InTexmap->NumSubTexmaps(); ++SubTexmapIndex )
		{
			if ( InTexmap->SubTexmapOn( SubTexmapIndex ) != 0 )
			{
				Texmap* SubTexmap = InTexmap->GetSubTexmap( SubTexmapIndex );

				if ( SubTexmap && !IsTexmapSupported( SubTexmap ) )
				{
					return false;
				}
			}
		}
	}

	return bIsTexmapSupported;
}

void FDatasmithMaxMaterialsToUEPbr::AddConvertedMap(const DatasmithMaxTexmapParser::FMapParameter& MapParameter)
{
	if (Context)
	{
		Context->TexmapsConverted.Add(MapParameter.Map);
	}
}

IDatasmithMaterialExpression* FDatasmithMaxMaterialsToUEPbr::ConvertTexmap( const DatasmithMaxTexmapParser::FMapParameter& MapParameter )
{
	if ( !MapParameter.bEnabled || !MapParameter.Map || FMath::IsNearlyZero( MapParameter.Weight ) )
	{
		return nullptr;
	}

	IDatasmithMaterialExpression* MaterialExpression = nullptr;

	for ( IDatasmithMaxTexmapToUEPbr& TexmapConverter : TexmapConverters )
	{
		if ( TexmapConverter.IsSupported( this, MapParameter.Map ) )
		{
			MaterialExpression = TexmapConverter.Convert( this, MapParameter.Map );
			break;
		}
	}

	if ( !MaterialExpression )
	{
		DatasmithMaxLogger::Get().AddUnsupportedMap(MapParameter.Map);
		return nullptr;
	}

	AddConvertedMap(MapParameter);

	// Check if the texmap has a texture ouput
	TextureOutput* TexOutput = nullptr;

	for ( int32 SubIndex = 0; SubIndex < MapParameter.Map->NumSubs(); ++SubIndex )
	{
		Animatable* SubAnim = MapParameter.Map->SubAnim( SubIndex );

		if ( SubAnim )
		{
			SClass_ID SubSuperClassID = SubAnim->SuperClassID();
		
			if ( SubSuperClassID == TEXOUTPUT_CLASS_ID )
			{
				TexOutput = static_cast< TextureOutput* >( SubAnim );
			}
		}
	}

	return FDatasmithMaxTexmapToUEPbrUtils::ConvertTextureOutput( this, MaterialExpression, TexOutput );
}
