// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DatasmithDefinitions.h"
#include "DatasmithMaxDirectLink.h"
#include "Containers/IndirectArray.h"
#include "Templates/SharedPointer.h"

class Class_ID;
class FDatasmithMaxMaterialsToUEPbr;
class IDatasmithBaseMaterialElement;
class IDatasmithMaterialExpression;
class IDatasmithMaterialExpressionTexture;
class IDatasmithMaxTexmapToUEPbr;
class IDatasmithScene;
class IDatasmithUEPbrMaterialElement;
class IDatasmithTextureElement;
class Mtl;
class Texmap;

namespace DatasmithMaxTexmapParser
{
	struct FMapParameter;
}

namespace DatasmithMaxDirectLink
{
	class FMaterialsCollectionTracker;

	struct FMaterialConversionContext
	{
		// Records texmaps that were asked for conversion while material conversion is performed
		TSet<Texmap*>& TexmapsConverted;
		FMaterialsCollectionTracker& MaterialsCollectionTracker;
	};

	class ITexmapToTextureElementConverter;
}


class FDatasmithMaxMaterialsToUEPbrManager
{
public:
	static FDatasmithMaxMaterialsToUEPbr* GetMaterialConverter( Mtl* Material );
	static void AddDatasmithMaterial(TSharedRef<IDatasmithScene> DatasmithScene, Mtl* Material, TSharedPtr<IDatasmithBaseMaterialElement> DatasmithMaterial);
	static const TCHAR* GetDatasmithMaterialName(Mtl* Material);
	static DatasmithMaxDirectLink::FMaterialConversionContext* Context;
};

class FDatasmithMaxMaterialsToUEPbr
{
public:
	FDatasmithMaxMaterialsToUEPbr();
	virtual ~FDatasmithMaxMaterialsToUEPbr() = default;

	virtual void Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath ) = 0;

	virtual bool IsSupported( Mtl* Material ) = 0; // Called by FDatasmithMaxMaterialsToUEPbrManager to see if instantiated converter can actually convert a material instance
	virtual bool IsTexmapSupported( Texmap* InTexmap ) const;

	/**
	 * Returns a LinearInterpolate expression if the map weight is smaller than 1.f, input 0 is not connected in that case
	 */
	IDatasmithMaterialExpression* ConvertTexmap( const DatasmithMaxTexmapParser::FMapParameter& MapParameter );

	// Texmap which needs to have a datasmith texture element created for
	void AddTexmap(Texmap* Texmap, const FString& DesiredTextureElementName, const TSharedPtr<DatasmithMaxDirectLink::ITexmapToTextureElementConverter>& Converter)
	{
		if (Context)
		{
			Context->MaterialsCollectionTracker.AddTexmapForConversion(Texmap, DesiredTextureElementName, Converter);
		}
	}

	struct FConvertState
	{
		TSharedPtr< IDatasmithScene > DatasmithScene;
		TSharedPtr< IDatasmithUEPbrMaterialElement > MaterialElement;
		FString AssetsPath;

		bool bCanBake = true;
		bool bIsMonoChannel = false; // true if we are parsing a mono channel (ie: opacity)
		bool bTreatNormalMapsAsLinear = false; // Corona has an option that treats all normal map inputs as linear

		EDatasmithTextureMode DefaultTextureMode = EDatasmithTextureMode::Diffuse;

	} ConvertState;

	void AddConvertedMap(const DatasmithMaxTexmapParser::FMapParameter& MapParameter);

	const TCHAR* GetMaterialName(Mtl* Material) const
	{
		return Context->MaterialsCollectionTracker.GetMaterialName(Material);
	}

	DatasmithMaxDirectLink::FMaterialConversionContext* Context = nullptr;

protected:
	TIndirectArray< IDatasmithMaxTexmapToUEPbr > TexmapConverters;

	//Structure used to have recursive import state.
	struct FScopedConvertState
	{
		FScopedConvertState(FConvertState& InCurrentConvertState) : CurrentConvertStateRef(InCurrentConvertState), PreviousConvertState(InCurrentConvertState)
		{
			CurrentConvertStateRef = FConvertState();
		}

		~FScopedConvertState()
		{
			CurrentConvertStateRef = PreviousConvertState;
		}
	private:
		FConvertState& CurrentConvertStateRef;
		const FConvertState PreviousConvertState;
	};
};

class IDatasmithMaxTexmapToUEPbr
{
public:
	virtual ~IDatasmithMaxTexmapToUEPbr() = default;

	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const = 0;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) = 0;
};


class IDatasmithMaterialExpression;
class IDatasmithMaterialExpressionScalar;
class IDatasmithMaterialExpressionColor;
class IDatasmithMaterialExpressionGeneric;
class IDatasmithExpressionInput;

class FDatasmithMaxMaterialsToUEPbrExpressions: public FDatasmithMaxMaterialsToUEPbr
{
public:
	TSharedPtr<IDatasmithUEPbrMaterialElement> GetMaterialElement();

	// Specify ParameterName to have constant exposed as material parameter
	IDatasmithMaterialExpression& Scalar(float Value, const TCHAR* ParameterName=nullptr);
	IDatasmithMaterialExpression& Color(const FLinearColor& Value, const TCHAR* ParameterName=nullptr);

	IDatasmithMaterialExpression* WeightTextureOrScalar(const DatasmithMaxTexmapParser::FMapParameter& TextureWeight, float Weight);

	IDatasmithMaterialExpression& Add(IDatasmithMaterialExpression& A, IDatasmithMaterialExpression& B);
	IDatasmithMaterialExpression& Add(IDatasmithMaterialExpression& A, float B);

	IDatasmithMaterialExpression& Subtract(IDatasmithMaterialExpression& A, IDatasmithMaterialExpression& B);
	IDatasmithMaterialExpression& Subtract(IDatasmithMaterialExpression& A, float B);

	IDatasmithMaterialExpression& Multiply(IDatasmithMaterialExpression& A, IDatasmithMaterialExpression& B);
	IDatasmithMaterialExpression& Multiply(IDatasmithMaterialExpression& A, float B);

	IDatasmithMaterialExpression& Divide(IDatasmithMaterialExpression& A, IDatasmithMaterialExpression& B);

	IDatasmithMaterialExpression& Desaturate(IDatasmithMaterialExpression& A);

	IDatasmithMaterialExpression& Power(IDatasmithMaterialExpression& A, IDatasmithMaterialExpression& B);
	IDatasmithMaterialExpression& Power(IDatasmithMaterialExpression& A, float Exp);

	IDatasmithMaterialExpression& Lerp(IDatasmithMaterialExpression& A, IDatasmithMaterialExpression& B, IDatasmithMaterialExpression& Alpha);

	IDatasmithMaterialExpression& Fresnel(IDatasmithMaterialExpression* Exponent=nullptr, IDatasmithMaterialExpression* BaseReflectFraction=nullptr); // Any input can be null

	IDatasmithMaterialExpression& PathTracingQualitySwitch(IDatasmithMaterialExpression& Normal, IDatasmithMaterialExpression& PathTraced);

	IDatasmithMaterialExpression* BlendTexmapWithColor(const DatasmithMaxTexmapParser::FMapParameter& Texmap, const FLinearColor& InColor, const TCHAR* ColorName = nullptr, const TCHAR* WeightName = nullptr);
	IDatasmithMaterialExpression* BlendTexmapWithScalar(const DatasmithMaxTexmapParser::FMapParameter& Texmap, float InScalar, const TCHAR* ScalarName = nullptr, const TCHAR* WeightName = nullptr);

	// Multiply RGB without multiplying A
	FLinearColor ScaleRGB(const FLinearColor& C, const float X) { return C * FLinearColor(X, X, X);}

	IDatasmithMaterialExpression* ApplyWeightExpression(IDatasmithMaterialExpression* ValueExpression, IDatasmithMaterialExpression* WeightExpression);

	IDatasmithMaterialExpression& CalcIORComplex(double IORn, double IORk, IDatasmithMaterialExpression& ToBeConnected90, IDatasmithMaterialExpression& ToBeConnected0);
	IDatasmithMaterialExpression& CalcIORComplex(IDatasmithMaterialExpression& IORn, IDatasmithMaterialExpression& IORk, IDatasmithMaterialExpression& ToBeConnected90, IDatasmithMaterialExpression& ToBeConnected0);

	void Connect(IDatasmithExpressionInput& Input, IDatasmithMaterialExpression& ValueExpression);
	bool Connect(IDatasmithExpressionInput& Input, IDatasmithMaterialExpression* ValueExpression); // Connect if not null

	IDatasmithMaterialExpression* TextureOrColor(const TCHAR* Name,
	                                             const DatasmithMaxTexmapParser::FMapParameter& Map, FLinearColor Color);

	IDatasmithMaterialExpression* TextureOrScalar(const TCHAR* Name,
	                                              const DatasmithMaxTexmapParser::FMapParameter& Map, float Value);

	IDatasmithMaterialExpression& OneMinus(IDatasmithMaterialExpression& Expression);
};

// Utility macros to simplify material expression composition:

// Create expression from expression parameters when they are all non-null or return default value
// E.g. instead of:
// IDatasmithMaterialExpression* SomeExpression = MakeExpressionCanReturnNull(...)
// IDatasmithMaterialExpression* ResultExpression = SomeExpression ? Desaturate(SomeExpression) : nullptr;
// Can do this:
// IDatasmithMaterialExpression* ResultExpression = COMPOSE_OR_NULL(Desaturate, MakeExpressionCanReturnNull(...));
// Which allows chaining expressions: COMPOSE_OR_DEFAULT2(&Scalar(1), Multiply, COMPOSE_OR_NULL(Desaturate, MakeExpressionCanReturnNull(...)), AnotherExpression)
// Implementation details: Pass params by value in order to prevent statement creating expression to be evaluated twice
// e.g. COMPOSE_OR_DEFAULT1(nullptr, Multiply, &Scalar(1.0), SomeExpression) won't create two expressions of each Scalar
// Func should be a method of this of free function - so it can be just substituted 
#define COMPOSE_OR_DEFAULT1(Default, Func, Param0) ([this](IDatasmithMaterialExpression* P0, IDatasmithMaterialExpression* D) {return P0 ? &Func(*P0) : D; } (Param0, Default))
#define COMPOSE_OR_DEFAULT2(Default, Func, Param0, Param1) ([this](IDatasmithMaterialExpression* P0, IDatasmithMaterialExpression* P1, IDatasmithMaterialExpression* D) { return (P0 && P1) ? &Func(*P0, *P1) : D;} (Param0, Param1, Default))
// Calls Func when param is not NULL, returns NULL otherwise
#define COMPOSE_OR_NULL(Func, Param0) ([this](IDatasmithMaterialExpression* P0) {return P0 ? &Func(*P0) : nullptr; } (Param0 ))
#define COMPOSE_OR_NULL2(Func, Param0, Param1) ([this](IDatasmithMaterialExpression* P0, IDatasmithMaterialExpression* P1) { return (P0 && P1) ? &Func(*P0, *P1) : nullptr;} (Param0, Param1))



