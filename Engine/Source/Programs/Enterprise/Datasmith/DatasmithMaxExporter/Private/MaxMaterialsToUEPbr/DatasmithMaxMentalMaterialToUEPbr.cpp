// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaxMaterialsToUEPbr/DatasmithMaxMentalMaterialToUEPbr.h"

#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithMaxWriter.h"
#include "DatasmithSceneFactory.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxTexmapToUEPbr.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxCoronaTexmapToUEPbr.h"

#include "DatasmithMaterialsUtils.h"


#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "max.h"
	#include "mtl.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

bool FDatasmithMaxMentalMaterialToUEPbr::IsSupported( Mtl* Material )
{
	return true;
}

struct FMaxMentalMaterial
{
	float DiffuseWeight = 0.f;
	DatasmithMaxTexmapParser::FMapParameter DiffuseColorMap;
	FLinearColor DiffuseColor;

	float Ior = 0;

	float ReflectionWeight = 0.f;
	DatasmithMaxTexmapParser::FMapParameter ReflectionColorMap;
	FLinearColor ReflectionColor;
	bool bReflectionUseFresnel = true;
	float bReflectionFuncLow = 0;
	float bReflectionFuncHigh = 1;
	float bReflectionFuncCurve = 1;
	bool bReflectionMetal = false;

	bool bThinWalled = false;
	float RefractionWeight = 0.f;
	DatasmithMaxTexmapParser::FMapParameter RefractionColorMap;
	FLinearColor RefractionColor;

	DatasmithMaxTexmapParser::FMapParameter CutoutMap;

	DatasmithMaxTexmapParser::FMapParameter GlossinessMap;
	float Glossiness = 0;

	bool bSelfIllumEnabled = false;
	bool bSelfIllumUseTemperature = true;
	FLinearColor SelfIllumColorFilter;
	float SelfIllumColorKelvin = 0.0f;
	DatasmithMaxTexmapParser::FMapParameter SelfIllumMap;

	float BumpAmount = 0;
	DatasmithMaxTexmapParser::FMapParameter BumpMap;

	void Parse(Mtl& Material)
	{
		int NumParamBlocks = Material.NumParamBlocks();

		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)j);
			// The the descriptor to 'decode'
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
			// Loop through all the defined parameters therein
			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM0")) == 0)
				{
					DiffuseColorMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM2")) == 0)
				{
					ReflectionColorMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM4")) == 0)
				{
					RefractionColorMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("cutout_map")) == 0)
				{
					CutoutMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("cutout_map_on")) == 0)
				{
					CutoutMap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("mapM3")) == 0)
				{
					GlossinessMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_color_mode")) == 0)
				{
					bSelfIllumUseTemperature = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_color_kelvin")) == 0)
				{
					SelfIllumColorKelvin = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_color_filter")) == 0)
				{
					SelfIllumColorFilter = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor((BMM_Color_fl)ParamBlock2->GetAColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_on")) == 0)
				{
					bSelfIllumEnabled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("self_illum_map")) == 0)
				{
					SelfIllumMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map")) == 0)
				{
					BumpMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map_amt")) == 0)
				{
					BumpAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("diff_color_map_on")) == 0)
				{
					DiffuseColorMap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_color_map_on")) == 0)
				{
					ReflectionColorMap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refr_color_map_on")) == 0)
				{
					RefractionColorMap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opts_1sided")) == 0)
				{
					bThinWalled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_gloss_map_on")) == 0)
				{
					GlossinessMap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map_on")) == 0)
				{
					BumpMap.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_func_fresnel")) == 0)
				{
					bReflectionUseFresnel = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_func_low")) == 0)
				{
					bReflectionFuncLow = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_func_high")) == 0)
				{
					bReflectionFuncHigh = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_func_curve")) == 0)
				{
					bReflectionFuncCurve = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_metal")) == 0)
				{
					bReflectionMetal = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("diff_weight")) == 0)
				{
					DiffuseWeight = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_weight")) == 0)
				{
					ReflectionWeight = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refr_weight")) == 0)
				{
					RefractionWeight = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_color")) == 0 )
				{
					ReflectionColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor((BMM_Color_fl)ParamBlock2->GetAColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refr_ior")) == 0) 
				{
					Ior = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("diff_color")) == 0)
				{
					DiffuseColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor((BMM_Color_fl)ParamBlock2->GetAColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_gloss")) == 0)
				{
					Glossiness = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}			
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refr_color")) == 0)
				{
					RefractionColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor((BMM_Color_fl)ParamBlock2->GetAColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
				}
			}
			ParamBlock2->ReleaseDesc();
		}

	}
};


void FDatasmithMaxMentalMaterialToUEPbr::Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath )
{
	if ( !Material )
	{
		return;
	}

	TSharedRef< IDatasmithUEPbrMaterialElement > PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial( GetMaterialName(Material) );
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	FMaxMentalMaterial MaterialProperties;
	MaterialProperties.Parse(*Material);

	// Diffuse
	IDatasmithMaterialExpression* DiffuseColorExpression = COMPOSE_OR_NULL2(Multiply, 
		TextureOrColor(TEXT("Diffuse Color"), MaterialProperties.DiffuseColorMap, MaterialProperties.DiffuseColor), 
		&Scalar(MaterialProperties.DiffuseWeight));

	// Emission
	if (MaterialProperties.bSelfIllumEnabled)
	{
		Connect(PbrMaterialElement->GetEmissiveColor(), MaterialProperties.bSelfIllumUseTemperature
			                                                ? &Color(DatasmithMaterialsUtils::TemperatureToColor(MaterialProperties.SelfIllumColorKelvin))
			                                                : TextureOrColor(
				                                                TEXT("Emissive Color"), MaterialProperties.SelfIllumMap,
				                                                MaterialProperties.SelfIllumColorFilter));
	}

	// Specular
	IDatasmithMaterialExpression* ReflectivityExpression = COMPOSE_OR_NULL2(Multiply,
		TextureOrColor(TEXT("Specular"), MaterialProperties.ReflectionColorMap, MaterialProperties.ReflectionColor),
		&Scalar(MaterialProperties.ReflectionWeight));
	Connect(PbrMaterialElement->GetSpecular(), COMPOSE_OR_NULL(Desaturate, ReflectivityExpression));

	if (MaterialProperties.bReflectionMetal)
	{
		Connect(PbrMaterialElement->GetMetallic(), COMPOSE_OR_NULL(Desaturate, ReflectivityExpression));		
	}

	// Transparency
	IDatasmithMaterialExpression* TransparencyExpression = nullptr;
	if (MaterialProperties.RefractionColorMap.IsMapPresentAndEnabled())
	{
		TransparencyExpression = COMPOSE_OR_NULL2(Multiply,
			TextureOrColor(TEXT("Opacity"), MaterialProperties.RefractionColorMap, MaterialProperties.RefractionColor), 
			&Scalar(MaterialProperties.RefractionWeight));
	}
	else
	{
		FLinearColor Transparency = MaterialProperties.RefractionColor * MaterialProperties.RefractionWeight;
	
		if (!Transparency.IsAlmostBlack()) // Don't create transparency when material is actually black(otherwise present transparency expression will make imported material Translucent instead of Opaque)
		{
			TransparencyExpression = &Color(Transparency);
		}
	}

	// Roughness
	if (IDatasmithMaterialExpression* RoughnessTextureExpression = ConvertTexmap(MaterialProperties.GlossinessMap))
	{
		Connect(PbrMaterialElement->GetRoughness(), COMPOSE_OR_NULL(OneMinus, COMPOSE_OR_NULL(Desaturate, RoughnessTextureExpression)));
	}
	else
	{
		// Use source glossiness when there's some reflectivity set in Max material(otherwise it's just a matte)
		float Roughness = (MaterialProperties.ReflectionColorMap.IsMapPresentAndEnabled() || !MaterialProperties.ReflectionColor.IsAlmostBlack())
			                  ? 1.0f - MaterialProperties.Glossiness // Just invert Glossiness to get Roughness
			                  : 0.75f;
		Connect(PbrMaterialElement->GetRoughness(), Scalar(Roughness));
	}

	// Opacity/refraction/masking
	IDatasmithMaterialExpression* CutoutExpression = ConvertTexmap(MaterialProperties.CutoutMap);
	IDatasmithMaterialExpression* BaseColorExpression = DiffuseColorExpression;
	if (TransparencyExpression)
	{
		IDatasmithMaterialExpression& TransparencyExpressionRef = *TransparencyExpression;
		IDatasmithMaterialExpression* BaseOpacityExpression = nullptr; // opacity before cutout mask applied
		
		// todo: PhysicalMaterial uses similar code to setup transparency/refraction only has additional param to force ThinTranslucent
		// ThinTranslucent, including when Ior is nearly 1
		bool ForceThinTranslucency = MaterialProperties.bThinWalled;
		if (ForceThinTranslucency || FMath::IsNearlyEqual(MaterialProperties.Ior, 1.0f))
		{
			PbrMaterialElement->SetShadingModel(EDatasmithShadingModel::ThinTranslucent);
			IDatasmithMaterialExpressionGeneric* ThinTranslucencyMaterialOutput = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			ThinTranslucencyMaterialOutput->SetExpressionName(TEXT("ThinTranslucentMaterialOutput"));

			Connect(*ThinTranslucencyMaterialOutput->GetInput(0), TransparencyExpression);

			BaseOpacityExpression = &OneMinus(Desaturate(TransparencyExpressionRef));
		}
		else
		{
			const bool bIsRefractiveGlass = MaterialProperties.bReflectionUseFresnel; // UseFresnel option is used for glass, water and other refractive dielectricsetc
			if (bIsRefractiveGlass)
			{
				// Ported FDatasmithMaterialExpressions::AddGlassNode to compute base color and opacity
				float Ior = MaterialProperties.Ior;

				IDatasmithMaterialExpression& Multiply09 = Multiply(Scalar(0.9f), TransparencyExpressionRef);

				BaseOpacityExpression = &OneMinus(Desaturate(CalcIORComplex(Ior, 0, Multiply(Scalar(0.5f), TransparencyExpressionRef), Multiply09)));

				Connect(PbrMaterialElement->GetRefraction(), Lerp(Scalar(1.0f), Scalar(MaterialProperties.Ior), Fresnel()));

				// todo: may add Diffuse color into the mix to accomodate non-clear opacity
				BaseColorExpression = &Multiply(Scalar(0.6f), Power(Multiply09, Scalar(5.0f))); 
			}
			else
			{
				BaseOpacityExpression = &OneMinus(Desaturate(TransparencyExpressionRef));
			}
		}
		Connect(PbrMaterialElement->GetOpacity(), COMPOSE_OR_DEFAULT2(BaseOpacityExpression, Multiply, BaseOpacityExpression, CutoutExpression));
	}
	else // NOT Transparent
	{
		if (CutoutExpression)
		{
			PbrMaterialElement->SetBlendMode(/*EBlendMode::BLEND_Masked*/1); // Specify blend mode when there's opacity mask
			Connect(PbrMaterialElement->GetOpacity(), CutoutExpression);
		}
	}

	Connect(PbrMaterialElement->GetBaseColor(), BaseColorExpression);

	// Bump
	// todo: similar code used in multiple places for UEPbr bump conversion
	{
		ConvertState.DefaultTextureMode = EDatasmithTextureMode::Bump; // Will change to normal if we pass through a normal map texmap
		ConvertState.bCanBake = false; // Current baking fails to produce proper normal maps

		MaterialProperties.BumpMap.Weight = MaterialProperties.BumpAmount;

		IDatasmithMaterialExpression* BumpExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, MaterialProperties.BumpMap, TEXT("Bump Map"), TOptional< FLinearColor >(), TOptional< float >() );

		if ( BumpExpression )
		{
			BumpExpression->ConnectExpression( PbrMaterialElement->GetNormal() );
			BumpExpression->SetName( TEXT("Bump Map") );
		}

		ConvertState.bCanBake = true;
	}

	MaterialElement = PbrMaterialElement;
}
