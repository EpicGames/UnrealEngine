// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaxMaterialsToUEPbr/DatasmithMaxPhysicalMaterialToUEPbr.h"

#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithMaxWriter.h"
#include "DatasmithSceneFactory.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxTexmapToUEPbr.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxCoronaTexmapToUEPbr.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "max.h"
	#include "mtl.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

TSharedPtr<IDatasmithUEPbrMaterialElement> FDatasmithMaxMaterialsToUEPbrExpressions::GetMaterialElement()
{
	return ConvertState.MaterialElement;
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Scalar(float Value, const TCHAR* ParameterName)
{
	IDatasmithMaterialExpressionScalar* ScalarExpression = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
	ScalarExpression->GetScalar() = Value;
	if (ParameterName)
	{
		ScalarExpression->SetName(ParameterName);
	}
	return *ScalarExpression;
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Color(const FLinearColor& Value, const TCHAR* ParameterName)
{
	IDatasmithMaterialExpressionColor* Result = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
	Result->GetColor() = Value;
	return *Result;
}

IDatasmithMaterialExpression* FDatasmithMaxMaterialsToUEPbrExpressions::WeightTextureOrScalar(
	const DatasmithMaxTexmapParser::FMapParameter& TextureWeight, float Weight)
{
	if (IDatasmithMaterialExpression* WeightMapExpression = ConvertTexmap(TextureWeight))
	{
		return WeightMapExpression;
	}

	if (!FMath::IsNearlyEqual(Weight, 1.0f))
	{
		return &Scalar(Weight);
	}
		
	return nullptr;
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Add(IDatasmithMaterialExpression& A,
	IDatasmithMaterialExpression& B)
{
	IDatasmithMaterialExpressionGeneric* Result = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
	Result->SetExpressionName(TEXT("Add"));

	A.ConnectExpression(*Result->GetInput(0));
	B.ConnectExpression(*Result->GetInput(1));
	return *Result;
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Add(IDatasmithMaterialExpression& A, float B)
{
	return Add(A, Scalar(B));
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Subtract(IDatasmithMaterialExpression& A,
	IDatasmithMaterialExpression& B)
{
	IDatasmithMaterialExpressionGeneric* Result = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
	Result->SetExpressionName(TEXT("Subtract"));

	A.ConnectExpression(*Result->GetInput(0));
	B.ConnectExpression(*Result->GetInput(1));
	return *Result;
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Subtract(IDatasmithMaterialExpression& A, float B)
{
	return Subtract(A, Scalar(B));
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Multiply(IDatasmithMaterialExpression& A, IDatasmithMaterialExpression& B)
{
	IDatasmithMaterialExpressionGeneric* Result = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
	Result->SetExpressionName(TEXT("Multiply"));

	A.ConnectExpression(*Result->GetInput(0));
	B.ConnectExpression(*Result->GetInput(1));
	return *Result;
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Multiply(IDatasmithMaterialExpression& A, float B)
{
	return Multiply(A, Scalar(B));
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Divide(IDatasmithMaterialExpression& A,
	IDatasmithMaterialExpression& B)
{
	IDatasmithMaterialExpressionGeneric* Result = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
	Result->SetExpressionName(TEXT("Divide"));

	A.ConnectExpression(*Result->GetInput(0));
	B.ConnectExpression(*Result->GetInput(1));
	return *Result;
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Desaturate(IDatasmithMaterialExpression& A)
{
	IDatasmithMaterialExpressionGeneric* Result = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
	Result->SetExpressionName(TEXT("Desaturation"));

	A.ConnectExpression(*Result->GetInput(0));
	return *Result;
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Power(IDatasmithMaterialExpression& A,
	IDatasmithMaterialExpression& B)
{
	IDatasmithMaterialExpressionGeneric* Result = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
	Result->SetExpressionName(TEXT("Power"));

	A.ConnectExpression(*Result->GetInput(0));
	B.ConnectExpression(*Result->GetInput(1));
	return *Result;
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Power(IDatasmithMaterialExpression& A, float Exp)
{
	TSharedRef<IDatasmithKeyValueProperty> ExpProperty = FDatasmithSceneFactory::CreateKeyValueProperty(TEXT("ConstExponent"));
	ExpProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Float);
	ExpProperty->SetValue(*LexToString(Exp));

	IDatasmithMaterialExpressionGeneric* Result = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
	Result->SetExpressionName(TEXT("Power"));

	A.ConnectExpression(*Result->GetInput(0));
	Result->AddProperty(ExpProperty);
	return *Result;
}


IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Lerp(IDatasmithMaterialExpression& A,
	IDatasmithMaterialExpression& B, IDatasmithMaterialExpression& Alpha)
{
	IDatasmithMaterialExpressionGeneric* Result = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
	Result->SetExpressionName(TEXT("LinearInterpolate"));

	A.ConnectExpression(*Result->GetInput(0));
	B.ConnectExpression(*Result->GetInput(1));
	Alpha.ConnectExpression(*Result->GetInput(2));
	return *Result;
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::Fresnel(IDatasmithMaterialExpression* Exponent,
	IDatasmithMaterialExpression* BaseReflectFraction)
{
	IDatasmithMaterialExpressionGeneric* Fresnel = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
	Fresnel->SetExpressionName(TEXT("Fresnel"));

	Connect(*Fresnel->GetInput(0), Exponent);
	Connect(*Fresnel->GetInput(1), BaseReflectFraction);
	return *Fresnel;
}


IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::PathTracingQualitySwitch(
	IDatasmithMaterialExpression& Normal, IDatasmithMaterialExpression& PathTraced)
{
	IDatasmithMaterialExpressionGeneric* Result = GetMaterialElement()->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
	Result->SetExpressionName(TEXT("PathTracingQualitySwitch"));

	Connect(*Result->GetInput(0), Normal);
	Connect(*Result->GetInput(1), PathTraced);
	return *Result;
}

IDatasmithMaterialExpression* FDatasmithMaxMaterialsToUEPbrExpressions::BlendTexmapWithColor(const DatasmithMaxTexmapParser::FMapParameter& Texmap, const FLinearColor& InColor, const TCHAR* ColorName, const TCHAR* WeightName)
{
	IDatasmithMaterialExpression* TexmapExpression = ConvertTexmap(Texmap);
	return TexmapExpression ? &Lerp(Color(InColor, ColorName), *TexmapExpression, Scalar(Texmap.Weight, WeightName)) : nullptr;
};

IDatasmithMaterialExpression* FDatasmithMaxMaterialsToUEPbrExpressions::BlendTexmapWithScalar(const DatasmithMaxTexmapParser::FMapParameter& Texmap, float InScalar, const TCHAR* ScalarName, const TCHAR* WeightName)
{
	IDatasmithMaterialExpression* TexmapExpression = ConvertTexmap(Texmap);
	return TexmapExpression ? &Lerp(Scalar(InScalar, ScalarName), *TexmapExpression, Scalar(Texmap.Weight, WeightName)) : nullptr;
};

IDatasmithMaterialExpression* FDatasmithMaxMaterialsToUEPbrExpressions::ApplyWeightExpression(
	IDatasmithMaterialExpression* ValueExpression, IDatasmithMaterialExpression* WeightExpression)
{
	return COMPOSE_OR_DEFAULT2(ValueExpression, Multiply, ValueExpression, WeightExpression);
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::CalcIORComplex(IDatasmithMaterialExpression& IORn, IDatasmithMaterialExpression& IORk, IDatasmithMaterialExpression& ToBeConnected90, IDatasmithMaterialExpression& ToBeConnected0)
{
	IDatasmithMaterialExpression& FresnelOrig = Fresnel(&Scalar(1.0f), &Scalar(0.0f));
	IDatasmithMaterialExpression& Fresnel = OneMinus(FresnelOrig);
	IDatasmithMaterialExpression& Fresnel2 = Multiply(Fresnel, Fresnel);

	// IORn->Desc = TEXT("IOR Value");
	IDatasmithMaterialExpression& ConstantIORn2 = Multiply(IORn, IORn);

	IDatasmithMaterialExpression& AddN2K2 = Add(ConstantIORn2, Multiply(IORk, IORk));

	IDatasmithMaterialExpression& Mul2NCPre = Multiply(IORn, Scalar(2.0f));
	IDatasmithMaterialExpression& Mul2NC = Multiply(Mul2NCPre, Fresnel);

	IDatasmithMaterialExpression& RsNumPre = Subtract(AddN2K2, Mul2NC);
	IDatasmithMaterialExpression& RsNum = Add(RsNumPre, Fresnel2);
	IDatasmithMaterialExpression& RsDenPre = Add(AddN2K2, Mul2NC);
	IDatasmithMaterialExpression& RsDen = Add(RsDenPre, Fresnel2);
	IDatasmithMaterialExpression& Rs = Divide(RsNum, RsDen);

	IDatasmithMaterialExpression& AddN2K2MulC2 = Multiply(AddN2K2, Fresnel2);

	IDatasmithMaterialExpression& RpNumPre = Subtract(AddN2K2MulC2, Mul2NC);
	IDatasmithMaterialExpression& RpNum = Add(RpNumPre, Scalar(1.0f));
	IDatasmithMaterialExpression& RpDenPre = Add(AddN2K2MulC2, Mul2NC);
	IDatasmithMaterialExpression& RpDen = Add(RpDenPre, Scalar(1.0f));
	IDatasmithMaterialExpression& Rp = Divide(RpNum, RpDen);

	IDatasmithMaterialExpression& Res = Multiply(Add(Rp, Rs), Scalar(0.5f));

	return Lerp(ToBeConnected0, ToBeConnected90, Power(Res, Scalar(0.5f)));
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::CalcIORComplex(double IORn, double IORk,
                                                                                       IDatasmithMaterialExpression& ToBeConnected90, IDatasmithMaterialExpression& ToBeConnected0)
{
	if (IORn >= 16)
	{
		return ToBeConnected90;
	}

	IORn = FMath::Max(IORn, 1.0f);
	IDatasmithMaterialExpression& ConstantIORn = Scalar(IORn);
	IDatasmithMaterialExpression& ConstantIORk = Scalar(IORk);

	return CalcIORComplex(ConstantIORn, ConstantIORk, ToBeConnected90, ToBeConnected0);
}

void FDatasmithMaxMaterialsToUEPbrExpressions::Connect(IDatasmithExpressionInput& Input, IDatasmithMaterialExpression& ValueExpression)
{
	ValueExpression.ConnectExpression(Input);
}

bool FDatasmithMaxMaterialsToUEPbrExpressions::Connect(IDatasmithExpressionInput& Input, IDatasmithMaterialExpression* ValueExpression)
{
	if (ValueExpression)
	{
		Connect(Input, *ValueExpression);
		return true;
	}
	return false;
}

IDatasmithMaterialExpression* FDatasmithMaxMaterialsToUEPbrExpressions::TextureOrColor(const TCHAR* Name,
	const DatasmithMaxTexmapParser::FMapParameter& Map, FLinearColor Color)
{
	return FDatasmithMaxTexmapToUEPbrUtils::MapOrValue(this, Map, Name, Color, TOptional<float>());
}

IDatasmithMaterialExpression* FDatasmithMaxMaterialsToUEPbrExpressions::TextureOrScalar(const TCHAR* Name,
	const DatasmithMaxTexmapParser::FMapParameter& Map, float Value)
{
	return FDatasmithMaxTexmapToUEPbrUtils::MapOrValue(this, Map, Name, TOptional<FLinearColor>(), Value);
}

IDatasmithMaterialExpression& FDatasmithMaxMaterialsToUEPbrExpressions::OneMinus(
	IDatasmithMaterialExpression& Expression)
{
	IDatasmithMaterialExpressionGeneric* Result = GetMaterialElement()->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
	Result->SetExpressionName(TEXT("OneMinus"));
	Connect(*Result->GetInput(0), &Expression);
	return *Result;
}

bool FDatasmithMaxPhysicalMaterialToUEPbr::IsSupported( Mtl* Material )
{
	return true;
}

struct FMaxPhysicalMaterial
{
	bool bTransparencyRoughessMapOn = true;
	bool bRoughnessInverted = true;
	bool bThinWalled = true;

	float DiffuseWeight = 0, Roughness = 0, Metalness = 0, Ior = 0;
	float Transparency = 0;
	float Reflectivity = 0;
	float EmittanceMultiplier = 0, EmittanceLuminance = 0, EmittanceTemperature = 0;
	float DisplacementMapAmount = 0;

	FLinearColor TransparencyColor;
	BMM_Color_fl DiffuseColor, EmittanceColor, ReflectionColor;

	DatasmithMaxTexmapParser::FMapParameter DiffuseWeightMap;
	DatasmithMaxTexmapParser::FMapParameter DiffuseColorMap;
	DatasmithMaxTexmapParser::FMapParameter TransparencyMap;
	DatasmithMaxTexmapParser::FMapParameter TransparencyColorMap;
	DatasmithMaxTexmapParser::FMapParameter MetalnessMap;
	DatasmithMaxTexmapParser::FMapParameter RoughnessMap;
	DatasmithMaxTexmapParser::FMapParameter BumpMap;
	DatasmithMaxTexmapParser::FMapParameter ReflectivityMap;
	DatasmithMaxTexmapParser::FMapParameter ReflectionColorMap;
	DatasmithMaxTexmapParser::FMapParameter EmittanceMap;
	DatasmithMaxTexmapParser::FMapParameter EmittanceColorMap;
	DatasmithMaxTexmapParser::FMapParameter CutoutMap;

	int MaterialMode = 0; //0 means simple, just metalness, 1 means specular + metalness


	void Parse(Mtl& Material)
	{
		int NumParamBlocks = Material.NumParamBlocks();


		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2 *ParamBlock2 = Material.GetParamBlockByID(j);
			// The the descriptor to 'decode'
			ParamBlockDesc2 *ParamBlockDesc = ParamBlock2->GetDesc();
			// Loop through all the defined parameters therein
			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];


				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("material_mode")) == 0)
				{
					MaterialMode = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}

				// booleans
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("base_weight_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						DiffuseWeightMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("base_color_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						DiffuseColorMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectivity_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						ReflectivityMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_color_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						ReflectionColorMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughness_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						RoughnessMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("metalness_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						MetalnessMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("transparency_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						TransparencyMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trans_color_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						TransparencyColorMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trans_rough_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						bTransparencyRoughessMapOn = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emission_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						EmittanceMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_color_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						EmittanceColorMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughness_inv")) == 0) // Set when 'Glossiness' selected in UI instead of Roughness
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						bRoughnessInverted = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("thin_walled")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						bThinWalled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						BumpMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("cutout_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						CutoutMap.bEnabled = false;
					}
				}

				// float values
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("base_weight")) == 0)
				{
					DiffuseWeight = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("roughness")) == 0)
				{
					Roughness = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("metalness")) == 0)
				{
					Metalness = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trans_ior")) == 0)
				{
					Ior = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("transparency")) == 0)
				{
					Transparency = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflectivity")) == 0)
				{
					Reflectivity = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emission")) == 0)
				{
					EmittanceMultiplier = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_luminance")) == 0)
				{
					EmittanceLuminance = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_kelvin")) == 0)
				{
					EmittanceTemperature = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map_amt")) == 0)
				{
					BumpMap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("displacement_map_amt")) == 0)
				{
					DisplacementMapAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}

				// plain colors
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("base_color")) == 0)
				{
					DiffuseColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trans_color")) == 0)
				{
					TransparencyColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor((BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime()));
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_color")) == 0)
				{
					EmittanceColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refl_color")) == 0)
				{
					ReflectionColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}

				// textures
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Base_Weight_Map")) == 0)
				{
					DiffuseWeightMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Base_Color_Map")) == 0)
				{
					DiffuseColorMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Reflectivity_Map")) == 0)
				{
					ReflectivityMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Roughness_Map")) == 0)
				{
					RoughnessMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Metalness_Map")) == 0)
				{
					MetalnessMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Refl_Color_Map")) == 0)
				{
					ReflectionColorMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Transparency_Map")) == 0)
				{
					TransparencyMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("trans_color_map")) == 0)
				{
					TransparencyColorMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Emission_Map")) == 0)
				{
					EmittanceMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("emit_color_map")) == 0)
				{
					EmittanceColorMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("bump_map")) == 0)
				{
					BumpMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("cutout_map")) == 0)
				{
					CutoutMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
			}

			ParamBlock2->ReleaseDesc();
		}
	}
};

struct FMaxPhysicalMaterialClearCoat
{
	bool bCoatRoughnessInverted = true;

	float CoatAmount = 0.f;
	float CoatRoughness = 0.f;
	float CoatIOR = 1.5f;

	DatasmithMaxTexmapParser::FMapParameter CoatWeightMap;
	DatasmithMaxTexmapParser::FMapParameter CoatColorMap;
	DatasmithMaxTexmapParser::FMapParameter CoatRoughnessMap;

	BMM_Color_fl CoatColor;

	bool HasClearCoat()
	{
		// There is no coating effect
		if (CoatAmount <= 0.0f || ((CoatColorMap.Map == nullptr || CoatColorMap.bEnabled == false) && CoatColor.r == 0.0f && CoatColor.g == 0.0f && CoatColor.b == 0))
		{
			return false;
		}
		return true;
	}

	void Parse(Mtl& Material){
		const int NumParamBlocks = Material.NumParamBlocks();

		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = Material.GetParamBlockByID(j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			// Loop through all the defined parameters therein
			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						CoatWeightMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_color_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						CoatColorMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_rough_map_on")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						CoatRoughnessMap.bEnabled = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_roughness_inv")) == 0)
				{
					if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
					{
						bCoatRoughnessInverted = false;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Coating_Weight_Map")) == 0)
				{
					CoatWeightMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Coating_Color_Map")) == 0)
				{
					CoatColorMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Coating_Roughness_Map")) == 0)
				{
					CoatRoughnessMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coating")) == 0)
				{
					CoatAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_roughness")) == 0)
				{
					CoatRoughness = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_ior")) == 0)
				{
					CoatIOR = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coat_color")) == 0)
				{
					CoatColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
			}

			ParamBlock2->ReleaseDesc();
		}
		
	}
};

void FDatasmithMaxPhysicalMaterialToUEPbr::Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath )
{
	if ( !Material )
	{
		return;
	}

	TSharedRef< IDatasmithUEPbrMaterialElement > PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial( GetMaterialName(Material) );
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	FMaxPhysicalMaterial PhysicalMaterialProperties;
	PhysicalMaterialProperties.Parse(*Material);

	IDatasmithMaterialExpression* DiffuseColorExpression = ApplyWeightExpression(
		TextureOrColor(TEXT("Diffuse Color"), PhysicalMaterialProperties.DiffuseColorMap, FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(PhysicalMaterialProperties.DiffuseColor)),
		WeightTextureOrScalar(PhysicalMaterialProperties.DiffuseWeightMap, PhysicalMaterialProperties.DiffuseWeight));

	IDatasmithMaterialExpression* TransparencyExpression = nullptr;
	if (PhysicalMaterialProperties.TransparencyColorMap.Map || PhysicalMaterialProperties.TransparencyMap.Map)
	{
		TransparencyExpression = ApplyWeightExpression(
					   TextureOrColor(TEXT("Opacity"), PhysicalMaterialProperties.TransparencyColorMap, PhysicalMaterialProperties.TransparencyColor),
					   WeightTextureOrScalar(PhysicalMaterialProperties.TransparencyMap, PhysicalMaterialProperties.Transparency));
	}
	else
	{
		FLinearColor Transparency = PhysicalMaterialProperties.TransparencyColor * PhysicalMaterialProperties.Transparency;

		if (!Transparency.IsAlmostBlack())
		{
			TransparencyExpression = &Color(Transparency);
		}
	}

	IDatasmithMaterialExpression* CutOutExpression = nullptr;
	if (PhysicalMaterialProperties.CutoutMap.IsMapPresentAndEnabled())
	{
		CutOutExpression = COMPOSE_OR_NULL(Desaturate, ConvertTexmap(PhysicalMaterialProperties.CutoutMap));
	}

	Connect(PbrMaterialElement->GetEmissiveColor(), 
		ApplyWeightExpression(
			TextureOrColor(TEXT("Emissive Color"), PhysicalMaterialProperties.EmittanceColorMap, FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(PhysicalMaterialProperties.EmittanceColor)),
			WeightTextureOrScalar(PhysicalMaterialProperties.EmittanceMap, PhysicalMaterialProperties.EmittanceMultiplier))
	);


	// todo: set opacity as mask and set blend mode
	// PbrMaterialElement->SetBlendMode(/*EBlendMode::BLEND_Masked*/1)
	// ExportPhysicalMaterialProperty(DatasmithScene, CutoutMap, bCutoutMapOn, NULL, NULL, BMM_Color_fl(0.0, 0.0, 0.0, 0.0), 1.0, MaterialShader->GetMaskComp(), DATASMITH_CLIPTEXNAME, DATASMITH_CLIPTEXNAME, false, true);

	Connect(PbrMaterialElement->GetMetallic(), TextureOrScalar(TEXT("Metallic"), PhysicalMaterialProperties.MetalnessMap, PhysicalMaterialProperties.Metalness));

	if (PhysicalMaterialProperties.MaterialMode == 1)
	{
		IDatasmithMaterialExpression* ReflectivityExpression = ApplyWeightExpression(
			TextureOrColor(TEXT("Specular"), PhysicalMaterialProperties.ReflectionColorMap, FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(PhysicalMaterialProperties.ReflectionColor)),
			WeightTextureOrScalar(PhysicalMaterialProperties.ReflectivityMap, PhysicalMaterialProperties.Reflectivity));

		Connect(PbrMaterialElement->GetSpecular(), COMPOSE_OR_NULL(Desaturate, ReflectivityExpression));
	}

	if (IDatasmithMaterialExpression* RoughnessTextureExpression = ConvertTexmap(PhysicalMaterialProperties.RoughnessMap))
	{
		Connect(PbrMaterialElement->GetRoughness(), PhysicalMaterialProperties.bRoughnessInverted ? COMPOSE_OR_NULL(OneMinus, RoughnessTextureExpression) : RoughnessTextureExpression);
	}
	else
	{
		float Roughness = 0.75f;
		if ((PhysicalMaterialProperties.MetalnessMap.Map && PhysicalMaterialProperties.MetalnessMap.bEnabled) 
			|| PhysicalMaterialProperties.Metalness > 0 
			|| (PhysicalMaterialProperties.MaterialMode == 1 && ((PhysicalMaterialProperties.ReflectionColorMap.Map && PhysicalMaterialProperties.ReflectionColorMap.bEnabled) || PhysicalMaterialProperties.Reflectivity > 0)))
		{
			Roughness = PhysicalMaterialProperties.bRoughnessInverted  ? 1.0f - PhysicalMaterialProperties.Roughness : PhysicalMaterialProperties.Roughness;
		}
		Connect(PbrMaterialElement->GetRoughness(), Scalar(Roughness));
	}

	if (PhysicalMaterialProperties.BumpMap.Map && PhysicalMaterialProperties.BumpMap.bEnabled== true)
	{
		ConvertState.DefaultTextureMode = EDatasmithTextureMode::Bump; // Will change to normal if we pass through a normal map texmap
		ConvertState.bCanBake = false; // Current baking fails to produce proper normal maps

		IDatasmithMaterialExpression* BumpExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue(this, PhysicalMaterialProperties.BumpMap, TEXT("Bump Map"), TOptional<FLinearColor>(), TOptional<float>());

		Connect(PbrMaterialElement->GetNormal(), BumpExpression);
		ConvertState.bCanBake = true;
	}

	IDatasmithMaterialExpression* BaseColorExpression = DiffuseColorExpression;
	// Build Material based on estimated material type
	if (TransparencyExpression)
	{
		IDatasmithMaterialExpression& OpacityFromTransparency = OneMinus(Desaturate(*TransparencyExpression));
		// ThinTranslucent, including when Ior is nearly 1
		if (PhysicalMaterialProperties.bThinWalled || FMath::IsNearlyEqual(PhysicalMaterialProperties.Ior, 1.0f))
		{
			PbrMaterialElement->SetShadingModel(EDatasmithShadingModel::ThinTranslucent);
			IDatasmithMaterialExpressionGeneric* ThinTranslucencyMaterialOutput = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			ThinTranslucencyMaterialOutput->SetExpressionName(TEXT("ThinTranslucentMaterialOutput"));

			TransparencyExpression->ConnectExpression(*ThinTranslucencyMaterialOutput->GetInput(0));

			Connect(PbrMaterialElement->GetOpacity(), COMPOSE_OR_DEFAULT2(&OpacityFromTransparency, Multiply, &OpacityFromTransparency, CutOutExpression));
		}
		else
		{
			if (PhysicalMaterialProperties.MaterialMode == 1) // Advanced mode, setup refractive glass
			{
				// Ported FDatasmithMaterialExpressions::AddGlassNode to compute base color and opacity
				float Ior = PhysicalMaterialProperties.Ior;

				IDatasmithMaterialExpression& Multiply09 = Multiply(Scalar(0.9f), *TransparencyExpression);

				// Modulate opacity by fresnel 
				IDatasmithMaterialExpression& OpacityWithFresnelExpression = OneMinus(Desaturate(CalcIORComplex(Ior, 0, Multiply(Scalar(0.5f), *TransparencyExpression), Multiply09)));

				Connect(PbrMaterialElement->GetOpacity(), COMPOSE_OR_DEFAULT2(&OpacityWithFresnelExpression, Multiply, &OpacityWithFresnelExpression, CutOutExpression));
				Connect(PbrMaterialElement->GetRefraction(), Lerp(Scalar(1.0f), Scalar(PhysicalMaterialProperties.Ior), Fresnel()));

				// todo: may add Diffuse color into the mix to accomodate non-clear opacity
				BaseColorExpression = &Multiply(Scalar(0.6f), Power(Multiply09, Scalar(5.0f))); 
			}
			else
			{
				Connect(PbrMaterialElement->GetOpacity(), COMPOSE_OR_DEFAULT2(&OpacityFromTransparency, Multiply, &OpacityFromTransparency, CutOutExpression));
			}
		}
	}
	else
	{
		Connect(PbrMaterialElement->GetOpacity(),  CutOutExpression);
	}

	Connect(PbrMaterialElement->GetBaseColor(), BaseColorExpression);

	FMaxPhysicalMaterialClearCoat ClearCoatProperties;
	ClearCoatProperties.Parse(*Material);

	if (ClearCoatProperties.HasClearCoat())
	{
		IDatasmithMaterialExpression* ClearCoatExpression = ApplyWeightExpression(
			TextureOrColor(TEXT("Clear Coat"), ClearCoatProperties.CoatColorMap, FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(ClearCoatProperties.CoatColor)),
			WeightTextureOrScalar(ClearCoatProperties.CoatWeightMap, ClearCoatProperties.CoatAmount) 
			);

		if (ClearCoatExpression)
		{
			// at 0 degrees we use the minumum and the actual input at 90 degrees
			IDatasmithMaterialExpression& ToBeConnected0 = Multiply(*ClearCoatExpression, Scalar(0.1f));

			Connect(PbrMaterialElement->GetClearCoat(), CalcIORComplex(ClearCoatProperties.CoatIOR, 0, *ClearCoatExpression, ToBeConnected0));
		}

		Connect(PbrMaterialElement->GetClearCoatRoughness(), TextureOrScalar(TEXT("Clear Coat Roughness"), ClearCoatProperties.CoatRoughnessMap, ClearCoatProperties.CoatRoughness));

		PbrMaterialElement->SetShadingModel(EDatasmithShadingModel::ClearCoat);
	}


	MaterialElement = PbrMaterialElement;
}
