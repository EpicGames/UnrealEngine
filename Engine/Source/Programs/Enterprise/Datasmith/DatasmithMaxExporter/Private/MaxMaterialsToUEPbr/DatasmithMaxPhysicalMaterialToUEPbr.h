// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithMaxMaterialsToUEPbr.h"

class FDatasmithMaxPhysicalMaterialToUEPbr : public FDatasmithMaxMaterialsToUEPbrExpressions
{
public:
	virtual bool IsSupported( Mtl* Material ) override;
	virtual void Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath ) override;
};

