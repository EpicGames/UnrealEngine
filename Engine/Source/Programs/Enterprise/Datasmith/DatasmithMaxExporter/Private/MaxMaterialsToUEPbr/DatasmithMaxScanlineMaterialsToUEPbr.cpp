// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaxMaterialsToUEPbr/DatasmithMaxScanlineMaterialsToUEPbr.h"

#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithMaxWriter.h"
#include "DatasmithSceneFactory.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxTexmapToUEPbr.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "max.h"
	#include "mtl.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

namespace DatasmithMaxScanlineMaterialsToUEPbrImpl
{
	enum class EScanlineMaterialMaps
	{
		Ambient,
		Diffuse,
		SpecularColor,
		SpecularLevel,
		Glossiness,
		SelfIllumination,
		Opacity,
		FilterColor,
		Bump,
		Reflection,
		Refraction,
	};

	struct FMaxScanlineMaterial
	{
		bool bIsTwoSided = false;

		// Diffuse
		FLinearColor DiffuseColor;
		DatasmithMaxTexmapParser::FMapParameter DiffuseMap;

		// Specular color
		FLinearColor SpecularColor;
		DatasmithMaxTexmapParser::FMapParameter SpecularColorMap;

		// Specular level
		float SpecularLevel;
		DatasmithMaxTexmapParser::FMapParameter SpecularLevelMap;

		// Glossiness
		float Glossiness = 0.f;
		DatasmithMaxTexmapParser::FMapParameter GlossinessMap;

		// Opacity
		float Opacity = 1.f;
		DatasmithMaxTexmapParser::FMapParameter OpacityMap;

		// Bump
		DatasmithMaxTexmapParser::FMapParameter BumpMap;

		// Self-illumination
		bool bUseSelfIllumColor = false;
		FLinearColor SelfIllumColor;
		DatasmithMaxTexmapParser::FMapParameter SelfIllumMap;
	};

	FMaxScanlineMaterial ParseScanlineMaterialProperties( Mtl& Material )
	{
		FMaxScanlineMaterial ScanlineMaterialProperties;

		const int NumParamBlocks = Material.NumParamBlocks();

		ScanlineMaterialProperties.DiffuseColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)Material.GetDiffuse() );
		ScanlineMaterialProperties.SpecularColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl) Material.GetSpecular() );
		ScanlineMaterialProperties.SpecularLevel = Material.GetShinStr();
		ScanlineMaterialProperties.Glossiness = Material.GetShininess();
		ScanlineMaterialProperties.bUseSelfIllumColor = ( Material.GetSelfIllumColorOn() != 0 );
		ScanlineMaterialProperties.SelfIllumColor = FDatasmithMaxMatHelper::MaxColorToFLinearColor( (BMM_Color_fl)Material.GetSelfIllumColor() ); // todo: this seems wrong(converting color to gamme space)

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				// Maps
				if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("maps") ) == 0 )
				{
					ScanlineMaterialProperties.DiffuseMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Diffuse );
					ScanlineMaterialProperties.SpecularColorMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SpecularColor );
					ScanlineMaterialProperties.SpecularLevelMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SpecularLevel );
					ScanlineMaterialProperties.GlossinessMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Glossiness );
					ScanlineMaterialProperties.OpacityMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Opacity );
					ScanlineMaterialProperties.BumpMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Bump );
					ScanlineMaterialProperties.SelfIllumMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SelfIllumination );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("mapEnables") ) == 0 )
				{
					ScanlineMaterialProperties.DiffuseMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Diffuse ) != 0 );
					ScanlineMaterialProperties.SpecularColorMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SpecularColor ) != 0 );
					ScanlineMaterialProperties.SpecularLevelMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SpecularLevel ) != 0 );
					ScanlineMaterialProperties.GlossinessMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Glossiness ) != 0 );
					ScanlineMaterialProperties.OpacityMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Opacity ) != 0 );
					ScanlineMaterialProperties.BumpMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Bump ) != 0 );
					ScanlineMaterialProperties.SelfIllumMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SelfIllumination ) != 0 );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("mapAmounts") ) == 0 )
				{
					ScanlineMaterialProperties.DiffuseMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Diffuse );
					ScanlineMaterialProperties.SpecularColorMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SpecularColor );
					ScanlineMaterialProperties.SpecularLevelMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SpecularLevel );
					ScanlineMaterialProperties.GlossinessMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Glossiness );
					ScanlineMaterialProperties.OpacityMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Opacity );
					ScanlineMaterialProperties.BumpMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::Bump );
					ScanlineMaterialProperties.SelfIllumMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime(), (int)EScanlineMaterialMaps::SelfIllumination );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("Opacity") ) == 0 )
				{
					ScanlineMaterialProperties.Opacity = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() );
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("twoSided") ) == 0 )
				{
					ScanlineMaterialProperties.bIsTwoSided = ( ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0 );
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		return ScanlineMaterialProperties;
	}
}

bool FDatasmithMaxScanlineMaterialsToUEPbr::IsSupported( Mtl* Material )
{
	return true;
}

void FDatasmithMaxScanlineMaterialsToUEPbr::Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath )
{
	if ( !Material )
	{
		return;
	}

	TSharedRef< IDatasmithUEPbrMaterialElement > PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial( GetMaterialName(Material) );
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	DatasmithMaxScanlineMaterialsToUEPbrImpl::FMaxScanlineMaterial ScanlineMaterialProperties = DatasmithMaxScanlineMaterialsToUEPbrImpl::ParseScanlineMaterialProperties( *Material );

	ConvertState.DefaultTextureMode = EDatasmithTextureMode::Diffuse;

	// Diffuse
	IDatasmithMaterialExpression* DiffuseExpression = nullptr;
	{
		DiffuseExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, ScanlineMaterialProperties.DiffuseMap, TEXT("Diffuse Color"),
			ScanlineMaterialProperties.DiffuseColor, TOptional< float >() );
	}

	// Glossiness
	IDatasmithMaterialExpression* GlossinessExpression = nullptr;
	{
		TGuardValue< bool > SetIsMonoChannel( ConvertState.bIsMonoChannel, true );

		ConvertState.DefaultTextureMode = EDatasmithTextureMode::Specular;

		GlossinessExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, ScanlineMaterialProperties.GlossinessMap, TEXT("Glossiness"), TOptional< FLinearColor >(), ScanlineMaterialProperties.Glossiness );

		if ( GlossinessExpression )
		{
			IDatasmithMaterialExpressionGeneric* OneMinusRougnessExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
			OneMinusRougnessExpression->SetExpressionName( TEXT("OneMinus") );

			GlossinessExpression->ConnectExpression( *OneMinusRougnessExpression->GetInput(0) );

			OneMinusRougnessExpression->ConnectExpression( PbrMaterialElement->GetRoughness() );
		}
	}

	// Specular
	ConvertState.DefaultTextureMode = EDatasmithTextureMode::Specular;

	IDatasmithMaterialExpression* SpecularColorExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, ScanlineMaterialProperties.SpecularColorMap, TEXT("Specular Color"), ScanlineMaterialProperties.SpecularColor, TOptional< float >() );
	IDatasmithMaterialExpression* SpecularExpression = SpecularColorExpression;

	if ( SpecularColorExpression )
	{
		SpecularColorExpression->SetName( TEXT("Specular") );

		IDatasmithMaterialExpressionScalar* SpecularLevelExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		SpecularLevelExpression->SetName( TEXT("Specular Level") );
		SpecularLevelExpression->GetScalar() = ScanlineMaterialProperties.SpecularLevel;

		IDatasmithMaterialExpressionGeneric* SpecularGlossinessExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		SpecularGlossinessExpression->SetExpressionName( TEXT("Multiply") );

		SpecularLevelExpression->ConnectExpression( *SpecularGlossinessExpression->GetInput(0), 0 );
		GlossinessExpression->ConnectExpression( *SpecularGlossinessExpression->GetInput(1), 0 );

		IDatasmithMaterialExpressionGeneric* WeightedSpecularExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		WeightedSpecularExpression->SetExpressionName( TEXT("Multiply") );

		SpecularColorExpression->ConnectExpression( *WeightedSpecularExpression->GetInput(0), 0 );
		SpecularGlossinessExpression->ConnectExpression( *WeightedSpecularExpression->GetInput(1), 0 );

		SpecularExpression = WeightedSpecularExpression;
	}

	// Opacity
	IDatasmithMaterialExpression* OpacityExpression = nullptr;
	{
		TGuardValue< bool > SetIsMonoChannel( ConvertState.bIsMonoChannel, true );

		TOptional< float > OpacityValue;
		if ( !FMath::IsNearlyEqual( ScanlineMaterialProperties.Opacity, 1.f ) )
		{
			OpacityValue = ScanlineMaterialProperties.Opacity;
		}

		OpacityExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, ScanlineMaterialProperties.OpacityMap, TEXT("Opacity"), TOptional< FLinearColor >(), OpacityValue );

		if ( OpacityExpression )
		{
			OpacityExpression->ConnectExpression( PbrMaterialElement->GetOpacity() );
		}
	}

	// Bump
	{
		ConvertState.DefaultTextureMode = EDatasmithTextureMode::Bump; // Will change to normal if we pass through a normal map texmap
		ConvertState.bCanBake = false; // Current baking fails to produce proper normal maps

		IDatasmithMaterialExpression* BumpExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, ScanlineMaterialProperties.BumpMap, TEXT("Bump Map"), TOptional< FLinearColor >(), TOptional< float >() );

		if ( BumpExpression )
		{
			BumpExpression->ConnectExpression( PbrMaterialElement->GetNormal() );
		}

		ConvertState.bCanBake = true;
	}

	// ConvertFromDiffSpec
	{
		IDatasmithMaterialExpressionFunctionCall* ConvertFromDiffSpecExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
		ConvertFromDiffSpecExpression->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions01/Shading/ConvertFromDiffSpec.ConvertFromDiffSpec") );

		DiffuseExpression->ConnectExpression( *ConvertFromDiffSpecExpression->GetInput(0), 0 );
		SpecularExpression->ConnectExpression( *ConvertFromDiffSpecExpression->GetInput(1), 0 );

		ConvertFromDiffSpecExpression->ConnectExpression( PbrMaterialElement->GetBaseColor(), 0 );
		ConvertFromDiffSpecExpression->ConnectExpression( PbrMaterialElement->GetMetallic(), 1 );
		ConvertFromDiffSpecExpression->ConnectExpression( PbrMaterialElement->GetSpecular(), 2 );
	}

	// Emissive
	{
		TOptional< FLinearColor > SelfIllumColor;

		if ( ScanlineMaterialProperties.bUseSelfIllumColor )
		{
			SelfIllumColor = ScanlineMaterialProperties.SelfIllumColor;
		}

		IDatasmithMaterialExpression* EmissiveExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, ScanlineMaterialProperties.SelfIllumMap, TEXT("Self illumination"), SelfIllumColor, TOptional< float >() );

		if ( EmissiveExpression )
		{
			EmissiveExpression->ConnectExpression( PbrMaterialElement->GetEmissiveColor() );
		}
	}

	PbrMaterialElement->SetTwoSided( ScanlineMaterialProperties.bIsTwoSided );

	MaterialElement = PbrMaterialElement;
}

bool FDatasmithMaxBlendMaterialsToUEPbr::IsSupported( Mtl* Material )
{
	if ( !Material )
	{
		return false;
	}

	bool bAllMaterialsSupported = true;

	if ( Mtl* BaseMaterial = Material->GetSubMtl(0) )
	{
		FDatasmithMaxMaterialsToUEPbr* MaterialConverter = FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter( BaseMaterial );
		bAllMaterialsSupported &= MaterialConverter != nullptr;
	}

	if ( Mtl* CoatMaterial = Material->GetSubMtl(1) )
	{
		FDatasmithMaxMaterialsToUEPbr* MaterialConverter = FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter( CoatMaterial );
		bAllMaterialsSupported &= MaterialConverter != nullptr;
	}

	return bAllMaterialsSupported;
}

void FDatasmithMaxBlendMaterialsToUEPbr::Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath )
{
	if ( !Material )
	{
		return;
	}

	Mtl* BaseMaterial = Material->GetSubMtl(0);
	Mtl* CoatMaterial = Material->GetSubMtl(1);
	DatasmithMaxTexmapParser::FMapParameter Mask;
	float MixAmount = 0.5f;

	for ( int ParamBlockIndex = 0; ParamBlockIndex < Material->NumParamBlocks(); ++ParamBlockIndex )
	{
		IParamBlock2* ParamBlock2 = Material->GetParamBlockByID( (short)ParamBlockIndex );
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for ( int ParamIndex = 0; ParamIndex < ParamBlockDesc->count; ++ParamIndex )
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[ParamIndex];

			if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("Mask")) == 0 )
			{
				Mask.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("bMaskEnabled")) == 0 )
			{
				Mask.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 );
			}
			else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("MixAmount")) == 0 )
			{
				MixAmount = (float)ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() );
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	TSharedRef< IDatasmithUEPbrMaterialElement > PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial(GetMaterialName(Material));
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	//Exporting the base material.
	IDatasmithMaterialExpressionFunctionCall* BaseMaterialFunctionCall = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
	if ( TSharedPtr<IDatasmithBaseMaterialElement> ExportedMaterial = FDatasmithMaxMatExport::ExportUniqueMaterial( DatasmithScene, BaseMaterial, AssetsPath ) )
	{
		BaseMaterialFunctionCall->SetFunctionPathName(ExportedMaterial->GetName());
	}

	IDatasmithMaterialExpression* PreviousExpression = BaseMaterialFunctionCall;

	//Exporting the coat material.
	if ( CoatMaterial != nullptr )
	{
		IDatasmithMaterialExpressionFunctionCall* BlendFunctionCall = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
		BlendFunctionCall->SetFunctionPathName( TEXT("/Engine/Functions/MaterialLayerFunctions/MatLayerBlend_Standard.MatLayerBlend_Standard") );
		PreviousExpression->ConnectExpression( *BlendFunctionCall->GetInput(0) );
		PreviousExpression = BlendFunctionCall;

		IDatasmithMaterialExpressionFunctionCall* LayerMaterialFunctionCall = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
		if (TSharedPtr<IDatasmithBaseMaterialElement> LayerMaterial = FDatasmithMaxMatExport::ExportUniqueMaterial( DatasmithScene, CoatMaterial, AssetsPath ))
		{
			LayerMaterialFunctionCall->SetFunctionPathName( LayerMaterial->GetName() );
		}
		LayerMaterialFunctionCall->ConnectExpression( *BlendFunctionCall->GetInput(1) );

		IDatasmithMaterialExpression* AlphaExpression = nullptr;

		IDatasmithMaterialExpression* MaskExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue(this, Mask, TEXT("MixAmount"),
			FLinearColor::White, TOptional< float >());
		AlphaExpression = MaskExpression;

		if ( !AlphaExpression )
		{
			IDatasmithMaterialExpressionScalar* MixAmountExpression = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
			MixAmountExpression->SetName( TEXT("Mix Amount") );
			MixAmountExpression->GetScalar() = MixAmount;

			AlphaExpression = MixAmountExpression;
		}

		//AlphaExpression is nullptr only when there is no mask and the mask weight is ~100% so we add scalar 0 instead.
		if ( !AlphaExpression )
		{
			IDatasmithMaterialExpressionScalar* WeightExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
			WeightExpression->GetScalar() = 0.f;
			AlphaExpression = WeightExpression;
		}

		AlphaExpression->ConnectExpression( *BlendFunctionCall->GetInput(2) );
	}

	PbrMaterialElement->SetUseMaterialAttributes( true );
	PreviousExpression->ConnectExpression( PbrMaterialElement->GetMaterialAttributes() );
	MaterialElement = PbrMaterialElement;
}

