// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaxMaterialsToUEPbr/DatasmithMaxTexmapToUEPbr.h"

#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxLogger.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithMaxWriter.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"
#include "DatasmithMaxHelper.h"

#include "Misc/Paths.h"
#include "DatasmithExportOptions.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "bitmap.h"
	#include "gamma.h"
	#include "stdmat.h"
	#include "pbbitmap.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"


namespace DatasmithMaxBitmapToUEPbrImpl
{
	struct FMaxRGBMultiplyParameters
	{
		DatasmithMaxTexmapParser::FMapParameter Map1;
		DatasmithMaxTexmapParser::FMapParameter Map2;

		FLinearColor Color1;
		FLinearColor Color2;
	};

	void ParseRGBMultiplyProperties( FMaxRGBMultiplyParameters& RGBMultiplyParameters, Texmap& InTexmap )
	{
		const int NumParamBlocks = InTexmap.NumParamBlocks();
		for ( int j = 0; j < NumParamBlocks; j++ )
		{
			IParamBlock2* ParamBlock2 = InTexmap.GetParamBlockByID( (short)j );
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp( ParamDefinition.int_name, TEXT("map1")) == 0 )
				{
					RGBMultiplyParameters.Map1.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime() );
				}
				else if (FCString::Stricmp( ParamDefinition.int_name, TEXT("map1Amount")) == 0 )
				{
					RGBMultiplyParameters.Map1.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) / 100.f;
				}
				else if (FCString::Stricmp( ParamDefinition.int_name, TEXT("map2")) == 0 )
				{
					RGBMultiplyParameters.Map2.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime() );
				}
				if (FCString::Stricmp( ParamDefinition.int_name, TEXT("map2Amount")) == 0 )
				{
					RGBMultiplyParameters.Map2.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) / 100.f;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map1enabled")) == 0)
				{
					RGBMultiplyParameters.Map1.bEnabled = (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("map2enabled")) == 0)
				{
					RGBMultiplyParameters.Map2.bEnabled = (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0);
				}
				else if (FCString::Stricmp( ParamDefinition.int_name, TEXT("Color1")) == 0 )
				{
					RGBMultiplyParameters.Color1 = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, GetCOREInterface()->GetTime() ) );
				}
				else if (FCString::Stricmp( ParamDefinition.int_name, TEXT("Color2")) == 0 )
				{
					RGBMultiplyParameters.Color2 = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, GetCOREInterface()->GetTime() ) );
				}
			}

			ParamBlock2->ReleaseDesc();
		}
	}

	struct FMaxRGBTintParameters
	{
		DatasmithMaxTexmapParser::FMapParameter Map1;

		DatasmithMaxTexmapParser::FWeightedColorParameter RColor;
		DatasmithMaxTexmapParser::FWeightedColorParameter GColor;
		DatasmithMaxTexmapParser::FWeightedColorParameter BColor;
	};

	FMaxRGBTintParameters ParseRGBTintProperties( Texmap& InTexmap )
	{
		FMaxRGBTintParameters RGBTintParameters;

		const int NumParamBlocks = InTexmap.NumParamBlocks();
		for ( int j = 0; j < NumParamBlocks; j++ )
		{
			IParamBlock2* ParamBlock2 = InTexmap.GetParamBlockByID( (short)j );
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp( ParamDefinition.int_name, TEXT("Red")) == 0 )
				{
					RGBTintParameters.RColor.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, GetCOREInterface()->GetTime() ) );
				}
				else if (FCString::Stricmp( ParamDefinition.int_name, TEXT("Green")) == 0 )
				{
					RGBTintParameters.GColor.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, GetCOREInterface()->GetTime() ) );
				}
				else if (FCString::Stricmp( ParamDefinition.int_name, TEXT("Blue")) == 0 )
				{
					RGBTintParameters.BColor.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, GetCOREInterface()->GetTime() ) );
				}
				else if (FCString::Stricmp( ParamDefinition.int_name, TEXT("Map1")) == 0 )
				{
					RGBTintParameters.Map1.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime() );
				}
				else if (FCString::Stricmp( ParamDefinition.int_name, TEXT("Map1Enabled")) == 0 )
				{
					RGBTintParameters.Map1.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 );
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		return RGBTintParameters;
	}

	struct FMaxMixParameters : public FMaxRGBMultiplyParameters
	{
		FMaxMixParameters()
			: MixAmount( 0.f )
		{
		}

		DatasmithMaxTexmapParser::FMapParameter MaskMap;

		float MixAmount;
	};

	FMaxMixParameters ParseMixProperties( Texmap& InTexmap )
	{
		FMaxMixParameters MixParameters;
		ParseRGBMultiplyProperties( MixParameters, InTexmap );

		const int NumParamBlocks = InTexmap.NumParamBlocks();
		for ( int j = 0; j < NumParamBlocks; j++ )
		{
			IParamBlock2* ParamBlock2 = InTexmap.GetParamBlockByID( (short)j );
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("MixAmount")) == 0)
				{
					MixParameters.MixAmount = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Mask")) == 0)
				{
					MixParameters.MaskMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("MaskEnabled")) == 0)
				{
					MixParameters.MaskMap.bEnabled = ( ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) != 0 );
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		return MixParameters;
	}

	struct FMaxFalloffParameters : public FMaxRGBMultiplyParameters
	{
		enum class EFalloffType
		{
			Perpendicular,
			Fresnel
		};

		FMaxFalloffParameters()
			: Type( EFalloffType::Fresnel )
		{
		}

		EFalloffType Type;
	};

	FMaxFalloffParameters ParseFalloffProperties( Texmap& InTexmap )
	{
		FMaxFalloffParameters FalloffParameters;
		ParseRGBMultiplyProperties( FalloffParameters, InTexmap );

		const int NumParamBlocks = InTexmap.NumParamBlocks();
		for ( int j = 0; j < NumParamBlocks; ++j )
		{
			IParamBlock2* ParamBlock2 = InTexmap.GetParamBlockByID( (short)j );
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for ( int i = 0; i < ParamBlockDesc->count; ++i )
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("Type") ) == 0 )
				{
					int32 TypeValue = ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() );

					if ( TypeValue == 1 )
					{
						FalloffParameters.Type = FMaxFalloffParameters::EFalloffType::Perpendicular;
					}
					else
					{
						FalloffParameters.Type = FMaxFalloffParameters::EFalloffType::Fresnel;
					}
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		return FalloffParameters;
	}

	struct FMaxNoiseParameters : public FMaxRGBMultiplyParameters
	{
		FMaxNoiseParameters()
			: Size( 1.f )
		{
		}

		float Size;
	};

	FMaxNoiseParameters ParseNoiseProperties( Texmap& InTexmap )
	{
		FMaxNoiseParameters NoiseParameters;
		ParseRGBMultiplyProperties( NoiseParameters, InTexmap );

		const int NumParamBlocks = InTexmap.NumParamBlocks();
		for ( int j = 0; j < NumParamBlocks; j++ )
		{
			IParamBlock2* ParamBlock2 = InTexmap.GetParamBlockByID( (short)j );
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Size")) == 0)
				{
					NoiseParameters.Size = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		return NoiseParameters;
	}

	FString GetBlendFunctionFromBlendMode( EDatasmithCompositeCompMode CompositeMode )
	{
		FString FunctionPathName;

		switch ( CompositeMode )
		{
		case EDatasmithCompositeCompMode::Burn:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_ColorBurn.Blend_ColorBurn");
			break;
		case EDatasmithCompositeCompMode::Dodge:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_ColorDodge.Blend_ColorDodge");
			break;
		case EDatasmithCompositeCompMode::Darken:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Darken.Blend_Darken");
			break;
		case EDatasmithCompositeCompMode::Difference:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Difference.Blend_Difference");
			break;
		case EDatasmithCompositeCompMode::Exclusion:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Exclusion.Blend_Exclusion");
			break;
		case EDatasmithCompositeCompMode::HardLight:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_HardLight.Blend_HardLight");
			break;
		case EDatasmithCompositeCompMode::Lighten:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Lighten.Blend_Lighten");
			break;
		case EDatasmithCompositeCompMode::Screen:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Screen.Blend_Screen");
			break;
		case EDatasmithCompositeCompMode::LinearBurn:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_LinearBurn.Blend_LinearBurn");
			break;
		case EDatasmithCompositeCompMode::LinearLight:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_LinearLight.Blend_LinearLight");
			break;
		case EDatasmithCompositeCompMode::LinearDodge:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_LinearDodge.Blend_LinearDodge");
			break;
		case EDatasmithCompositeCompMode::Overlay:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_Overlay.Blend_Overlay");
			break;
		case EDatasmithCompositeCompMode::PinLight:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_PinLight.Blend_PinLight");
			break;
		case EDatasmithCompositeCompMode::SoftLight:
			FunctionPathName = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_SoftLight.Blend_SoftLight");
			break;
		}

		return FunctionPathName;
	}
}

IDatasmithMaterialExpression* FDatasmithMaxTexmapToUEPbrUtils::ConvertTextureOutput( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, IDatasmithMaterialExpression* InputExpression, TextureOutput* InTextureOutput )
{
	if ( !InTextureOutput )
	{
		return InputExpression;
	}

	IDatasmithMaterialExpression* ResultExpression = InputExpression;

	// Output Level
	StdTexoutGen* StandardTextureOutput = nullptr;
	if ( InTextureOutput->IsSubClassOf( Class_ID( STDTEXOUT_CLASS_ID, 0 ) ) )
	{
		StandardTextureOutput = static_cast< StdTexoutGen* >( InTextureOutput );
	}

	const float OutputLevel = StandardTextureOutput
		? StandardTextureOutput->GetOutAmt( GetCOREInterface()->GetTime() ) * StandardTextureOutput->GetRGBAmt(GetCOREInterface()->GetTime())
		: InTextureOutput->GetOutputLevel( GetCOREInterface()->GetTime() );

	if ( !FMath::IsNearlyEqual( OutputLevel, 1.f ) )
	{
		IDatasmithMaterialExpressionGeneric* Multiply = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
		Multiply->SetExpressionName( TEXT("Multiply") );

		IDatasmithMaterialExpressionScalar* OutputLevelExpression = static_cast< IDatasmithMaterialExpressionScalar* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantScalar ) );
		OutputLevelExpression->SetName( TEXT("Output Level") );
		OutputLevelExpression->GetScalar() = OutputLevel;

		ResultExpression->ConnectExpression( *Multiply->GetInput(0) );
		OutputLevelExpression->ConnectExpression( *Multiply->GetInput(1) );

		ResultExpression = Multiply;
	}

	if ( StandardTextureOutput && StandardTextureOutput->IsStdTexoutGen() )
	{
		// RGB Offset
		const float RGBOffset = StandardTextureOutput->GetRGBOff( GetCOREInterface()->GetTime() );

		if ( !FMath::IsNearlyZero( RGBOffset ) )
		{
			IDatasmithMaterialExpressionGeneric* Add = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
			Add->SetExpressionName( TEXT("Add") );

			ResultExpression->ConnectExpression( *Add->GetInput(0) );

			IDatasmithMaterialExpressionScalar* RGBOffsetExpression = static_cast< IDatasmithMaterialExpressionScalar* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantScalar ) );
			RGBOffsetExpression->SetName( TEXT("RGB Offset") );
			RGBOffsetExpression->GetScalar() = RGBOffset;		

			RGBOffsetExpression->ConnectExpression( *Add->GetInput(1) );

			ResultExpression = Add;
		}

		// Clamp
		if ( StandardTextureOutput->GetClamp() != 0 )
		{
			IDatasmithMaterialExpressionGeneric* Clamp = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
			Clamp->SetExpressionName( TEXT("Clamp") );

			ResultExpression->ConnectExpression( *Clamp->GetInput(0) );

			ResultExpression = Clamp;
		}
	}


	// Invert
	if (InTextureOutput->GetInvert() != 0)
	{
		IDatasmithMaterialExpressionGeneric* OneMinus = static_cast<IDatasmithMaterialExpressionGeneric*>(MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression(EDatasmithMaterialExpressionType::Generic));
		OneMinus->SetExpressionName(TEXT("OneMinus"));

		ResultExpression->ConnectExpression( *OneMinus->GetInput(0) );

		ResultExpression = OneMinus;
	}

	return ResultExpression;
}

void FDatasmithMaxTexmapToUEPbrUtils::SetupTextureCoordinates( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, IDatasmithExpressionInput& UVCoordinatesInput, Texmap* InTexmap )
{
	StdUVGen* UV = nullptr;

	if ( InTexmap )
	{
		UVGen* BaseUV = InTexmap->GetTheUVGen();
		if ( BaseUV && BaseUV->IsStdUVGen() )
		{
			UV = static_cast< StdUVGen* >( BaseUV );
		}
	}

	if ( !UV )
	{
		return;
	}

	const TimeValue CurrentTime = GetCOREInterface()->GetTime();

	bool bIsUsingRealWorldScale = UV->GetUseRealWorldScale() != 0;
	float UScale = UV->GetUScl( CurrentTime );
	float VScale = UV->GetVScl( CurrentTime );
	bool bCroppedTexture = false;

	// Determine the sub-class of Texmap
	Class_ID ClassID = InTexmap->ClassID();

	bool bIsBitmapTex = ( ClassID.PartA() == BMTEX_CLASS_ID );
	if ( !bIsBitmapTex )
	{
		// Only procedural textures requires some processing of the UV scale values
		if ( !bIsUsingRealWorldScale )
		{
			// If RealWorldScale is not used, the UV values smaller than 1 are treated as 1
			UScale = FMath::Max(UScale, 1.0f);
			VScale = FMath::Max(VScale, 1.0f);

			// When a scale is greater than 1, the tiling is baked into the texture (does not apply when using RealWorldScale because the scales are inverted)
			if (UScale > 1.0f || VScale > 1.0f)
			{
				UScale = 1.0f;
				VScale = 1.0f;
			}
		}
		else
		{
			// Determine if texture has some fraction of the repeating pattern
			// If so, UV cropping will be needed to have seamless tiling
			bool bHasFractionalUV = false;
			if (UScale != VScale)
			{
				// Check how many times the smallest value fits within the greater value and look if there's a fractional part
				float Numerator = FMath::Max(UScale, VScale);
				float Denominator = FMath::Min(UScale, VScale);

				float Quotient = Numerator / Denominator;
				bHasFractionalUV = FMath::Frac(Quotient) > THRESH_UVS_ARE_SAME;
			}
			bCroppedTexture = bHasFractionalUV;
		}
	}

	float IntegerPart;
	float UOffset = (UV->GetUOffs(CurrentTime)) * UScale;
	if (!bIsUsingRealWorldScale)
	{
		UOffset += (-0.5f + 0.5f * UScale);
	}

	UOffset = 1.0f - UOffset;
	UOffset = FMath::Modf(UOffset, &IntegerPart);

	float VOffset = (UV->GetVOffs(CurrentTime)) * VScale;
	if (!bIsUsingRealWorldScale)
	{
		VOffset += (0.5f - 0.5f * VScale);
	}
	else
	{
		VOffset -= VScale;
	}

	VOffset = FMath::Modf(VOffset, &IntegerPart);

	int TextureTilingDirection = UV->GetTextureTiling();
	int MirrorU = 1;
	int MirrorV = 1;
	if (TextureTilingDirection & U_MIRROR)
	{
		MirrorU = 2;
	}
	if (TextureTilingDirection & V_MIRROR)
	{
		MirrorV = 2;
	}

	int Slot = 0;

	int CoordinateIndex = 0;
	if ( UV->GetSlotType() == MAPSLOT_TEXTURE )
	{
		if (UV->GetUVWSource() == UVWSRC_EXPLICIT)
		{
			CoordinateIndex = InTexmap->GetMapChannel() - 1;
		}
	}

	float WRotation = -UV->GetWAng( CurrentTime ) / ( 2.f * PI );

	IDatasmithMaterialExpressionTextureCoordinate* TextureCoordinateExpression = nullptr;

	if ( CoordinateIndex != 0 || !FMath::IsNearlyEqual( UScale, 1.f ) || !FMath::IsNearlyEqual( VScale, 1.f ) )
	{
		TextureCoordinateExpression = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionTextureCoordinate >();
		TextureCoordinateExpression->SetCoordinateIndex( CoordinateIndex );
		TextureCoordinateExpression->SetUTiling( UScale );
		TextureCoordinateExpression->SetVTiling( VScale );
	}

	IDatasmithMaterialExpressionFunctionCall* UVEditExpression = nullptr;

	if ( MirrorU > 1 || MirrorV > 1 || !FMath::IsNearlyZero( UOffset ) || !FMath::IsNearlyZero( VOffset ) || !FMath::IsNearlyZero( WRotation ) )
	{
		UVEditExpression = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
		UVEditExpression->SetFunctionPathName( TEXT("/DatasmithContent/Materials/UVEdit.UVEdit") );

		UVEditExpression->ConnectExpression( UVCoordinatesInput );

		// Mirror
		IDatasmithMaterialExpressionBool* MirrorUFlag = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionBool >();
		MirrorUFlag->SetName( TEXT("Mirror U") );
		MirrorUFlag->GetBool() = ( MirrorU > 1 );

		MirrorUFlag->ConnectExpression( *UVEditExpression->GetInput(3) );

		IDatasmithMaterialExpressionBool* MirrorVFlag = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionBool >();
		MirrorVFlag->SetName( TEXT("Mirror V") );
		MirrorVFlag->GetBool() = ( MirrorV > 1 );

		MirrorVFlag->ConnectExpression( *UVEditExpression->GetInput(4) );

		// Tiling
		IDatasmithMaterialExpressionColor* TilingValue = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
		TilingValue->SetName( TEXT("UV Tiling") );
		TilingValue->GetColor() = FLinearColor( MirrorU, MirrorV, 0.f );

		TilingValue->ConnectExpression( *UVEditExpression->GetInput(2) );

		IDatasmithMaterialExpressionColor* OffsetValue = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
		OffsetValue->SetName( TEXT("UV Offset") );
		OffsetValue->GetColor() = FLinearColor( UOffset, VOffset, 0.f );

		OffsetValue->ConnectExpression( *UVEditExpression->GetInput(7) );

		IDatasmithMaterialExpressionColor* TilingPivot = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
		TilingPivot->SetName( TEXT("Tiling Pivot") );
		TilingPivot->GetColor() = bIsUsingRealWorldScale && !MirrorUFlag->GetBool() ? FLinearColor( 0.5f, 0.5f, 0.f ) : FLinearColor( 0.f, 0.5f, 0.f );

		TilingPivot->ConnectExpression( *UVEditExpression->GetInput(1) );

		// Rotation
		if ( !FMath::IsNearlyZero( WRotation ) )
		{
			IDatasmithMaterialExpressionScalar* RotationValue = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
			RotationValue->SetName( TEXT("W Rotation") );
			RotationValue->GetScalar() = WRotation;

			RotationValue->ConnectExpression( *UVEditExpression->GetInput(6) );

			IDatasmithMaterialExpressionColor* RotationPivot = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
			RotationPivot->SetName( TEXT("Rotation Pivot") );
			RotationPivot->GetColor() = bIsUsingRealWorldScale ? FLinearColor( 0.5f, 0.5f, 0.f ) : FLinearColor( 0.f, 1.f, 0.f );

			RotationPivot->ConnectExpression( *UVEditExpression->GetInput(5) );
		}

		// A texture coordinate is mandatory for the UV Edit function
		if ( !TextureCoordinateExpression )
		{
			TextureCoordinateExpression = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionTextureCoordinate >();
			TextureCoordinateExpression->SetCoordinateIndex( 0 );
		}
	}

	if ( TextureCoordinateExpression )
	{
		if ( UVEditExpression )
		{
			TextureCoordinateExpression->ConnectExpression( *UVEditExpression->GetInput( 0 ) );
		}
		else
		{
			TextureCoordinateExpression->ConnectExpression( UVCoordinatesInput );
		}
	}

	//return FDatasmithTextureSampler(CoordinateIndex, UScale, VScale, UOffset, VOffset, -(UV->GetWAng(CurrentTime)) / (2 * PI), Multiplier, bForceInvert, Slot, bCroppedTexture, MirrorU, MirrorV);
}

IDatasmithMaterialExpression* FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, const DatasmithMaxTexmapParser::FMapParameter& MapParameter, const TCHAR* ParameterName,
	TOptional< FLinearColor > Color, TOptional< float > Scalar )
{
	IDatasmithMaterialExpression* Expression = MaxMaterialToUEPbr->ConvertTexmap( MapParameter );

	if ( !Expression || !FMath::IsNearlyEqual( MapParameter.Weight, 1.f ) )
	{
		if ( Expression )
		{
			TSharedRef< IDatasmithUEPbrMaterialElement > MaterialElement = MaxMaterialToUEPbr->ConvertState.MaterialElement.ToSharedRef();

			if ( MaxMaterialToUEPbr->ConvertState.DefaultTextureMode == EDatasmithTextureMode::Bump ||
				MaxMaterialToUEPbr->ConvertState.DefaultTextureMode == EDatasmithTextureMode::Normal )
			{
				// Scale only Red and Green by MapParameter.Weight. Normalization will happen so we can't scale all 3 parameters and the intensity of a normal is RG vs B.
				IDatasmithMaterialExpressionFunctionCall* RGBComponents = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
				RGBComponents->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakOutFloat3Components.BreakOutFloat3Components") );

				Expression->ConnectExpression( *RGBComponents->GetInput(0) );

				IDatasmithMaterialExpressionGeneric* MultiplyRed = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				MultiplyRed->SetExpressionName( TEXT("Multiply") );

				IDatasmithMaterialExpressionScalar* Intensity = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
				Intensity->SetName( TEXT("Normal Intensity") );
				Intensity->GetScalar() = MapParameter.Weight;

				RGBComponents->ConnectExpression( *MultiplyRed->GetInput(0), 0 ); // Red
				Intensity->ConnectExpression( *MultiplyRed->GetInput(1) );

				IDatasmithMaterialExpressionGeneric* MultiplyGreen = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				MultiplyGreen->SetExpressionName( TEXT("Multiply") );

				RGBComponents->ConnectExpression( *MultiplyGreen->GetInput(0), 1 ); // Green
				Intensity->ConnectExpression( *MultiplyGreen->GetInput(1) );

				// Blue is reconstructed from the other colors ( sqrt( 1-( saturate( dot([Red,Green], [Red,Green]) ) ) ) )
				IDatasmithMaterialExpressionGeneric* AppendRedAndGreen = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				AppendRedAndGreen->SetExpressionName( TEXT("AppendVector") );
				MultiplyRed->ConnectExpression( *AppendRedAndGreen->GetInput(0) );
				MultiplyGreen->ConnectExpression( *AppendRedAndGreen->GetInput(1) );

				IDatasmithMaterialExpressionGeneric* DotProduct = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				DotProduct->SetExpressionName( TEXT("DotProduct") );
				AppendRedAndGreen->ConnectExpression( *DotProduct->GetInput(0) );
				AppendRedAndGreen->ConnectExpression( *DotProduct->GetInput(1) );

				IDatasmithMaterialExpressionGeneric* OneMinus = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				OneMinus->SetExpressionName( TEXT("OneMinus") );
				DotProduct->ConnectExpression( *OneMinus->GetInput(0) );

				IDatasmithMaterialExpressionGeneric* Saturate = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				Saturate->SetExpressionName( TEXT("Saturate") );
				OneMinus->ConnectExpression( *Saturate->GetInput(0) );

				IDatasmithMaterialExpressionGeneric* SquareRoot = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				SquareRoot->SetExpressionName( TEXT("SquareRoot") );
				Saturate->ConnectExpression( *SquareRoot->GetInput(0) ); // Blue

				IDatasmithMaterialExpressionFunctionCall* MakeRGB = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
				MakeRGB->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/MakeFloat3.MakeFloat3") );

				MultiplyRed->ConnectExpression( *MakeRGB->GetInput(0) );
				MultiplyGreen->ConnectExpression( *MakeRGB->GetInput(1) );
				SquareRoot->ConnectExpression( *MakeRGB->GetInput(2) );

				Expression = MakeRGB;
			}
			else
			{
				IDatasmithMaterialExpression* ValueExpression = nullptr;
		
				if ( Color )
				{
					IDatasmithMaterialExpressionColor* ColorExpression = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
					ColorExpression->SetName( ParameterName );
					ColorExpression->GetColor() = Color.GetValue();

					ValueExpression = ColorExpression;
				}
				else if ( Scalar )
				{
					IDatasmithMaterialExpressionScalar* ScalarExpression = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
					ScalarExpression->SetName( ParameterName );
					ScalarExpression->GetScalar() = Scalar.GetValue();

					ValueExpression = ScalarExpression;
				}

				IDatasmithMaterialExpressionGeneric* MapWeightLerp = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				MapWeightLerp->SetExpressionName( TEXT("LinearInterpolate") );

				IDatasmithMaterialExpressionScalar* MapWeight = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
				MapWeight->SetName( *(FString(ParameterName) + TEXT("Map Weight")) );
				MapWeight->GetScalar() = MapParameter.Weight;

				if ( ValueExpression )
				{
					ValueExpression->ConnectExpression( *MapWeightLerp->GetInput(0) );
				}

				Expression->ConnectExpression( *MapWeightLerp->GetInput(1) );
				MapWeight->ConnectExpression( *MapWeightLerp->GetInput(2) );

				Expression = MapWeightLerp;
			}
		}
		else
		{
			IDatasmithMaterialExpression* ValueExpression = nullptr;
		
			if ( Color )
			{
				IDatasmithMaterialExpressionColor* ColorExpression = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionColor >();
				ColorExpression->SetName( ParameterName );
				ColorExpression->GetColor() = Color.GetValue();

				ValueExpression = ColorExpression;
			}
			else if ( Scalar )
			{
				IDatasmithMaterialExpressionScalar* ScalarExpression = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
				ScalarExpression->SetName( ParameterName );
				ScalarExpression->GetScalar() = Scalar.GetValue();

				ValueExpression = ScalarExpression;
			}

			Expression = ValueExpression;
		}
	}

	return Expression;
}

IDatasmithMaterialExpression* FDatasmithMaxTexmapToUEPbrUtils::ConvertBitMap(FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap, FString& ActualBitmapName, bool bUseAlphaAsMono, bool bIsSRGB)
{
	const TSharedRef< IDatasmithUEPbrMaterialElement > MaterialElement = MaxMaterialToUEPbr->ConvertState.MaterialElement.ToSharedRef();

	IDatasmithMaterialExpression* ResultExpression = nullptr;
	IDatasmithExpressionInput* UVCoordinatesInput = nullptr;

	EDatasmithTextureMode TextureMode = MaxMaterialToUEPbr->ConvertState.DefaultTextureMode;

	if (TextureMode == EDatasmithTextureMode::Bump)
	{
		IDatasmithMaterialExpressionGeneric* TextureObject = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		TextureObject->SetName(TEXT("Bump Map"));
		TextureObject->SetExpressionName(TEXT("TextureObjectParameter"));

		TSharedRef< IDatasmithKeyValueProperty > TextureProperty = FDatasmithSceneFactory::CreateKeyValueProperty(TEXT("Texture"));
		TextureProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Texture);
		TextureProperty->SetValue(*ActualBitmapName);

		TextureObject->AddProperty(MoveTemp(TextureProperty));

		IDatasmithMaterialExpressionFunctionCall* NormalFromHeightmap = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
		NormalFromHeightmap->SetFunctionPathName(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Procedurals/NormalFromHeightmap.NormalFromHeightmap"));

		IDatasmithMaterialExpressionScalar* UVOffset = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		UVOffset->GetScalar() = 0.001f;

		TextureObject->ConnectExpression(*NormalFromHeightmap->GetInput(0));
		UVOffset->ConnectExpression(*NormalFromHeightmap->GetInput(2));

		UVCoordinatesInput = NormalFromHeightmap->GetInput(3);
		ResultExpression = NormalFromHeightmap;

		TextureMode = EDatasmithTextureMode::Diffuse;
	}
	else
	{
		IDatasmithMaterialExpressionTexture* MaterialExpressionTexture = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionTexture >();
		MaterialExpressionTexture->SetTexturePathName(*ActualBitmapName);

		UVCoordinatesInput = &MaterialExpressionTexture->GetInputCoordinate();
		ResultExpression = MaterialExpressionTexture;

		if (MaxMaterialToUEPbr->ConvertState.bIsMonoChannel)
		{
			if (bUseAlphaAsMono)
			{
				constexpr int32 AlphaChannelIndex = 4;
				MaterialExpressionTexture->SetDefaultOutputIndex(AlphaChannelIndex);
			}
			else
			{
				IDatasmithMaterialExpressionGeneric* DesaturateRGB = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				DesaturateRGB->SetExpressionName(TEXT("Desaturation"));

				constexpr int32 RGBOutputIndex = 0;
				MaterialExpressionTexture->ConnectExpression(*DesaturateRGB->GetInput(0), RGBOutputIndex);

				ResultExpression = DesaturateRGB;
			}
		}
	}

	for (int32 TextureIndex = 0; TextureIndex < MaxMaterialToUEPbr->ConvertState.DatasmithScene->GetTexturesCount(); ++TextureIndex)
	{
		const TSharedPtr< IDatasmithTextureElement >& TextureElement = MaxMaterialToUEPbr->ConvertState.DatasmithScene->GetTexture(TextureIndex);

		if (ActualBitmapName == TextureElement->GetName())
		{
			if ( TextureElement->GetTextureMode() != TextureMode )
			{
				TextureElement->SetTextureMode(TextureMode);

				if ( bIsSRGB && TextureElement->GetRGBCurve() > 0.f &&
					MaxMaterialToUEPbr->ConvertState.bTreatNormalMapsAsLinear == false &&
					( TextureElement->GetTextureMode() == EDatasmithTextureMode::Normal || TextureElement->GetTextureMode() == EDatasmithTextureMode::NormalGreenInv ) )
				{
					TextureElement->SetRGBCurve(TextureElement->GetRGBCurve() * 2.2f); // In UE, normal maps are always imported as linear so adjust the RGB curve to compensate.
				}
			}

			break;
		}
	}

	if (UVCoordinatesInput)
	{
		FDatasmithMaxTexmapToUEPbrUtils::SetupTextureCoordinates(MaxMaterialToUEPbr, *UVCoordinatesInput, InTexmap);
	}

	return ResultExpression;
}

bool FDatasmithMaxBitmapToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap && InTexmap->ClassID() == RBITMAPCLASS )
	{
		return true;
	}

	return false;
}

class FBitmapToTextureElementConverter: public DatasmithMaxDirectLink::ITexmapToTextureElementConverter
{
public:
	virtual TSharedPtr<IDatasmithTextureElement> Convert(DatasmithMaxDirectLink::FMaterialsCollectionTracker& MaterialsTracker, const FString& TextureName) override
	{
		// Bitmap name is build using all bitmap settings used for the texmap - gamma and cropping
		// Don't convert same bitmap with same settings twice
		if (TSharedPtr<IDatasmithTextureElement>* Found =  MaterialsTracker.BitmapTextureElements.Find(ActualBitmapName))
		{
			return *Found;
		}

		FString Path = FDatasmithMaxMatWriter::GetActualBitmapPath(Tex);

		TSharedPtr< IDatasmithTextureElement > TextureElement = FDatasmithSceneFactory::CreateTexture(*TextureName);
		MaterialsTracker.BitmapTextureElements.Add(ActualBitmapName, TextureElement);

		if (gammaMgr.IsEnabled())
		{
			const float Gamma = FDatasmithMaxMatHelper::GetBitmapGamma(Tex);

			if (FDatasmithMaxMatHelper::IsSRGB(*Tex))
			{
				TextureElement->SetRGBCurve(Gamma / 2.2f);
			}
			else
			{
				TextureElement->SetRGBCurve(Gamma);
			}
		}

		// todo: ideally TextureMode should be set for Texture element
		// But Unreal Datasmith Importer currently has some legacy decisions that contradict with the plain logic of Texture Mode
		// e.g. setting to Bump makes heightmap to convert to normals on import. Which is not desired when that texture is used as heightmap in material graph
		//TextureElement->SetTextureMode(TextureMode);

		EDatasmithColorSpace ColorSpace = EDatasmithColorSpace::sRGB;
		switch (TextureMode)
		{
		case EDatasmithTextureMode::Normal: 
		case EDatasmithTextureMode::NormalGreenInv: 
		case EDatasmithTextureMode::Bump: 
		case EDatasmithTextureMode::Ies:
			ColorSpace = EDatasmithColorSpace::Linear;
			break;
		default: ;
		}
		TextureElement->SetSRGB(ColorSpace);
		
		if (!FPaths::FileExists(*Path))
		{
			DatasmithMaxDirectLink::LogWarning(FString::Printf(TEXT("Bitmap texture '%s' has missing external file '%s'"), Tex->GetName().data(), *Path));
		}

		TextureElement->SetFile(*Path);
		return TextureElement;
	}
	BitmapTex* Tex;

	bool bIsSRGB;
	FString ActualBitmapName;
	EDatasmithTextureMode TextureMode;
};

IDatasmithMaterialExpression* FDatasmithMaxBitmapToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	BitmapTex* InBitmapTex = (BitmapTex*)InTexmap;
	FDatasmithMaxMatWriter::CropBitmap( InBitmapTex ); // Crop if necessary
	FString ActualBitmapName = FDatasmithMaxMatWriter::GetActualBitmapName( InBitmapTex );
	bool bUseAlphaAsMono = InBitmapTex->GetAlphaAsMono(true);
	bool bIsSRGB = FDatasmithMaxMatHelper::IsSRGB(*InBitmapTex);

	TSharedRef<FBitmapToTextureElementConverter> Converter = MakeShared<FBitmapToTextureElementConverter>();
	Converter->Tex = InBitmapTex;
	Converter->bIsSRGB = bIsSRGB;
	Converter->ActualBitmapName = ActualBitmapName;
	Converter->TextureMode = MaxMaterialToUEPbr->ConvertState.DefaultTextureMode;

	MaxMaterialToUEPbr->AddTexmap(InTexmap, ActualBitmapName, Converter);

	return FDatasmithMaxTexmapToUEPbrUtils::ConvertBitMap(MaxMaterialToUEPbr, InTexmap, ActualBitmapName, bUseAlphaAsMono, bIsSRGB);
}

bool FDatasmithMaxAutodeskBitmapToUEPbr::IsSupported(const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap) const
{
	if (InTexmap)
	{
		MSTR ClassName;
		InTexmap->GetClassName(ClassName);
		//Somehow, there are multiple autodesk map classes using the same ClassID, we only support Autodesk Bitmap.
		if (InTexmap->ClassID() == AUTODESKBITMAPCLASS && FCString::Stricmp(ClassName, TEXT("Autodesk Bitmap")) == 0)
		{
			return true;
		}
	}

	return false;
}


class FAutodeskBitmapToTextureElementConverter: public DatasmithMaxDirectLink::ITexmapToTextureElementConverter
{
public:
	virtual TSharedPtr<IDatasmithTextureElement> Convert(DatasmithMaxDirectLink::FMaterialsCollectionTracker& MaterialsTracker, const FString& ActualBitmapName) override
	{
		if (PBBitmap* BitmapSourceFile = DatasmithMaxTexmapParser::ParseAutodeskBitmap(Tex).SourceFile)
		{
			FScopedBitMapPtr ActualBitmap(BitmapSourceFile->bi, BitmapSourceFile->bm);
			if (!ActualBitmap.Map)
			{
				return {};
			}

			TSharedRef<IDatasmithScene> DatasmithScene = MaterialsTracker.SceneTracker.GetDatasmithSceneRef();
			for (int i = 0; i < DatasmithScene->GetTexturesCount(); i++)
			{
				if (DatasmithScene->GetTexture(i)->GetFile() == Path && DatasmithScene->GetTexture(i)->GetName() == ActualBitmapName)
				{
					return {};
				}
			}

			TSharedPtr< IDatasmithTextureElement > TextureElement = FDatasmithSceneFactory::CreateTexture(*ActualBitmapName);
			if (gammaMgr.IsEnabled())
			{
				const float Gamma = FDatasmithMaxMatHelper::GetBitmapGamma(&ActualBitmap.MapInfo);

				if (FDatasmithMaxMatHelper::IsSRGB(*ActualBitmap.Map))
				{
					TextureElement->SetRGBCurve(Gamma / 2.2f);
				}
				else
				{
					TextureElement->SetRGBCurve(Gamma);
				}
			}

			TextureElement->SetFile(*Path);
			return TextureElement;
		}
		return {};
	}
	Texmap* Tex;

	bool bIsSRGB;
	FString Path;
};


IDatasmithMaterialExpression* FDatasmithMaxAutodeskBitmapToUEPbr::Convert(FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap)
{
	DatasmithMaxTexmapParser::FAutodeskBitmapParameters AutodeskBitmapParameters = DatasmithMaxTexmapParser::ParseAutodeskBitmap(InTexmap);
	if (AutodeskBitmapParameters.SourceFile)
	{
		FScopedBitMapPtr ActualBitmap(AutodeskBitmapParameters.SourceFile->bi, AutodeskBitmapParameters.SourceFile->bm);
		FString Path = FDatasmithMaxMatWriter::GetActualBitmapPath(&ActualBitmap.MapInfo);
		FString ActualBitmapName = FDatasmithMaxMatWriter::GetActualBitmapName(&ActualBitmap.MapInfo);
		bool bUseAlphaAsMono = (ActualBitmap.Map->HasAlpha() != 0);
		bool bIsSRGB = FDatasmithMaxMatHelper::IsSRGB(*ActualBitmap.Map);


		TSharedRef<FAutodeskBitmapToTextureElementConverter> Converter = MakeShared<FAutodeskBitmapToTextureElementConverter>();
		Converter->Tex = InTexmap;
		Converter->bIsSRGB = bIsSRGB;
		Converter->Path = Path;

		MaxMaterialToUEPbr->AddTexmap(InTexmap, ActualBitmapName, Converter);

		return FDatasmithMaxTexmapToUEPbrUtils::ConvertBitMap(MaxMaterialToUEPbr, InTexmap, ActualBitmapName, bUseAlphaAsMono, bIsSRGB);
	}
	return nullptr;
}

bool FDatasmithMaxNormalToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap )
	{
		if ( InTexmap->ClassID() == REGULARNORMALCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == VRAYNORMALCLASS )
		{
			return true;
		}
	}

	return false;
}

IDatasmithMaterialExpression* FDatasmithMaxNormalToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	DatasmithMaxTexmapParser::FNormalMapParameters NormalMapParameters = ParseMap( InTexmap );

	IDatasmithMaterialExpression* NormalExpression = nullptr;
	{
		TGuardValue< EDatasmithTextureMode > TextureModeValueGuard( MaxMaterialToUEPbr->ConvertState.DefaultTextureMode, EDatasmithTextureMode::Normal );

		NormalExpression = MaxMaterialToUEPbr->ConvertTexmap( NormalMapParameters.NormalMap );
	}
	
	IDatasmithMaterialExpression* BumpExpression = MaxMaterialToUEPbr->ConvertTexmap( NormalMapParameters.BumpMap );

	if ( !NormalExpression && !BumpExpression )
	{
		return nullptr;
	}

	TSharedRef< IDatasmithUEPbrMaterialElement > MaterialElement = MaxMaterialToUEPbr->ConvertState.MaterialElement.ToSharedRef();

	IDatasmithMaterialExpression* ResultExpression = NormalExpression;

	if ( NormalExpression && BumpExpression )
	{
		IDatasmithMaterialExpressionFunctionCall* BlendNormalsExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
		BlendNormalsExpression->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BlendAngleCorrectedNormals.BlendAngleCorrectedNormals") );

		NormalExpression->ConnectExpression( *BlendNormalsExpression->GetInput(0) );
		BumpExpression->ConnectExpression( *BlendNormalsExpression->GetInput(1) );

		ResultExpression = BlendNormalsExpression;
	}
	else if ( BumpExpression ) // Bump map only, no normal map
	{
		ResultExpression = BumpExpression;
	}

	IDatasmithMaterialExpression* RedExpression = nullptr;
	IDatasmithMaterialExpression* GreenExpression = nullptr;

	IDatasmithMaterialExpressionFunctionCall* BreakOutComponents = nullptr;

	constexpr int32 RedOutputIndex = 0;
	constexpr int32 GreenOutputIndex = 1;
	constexpr int32 BlueOutputIndex = 2;

	auto CreateBreakOutComponentExpression =
		[ &BreakOutComponents, ResultExpression, MaterialElement ]()
		{
			BreakOutComponents = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
			BreakOutComponents->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakOutFloat3Components.BreakOutFloat3Components") );

			ResultExpression->ConnectExpression( *BreakOutComponents->GetInput(0) );
		};

	if ( NormalMapParameters.bFlipRed )
	{
		IDatasmithMaterialExpressionGeneric* OneMinusRed = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		OneMinusRed->SetExpressionName( TEXT("OneMinus") );

		RedExpression = OneMinusRed;

		if ( !BreakOutComponents )
		{
			CreateBreakOutComponentExpression();
		}

		BreakOutComponents->ConnectExpression( *OneMinusRed->GetInput(0), RedOutputIndex );
	}
	
	if ( NormalMapParameters.bFlipGreen )
	{
		IDatasmithMaterialExpressionGeneric* OneMinusGreen = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		OneMinusGreen->SetExpressionName( TEXT("OneMinus") );

		GreenExpression = OneMinusGreen;

		if ( !BreakOutComponents )
		{
			CreateBreakOutComponentExpression();
		}

		BreakOutComponents->ConnectExpression( *OneMinusGreen->GetInput(0), GreenOutputIndex );
	}

	if ( !BreakOutComponents && NormalMapParameters.bSwapRedAndGreen )
	{
		CreateBreakOutComponentExpression();
	}

	if ( BreakOutComponents )
	{		
		IDatasmithMaterialExpressionFunctionCall* MakeFloat3 = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
		MakeFloat3->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/MakeFloat3.MakeFloat3") );

		IDatasmithExpressionInput* InputForRed = NormalMapParameters.bSwapRedAndGreen ? MakeFloat3->GetInput(1) : MakeFloat3->GetInput(0);

		if ( RedExpression )
		{
			RedExpression->ConnectExpression( *InputForRed );
		}
		else
		{
			BreakOutComponents->ConnectExpression( *InputForRed, RedOutputIndex );
		}

		IDatasmithExpressionInput* InputForGreen = NormalMapParameters.bSwapRedAndGreen ? MakeFloat3->GetInput(0) : MakeFloat3->GetInput(1);

		if ( GreenExpression )
		{
			GreenExpression->ConnectExpression( *InputForGreen );
		}
		else
		{
			BreakOutComponents->ConnectExpression( *InputForGreen, GreenOutputIndex );
		}

		BreakOutComponents->ConnectExpression( *MakeFloat3->GetInput(2), BlueOutputIndex );

		ResultExpression = MakeFloat3;
	}

	return ResultExpression;
}

DatasmithMaxTexmapParser::FNormalMapParameters FDatasmithMaxNormalToUEPbr::ParseMap( Texmap* InTexmap )
{
	return DatasmithMaxTexmapParser::ParseNormalMap( InTexmap );
}

bool FDatasmithMaxRGBMultiplyToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	return InTexmap ? (bool)( InTexmap->ClassID() == RGBMULTIPLYCLASS ) : false;
}

IDatasmithMaterialExpression* FDatasmithMaxRGBMultiplyToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	IDatasmithMaterialExpressionGeneric* MultiplyExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
	MultiplyExpression->SetExpressionName( TEXT("Multiply") );

	DatasmithMaxBitmapToUEPbrImpl::FMaxRGBMultiplyParameters RGBMultiplyParameters;
	DatasmithMaxBitmapToUEPbrImpl::ParseRGBMultiplyProperties( RGBMultiplyParameters, *InTexmap );

	IDatasmithMaterialExpression* Expression1 = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue(MaxMaterialToUEPbr, RGBMultiplyParameters.Map1, TEXT("RGBMultiply1"), RGBMultiplyParameters.Color1, TOptional<float>());
	if ( Expression1 )
	{
		Expression1->ConnectExpression( *MultiplyExpression->GetInput(0) );
	}

	IDatasmithMaterialExpression* Expression2 = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue(MaxMaterialToUEPbr, RGBMultiplyParameters.Map2, TEXT("RGBMultiply2"), RGBMultiplyParameters.Color2, TOptional<float>());
	if ( Expression2 )
	{
		Expression2->ConnectExpression( *MultiplyExpression->GetInput(1) );
	}

	return MultiplyExpression;
}

bool FDatasmithMaxRGBTintToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	return InTexmap ? (bool)( InTexmap->ClassID() == RGBTINTCLASS ) : false;
}

IDatasmithMaterialExpression* FDatasmithMaxRGBTintToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	DatasmithMaxBitmapToUEPbrImpl::FMaxRGBTintParameters RGBTintParameters = DatasmithMaxBitmapToUEPbrImpl::ParseRGBTintProperties( *InTexmap );

	// Red
	IDatasmithMaterialExpressionColor* RColorExpression = static_cast< IDatasmithMaterialExpressionColor* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantColor ) );
	RColorExpression->SetName( TEXT("Red Tint") );
	RColorExpression->GetColor() = RGBTintParameters.RColor.Value;

	IDatasmithMaterialExpressionGeneric* MultiplyR = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
	MultiplyR->SetExpressionName( TEXT("Multiply") );

	RColorExpression->ConnectExpression( *MultiplyR->GetInput(1) );

	// Green
	IDatasmithMaterialExpressionColor* GColorExpression = static_cast< IDatasmithMaterialExpressionColor* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantColor ) );
	GColorExpression->SetName( TEXT("Green Tint") );
	GColorExpression->GetColor() = RGBTintParameters.GColor.Value;

	IDatasmithMaterialExpressionGeneric* MultiplyG = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
	MultiplyG->SetExpressionName( TEXT("Multiply") );

	GColorExpression->ConnectExpression( *MultiplyG->GetInput(1) );

	// Blue
	IDatasmithMaterialExpressionColor* BColorExpression = static_cast< IDatasmithMaterialExpressionColor* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantColor ) );
	BColorExpression->SetName( TEXT("Blue Tint") );
	BColorExpression->GetColor() = RGBTintParameters.BColor.Value;

	IDatasmithMaterialExpressionGeneric* MultiplyB = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
	MultiplyB->SetExpressionName( TEXT("Multiply") );

	BColorExpression->ConnectExpression( *MultiplyB->GetInput(1) );

	// Add
	IDatasmithMaterialExpressionGeneric* AddRG = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
	AddRG->SetExpressionName( TEXT("Add") );

	MultiplyR->ConnectExpression( *AddRG->GetInput( 0 ) );
	MultiplyG->ConnectExpression( *AddRG->GetInput( 1 ) );

	IDatasmithMaterialExpressionGeneric* AddRGB = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
	AddRGB->SetExpressionName( TEXT("Add") );

	MultiplyB->ConnectExpression( *AddRGB->GetInput( 0 ) );
	AddRG->ConnectExpression( *AddRGB->GetInput( 1 ) );

	IDatasmithMaterialExpression* MapExpression = MaxMaterialToUEPbr->ConvertTexmap( RGBTintParameters.Map1 );

	if ( MapExpression )
	{
		constexpr int32 RedOutputIndex = 1;
		MapExpression->ConnectExpression( *MultiplyR->GetInput(0), RedOutputIndex );

		constexpr int32 GreenOutputIndex = 2;
		MapExpression->ConnectExpression( *MultiplyG->GetInput(0), GreenOutputIndex );

		constexpr int32 BlueOutputIndex = 3;
		MapExpression->ConnectExpression( *MultiplyB->GetInput(0), BlueOutputIndex );
	}

	return AddRGB;
}

bool FDatasmithMaxMixToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap && InTexmap->ClassID() == MIXCLASS )
	{
		return true;
	}

	return false;
}

IDatasmithMaterialExpression* FDatasmithMaxMixToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	DatasmithMaxBitmapToUEPbrImpl::FMaxMixParameters MixParameters = DatasmithMaxBitmapToUEPbrImpl::ParseMixProperties( *InTexmap );

	IDatasmithMaterialExpressionGeneric* LerpExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
	LerpExpression->SetExpressionName( TEXT("LinearInterpolate") );

	IDatasmithMaterialExpression* MixAmountExpression = MaxMaterialToUEPbr->ConvertTexmap( MixParameters.MaskMap );

	if ( !MixAmountExpression )
	{
		IDatasmithMaterialExpressionScalar* MixAmount = static_cast< IDatasmithMaterialExpressionScalar* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantScalar ) );
		MixAmount->SetName( TEXT("Mix Amount") );
		MixAmount->GetScalar() = MixParameters.MixAmount;

		MixAmountExpression = MixAmount;
	}

	if ( MixAmountExpression )
	{
		MixAmountExpression->ConnectExpression( *LerpExpression->GetInput(2) );
	}

	IDatasmithMaterialExpression* Map1Expression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( MaxMaterialToUEPbr, MixParameters.Map1, TEXT("Mix 1"), MixParameters.Color1, TOptional< float >() );

	if ( Map1Expression )
	{
		Map1Expression->ConnectExpression( *LerpExpression->GetInput(0) );
	}

	IDatasmithMaterialExpression* Map2Expression =  FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( MaxMaterialToUEPbr, MixParameters.Map2, TEXT("Mix 2"), MixParameters.Color2, TOptional< float >() );

	if ( Map2Expression )
	{
		Map2Expression->ConnectExpression( *LerpExpression->GetInput(1) );
	}

	return LerpExpression;
}

bool FDatasmithMaxFalloffToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap && InTexmap->ClassID() == FALLOFFCLASS )
	{
		return true;
	}

	return false;
}

IDatasmithMaterialExpression* FDatasmithMaxFalloffToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	IDatasmithUEPbrMaterialElement& MaterialElement = *MaxMaterialToUEPbr->ConvertState.MaterialElement.Get();

	DatasmithMaxBitmapToUEPbrImpl::FMaxFalloffParameters FalloffParameters = DatasmithMaxBitmapToUEPbrImpl::ParseFalloffProperties( *InTexmap );

	IDatasmithMaterialExpression* FalloffOuput = nullptr;
	IDatasmithMaterialExpression* FalloffRatio = nullptr;

	IDatasmithMaterialExpressionGeneric* VertexNormalWS = MaterialElement.AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
	VertexNormalWS->SetExpressionName( TEXT("VertexNormalWS") );

	//if ( FalloffParameters.Type == DatasmithMaxBitmapToUEPbrImpl::FMaxFalloffParameters::EFalloffType::Fresnel )
	{
		IDatasmithMaterialExpressionGeneric* Fresnel = MaterialElement.AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		Fresnel->SetExpressionName( TEXT("Fresnel") );

		VertexNormalWS->ConnectExpression( *Fresnel->GetInput(2) );

		FalloffRatio = Fresnel;
	}
	/*else // Perpendicular / Parallel
	{
		IDatasmithMaterialExpressionGeneric* DotProduct = MaterialElement.AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		DotProduct->SetExpressionName( TEXT("DotProduct") );

		IDatasmithMaterialExpressionGeneric* CameraVectorWS = MaterialElement.AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		CameraVectorWS->SetExpressionName( TEXT("CameraVectorWS") );

		IDatasmithMaterialExpressionGeneric* VertexNormalWS = MaterialElement.AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		VertexNormalWS->SetExpressionName( TEXT("VertexNormalWS") );

		CameraVectorWS->ConnectExpression( *DotProduct->GetInput(0) );
		VertexNormalWS->ConnectExpression( *DotProduct->GetInput(1) );

		FalloffRatio = DotProduct;
	}*/

	/*if ( !FDatasmithMaxMatHelper::HasNonBakeableSubmap( FalloffParameters.Map1.Map ) && !FDatasmithMaxMatHelper::HasNonBakeableSubmap( FalloffParameters.Map2.Map ) )
	{
		TSharedPtr< IDatasmithTextureElement > BakedTextureElement = FDatasmithMaxMatWriter::AddBakeable( MaxMaterialToUEPbr->ConvertState.DatasmithScene.ToSharedRef(),
			InTexmap, *MaxMaterialToUEPbr->ConvertState.AssetsPath );

		IDatasmithMaterialExpressionTexture* MaterialExpressionTexture = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionTexture >();
		MaterialExpressionTexture->SetTexturePathName( BakedTextureElement->GetName() );

		FalloffRatio->ConnectExpression( MaterialExpressionTexture->GetInputCoordinate() );

		FalloffOuput = MaterialExpressionTexture;
	}
	else*/
	{
		IDatasmithMaterialExpression* FrontExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( MaxMaterialToUEPbr, FalloffParameters.Map1, TEXT("Falloff Front"), FalloffParameters.Color1, TOptional< float >() );
		IDatasmithMaterialExpression* SideExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( MaxMaterialToUEPbr, FalloffParameters.Map2, TEXT("Falloff Side"), FalloffParameters.Color2, TOptional< float >() );

		IDatasmithMaterialExpressionGeneric* Lerp = MaterialElement.AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		Lerp->SetExpressionName( TEXT("LinearInterpolate") );

		FrontExpression->ConnectExpression( *Lerp->GetInput(0) );
		SideExpression->ConnectExpression( *Lerp->GetInput(1) );

		FalloffRatio->ConnectExpression( *Lerp->GetInput(2) );

		FalloffOuput = Lerp;
	}

	return FalloffOuput;
}

bool FDatasmithMaxNoiseToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap && InTexmap->ClassID() == NOISECLASS )
	{
		return true;
	}

	return false;
}

IDatasmithMaterialExpression* FDatasmithMaxNoiseToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	IDatasmithUEPbrMaterialElement& MaterialElement = *MaxMaterialToUEPbr->ConvertState.MaterialElement.Get();

	DatasmithMaxBitmapToUEPbrImpl::FMaxNoiseParameters NoiseParameters = DatasmithMaxBitmapToUEPbrImpl::ParseNoiseProperties( *InTexmap );

	IDatasmithMaterialExpression* Map1 = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( MaxMaterialToUEPbr, NoiseParameters.Map1, TEXT("Noise 1"), NoiseParameters.Color1, TOptional< float >() );
	IDatasmithMaterialExpression* Map2 = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( MaxMaterialToUEPbr, NoiseParameters.Map2, TEXT("Noise 2"), NoiseParameters.Color2, TOptional< float >() );

	IDatasmithMaterialExpressionGeneric* Lerp = static_cast< IDatasmithMaterialExpressionGeneric* >( MaterialElement.AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
	Lerp->SetExpressionName( TEXT("LinearInterpolate") );

	Map1->ConnectExpression( *Lerp->GetInput(0) );
	Map2->ConnectExpression( *Lerp->GetInput(1) );

	IDatasmithMaterialExpressionScalar* NoiseScale = static_cast< IDatasmithMaterialExpressionScalar* >( MaterialElement.AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantScalar ) );
	NoiseScale->SetName( TEXT("Noise Scale") );
	NoiseScale->GetScalar() = 0.1f / NoiseParameters.Size;

	IDatasmithMaterialExpressionGeneric* NoiseScaleMultiply = static_cast< IDatasmithMaterialExpressionGeneric* >( MaterialElement.AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
	NoiseScaleMultiply->SetExpressionName( TEXT("Multiply") );

	NoiseScale->ConnectExpression( *NoiseScaleMultiply->GetInput(1) );

	IDatasmithMaterialExpressionFunctionCall* LocalPosition = static_cast< IDatasmithMaterialExpressionFunctionCall* >( MaterialElement.AddMaterialExpression( EDatasmithMaterialExpressionType::FunctionCall ) );
	LocalPosition->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/WorldPositionOffset/LocalPosition") );

	LocalPosition->ConnectExpression( *NoiseScaleMultiply->GetInput(0) );

	IDatasmithMaterialExpressionGeneric* Noise = static_cast< IDatasmithMaterialExpressionGeneric* >( MaterialElement.AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
	Noise->SetExpressionName( TEXT("Noise") );

	NoiseScaleMultiply->ConnectExpression( *Noise->GetInput(0) );

	Noise->ConnectExpression( *Lerp->GetInput(2) );

	return Lerp;
}

bool FDatasmithMaxCompositeToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap && FDatasmithMaxMatHelper::HasNonBakeableSubmap( InTexmap ) && InTexmap->ClassID() == COMPOSITETEXCLASS ) // Only convert if it can't be baked
	{
		return true;
	}

	return false;
}

IDatasmithMaterialExpression* FDatasmithMaxCompositeToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	IDatasmithUEPbrMaterialElement* MaterialElement = MaxMaterialToUEPbr->ConvertState.MaterialElement.Get();

	if ( !MaterialElement )
	{
		return nullptr;
	}

	IDatasmithMaterialExpression* BaseLayerExpression = nullptr;

	DatasmithMaxTexmapParser::FCompositeTexmapParameters CompositeParameters = DatasmithMaxTexmapParser::ParseCompositeTexmap( InTexmap );

	uint32 LayerIndex = 0;

	// Layer 2 modifies the output of Layer 1, etc.
	for ( DatasmithMaxTexmapParser::FCompositeTexmapParameters::FLayer& Layer : CompositeParameters.Layers )
	{
		if ( Layer.Map.bEnabled && Layer.Map.Map != nullptr && ( !FMath::IsNearlyZero( Layer.Map.Weight ) || Layer.Mask.Map != nullptr ) )
		{
			IDatasmithMaterialExpression* CurrentLayerExpression = nullptr;

			if ( !BaseLayerExpression )
			{
				CurrentLayerExpression = MaxMaterialToUEPbr->ConvertTexmap( Layer.Map );

				if ( CurrentLayerExpression )
				{
					CurrentLayerExpression->SetName( *FString::Printf( TEXT("Composite Layer %u"), LayerIndex ) );
				}
				else
				{
					continue;
				}
			}
			else if ( Layer.CompositeMode == EDatasmithCompositeCompMode::Hue || Layer.CompositeMode == EDatasmithCompositeCompMode::Saturation ||
				Layer.CompositeMode == EDatasmithCompositeCompMode::Color || Layer.CompositeMode == EDatasmithCompositeCompMode::Value )
			{
				IDatasmithMaterialExpression* BlendLayerExpression = MaxMaterialToUEPbr->ConvertTexmap( Layer.Map );

				if ( !BlendLayerExpression )
				{
					continue;
				}

				// Current layer to HSV
				IDatasmithMaterialExpressionFunctionCall* RGBToHSVA = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
				RGBToHSVA->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/Math/RGBtoHSV.RGBtoHSV") );

				BlendLayerExpression->ConnectExpression( *RGBToHSVA->GetInput(0) );

				IDatasmithMaterialExpressionFunctionCall* BreakOutComponentsA = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
				BreakOutComponentsA->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakOutFloat3Components.BreakOutFloat3Components") );

				RGBToHSVA->ConnectExpression( *BreakOutComponentsA->GetInput(0) );

				// Previous layer to HSV
				IDatasmithMaterialExpressionFunctionCall* RGBToHSVB = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
				RGBToHSVB->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/Math/RGBtoHSV.RGBtoHSV") );

				BaseLayerExpression->ConnectExpression( *RGBToHSVB->GetInput(0) );

				IDatasmithMaterialExpressionFunctionCall* BreakOutComponentsB = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
				BreakOutComponentsB->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakOutFloat3Components.BreakOutFloat3Components") );

				RGBToHSVB->ConnectExpression( *BreakOutComponentsB->GetInput(0) );

				IDatasmithMaterialExpressionFunctionCall* MakeFloat3 = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
				MakeFloat3->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/MakeFloat3.MakeFloat3") );

				constexpr int32 RedOutputIndex = 0;
				constexpr int32 GreenOutputIndex = 1;
				constexpr int32 BlueOutputIndex = 2;

				if ( Layer.CompositeMode == EDatasmithCompositeCompMode::Hue )
				{
					BreakOutComponentsB->ConnectExpression( *MakeFloat3->GetInput(0), RedOutputIndex );
					BreakOutComponentsA->ConnectExpression( *MakeFloat3->GetInput(1), GreenOutputIndex );
					BreakOutComponentsA->ConnectExpression( *MakeFloat3->GetInput(2), BlueOutputIndex );
				}
				else if ( Layer.CompositeMode == EDatasmithCompositeCompMode::Saturation )
				{
					BreakOutComponentsA->ConnectExpression( *MakeFloat3->GetInput(0), RedOutputIndex );
					BreakOutComponentsB->ConnectExpression( *MakeFloat3->GetInput(1), GreenOutputIndex );
					BreakOutComponentsA->ConnectExpression( *MakeFloat3->GetInput(2), BlueOutputIndex );
				}
				else if ( Layer.CompositeMode == EDatasmithCompositeCompMode::Color )
				{
					BreakOutComponentsB->ConnectExpression( *MakeFloat3->GetInput(0), RedOutputIndex );
					BreakOutComponentsB->ConnectExpression( *MakeFloat3->GetInput(1), GreenOutputIndex );
					BreakOutComponentsA->ConnectExpression( *MakeFloat3->GetInput(2), BlueOutputIndex );
				}
				else
				{
					BreakOutComponentsA->ConnectExpression( *MakeFloat3->GetInput(0), RedOutputIndex );
					BreakOutComponentsA->ConnectExpression( *MakeFloat3->GetInput(1), GreenOutputIndex );
					BreakOutComponentsB->ConnectExpression( *MakeFloat3->GetInput(2), BlueOutputIndex );
				}

				IDatasmithMaterialExpressionFunctionCall* HSVToRGB = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
				HSVToRGB->SetFunctionPathName( TEXT("/DatasmithContent/Materials/HSVtoRGB.HSVtoRGB") );

				MakeFloat3->ConnectExpression( *HSVToRGB->GetInput(0) );

				CurrentLayerExpression = HSVToRGB;
			}
			else
			{
				IDatasmithMaterialExpression* BlendExpression = nullptr;

				switch ( Layer.CompositeMode )
				{
				case EDatasmithCompositeCompMode::Average:
					{
						IDatasmithMaterialExpressionGeneric* LerpExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
						LerpExpression->SetExpressionName( TEXT("LinearInterpolate") );
						BlendExpression = LerpExpression;
					}
					break;
				case EDatasmithCompositeCompMode::Add:
					{
						IDatasmithMaterialExpressionGeneric* AddExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
						AddExpression->SetExpressionName( TEXT("Add") );
						BlendExpression = AddExpression;
					}
					break;
				case EDatasmithCompositeCompMode::Sub:
					{
						IDatasmithMaterialExpressionGeneric* SubExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
						SubExpression->SetExpressionName( TEXT("Subtract") );
						BlendExpression = SubExpression;
					}
					break;
				case EDatasmithCompositeCompMode::Mult:
					{
						IDatasmithMaterialExpressionGeneric* MultiplyExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
						MultiplyExpression->SetExpressionName( TEXT("Multiply") );
						BlendExpression = MultiplyExpression;
					}
					break;
				default:
					{
						const FString BlendFunction = DatasmithMaxBitmapToUEPbrImpl::GetBlendFunctionFromBlendMode( Layer.CompositeMode );

						if ( !BlendFunction.IsEmpty() )
						{
							IDatasmithMaterialExpressionFunctionCall* BlendFunctionCall = static_cast< IDatasmithMaterialExpressionFunctionCall* >( MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::FunctionCall ) );
							BlendFunctionCall->SetFunctionPathName(*BlendFunction);

							BlendExpression = BlendFunctionCall;
						}
						else
						{
							if ( !BaseLayerExpression )
							{
								BlendExpression = nullptr; // Nothing to blend (Max normal mode)
							}
							else
							{
								IDatasmithMaterialExpressionGeneric* LerpExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
								LerpExpression->SetExpressionName( TEXT("LinearInterpolate") );
								BlendExpression = LerpExpression;
							}
						}
					}
					break;
				}

				IDatasmithMaterialExpression* BlendLayerExpression = MaxMaterialToUEPbr->ConvertTexmap( Layer.Map );

				if ( BlendLayerExpression )
				{
					BlendLayerExpression->SetName( *FString::Printf( TEXT("Composite Layer %u"), LayerIndex ) );

					if ( BlendExpression )
					{
						BaseLayerExpression->ConnectExpression( *BlendExpression->GetInput(0) );
						BlendLayerExpression->ConnectExpression( *BlendExpression->GetInput(1) );

						CurrentLayerExpression = BlendExpression; // The new base is the result of the blend
					}
					else
					{
						CurrentLayerExpression = BlendLayerExpression;
					}
				}
			}

			if ( CurrentLayerExpression )
			{
				if ( !FMath::IsNearlyEqual( Layer.Map.Weight, 1.f ) )
				{
					IDatasmithMaterialExpressionGeneric* LerpLayers = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
					LerpLayers->SetExpressionName( TEXT("LinearInterpolate") );

					if ( BaseLayerExpression )
					{
						BaseLayerExpression->ConnectExpression( *LerpLayers->GetInput(0) );
					}
					
					CurrentLayerExpression->ConnectExpression( *LerpLayers->GetInput(1) );

					IDatasmithMaterialExpressionScalar* LayerWeight = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
					LayerWeight->SetName( TEXT("Layer Weight") );
					LayerWeight->GetScalar() = Layer.Map.Weight;

					LayerWeight->ConnectExpression( *LerpLayers->GetInput(2) );

					CurrentLayerExpression = LerpLayers;
				}

				IDatasmithMaterialExpression* MaskExpression = MaxMaterialToUEPbr->ConvertTexmap( Layer.Mask );
				if ( MaskExpression )
				{
					IDatasmithMaterialExpressionGeneric* LerpLayers = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
					LerpLayers->SetExpressionName( TEXT("LinearInterpolate") );

					if ( BaseLayerExpression )
					{
						BaseLayerExpression->ConnectExpression( *LerpLayers->GetInput(0) );
					}

					CurrentLayerExpression->ConnectExpression( *LerpLayers->GetInput(1) );
					MaskExpression->ConnectExpression( *LerpLayers->GetInput(2) );

					CurrentLayerExpression = LerpLayers;
				}

				BaseLayerExpression = CurrentLayerExpression;
			}
		}

		++LayerIndex;
	}

	return BaseLayerExpression;
}

bool FDatasmithMaxTextureOutputToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap && InTexmap->ClassID() == OUTPUTMAPCLASS )
	{
		return true;
	}

	return false;
}

IDatasmithMaterialExpression* FDatasmithMaxTextureOutputToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	DatasmithMaxTexmapParser::FMapParameter MapParameter;
	MapParameter.Map = InTexmap->GetSubTexmap( 0 );
	MapParameter.bEnabled = ( InTexmap->SubTexmapOn( 0 ) != 0 );

	return MaxMaterialToUEPbr->ConvertTexmap( MapParameter );
}

bool FDatasmithMaxColorCorrectionToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	return ( InTexmap && InTexmap->ClassID() == COLORCORRECTCLASS &&
		( FDatasmithMaxMatHelper::HasNonBakeableSubmap( InTexmap ) || !MaxMaterialToUEPbr->ConvertState.bCanBake ) );
}

IDatasmithMaterialExpression* FDatasmithMaxColorCorrectionToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	TSharedPtr< IDatasmithUEPbrMaterialElement > MaterialElement = MaxMaterialToUEPbr->ConvertState.MaterialElement;

	DatasmithMaxTexmapParser::FColorCorrectionParameters ColorCorrectionParameters = DatasmithMaxTexmapParser::ParseColorCorrection(InTexmap);

	DatasmithMaxTexmapParser::FMapParameter MapParameter;
	MapParameter.Map = ColorCorrectionParameters.TextureSlot1;
	MapParameter.Weight = 1.f;
	MapParameter.bEnabled = ( InTexmap->SubTexmapOn(0) != 0 );

	IDatasmithMaterialExpression* ColorCorrectionExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( MaxMaterialToUEPbr, MapParameter, TEXT("Submap"), ColorCorrectionParameters.Color1, TOptional< float >() );

	if ( !ColorCorrectionExpression )
	{
		return nullptr;
	}

	// Hue shift
	if ( !FMath::IsNearlyZero( ColorCorrectionParameters.HueShift ) )
	{
		IDatasmithMaterialExpressionFunctionCall* HueShiftExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionFunctionCall >();
		HueShiftExpression->SetFunctionPathName( TEXT("/Engine/Functions/Engine_MaterialFunctions02/HueShift.HueShift") );

		IDatasmithMaterialExpressionScalar* HueShiftValue = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		HueShiftValue->SetName( TEXT("Hue Shift") );
		HueShiftValue->GetScalar() = ColorCorrectionParameters.HueShift;

		HueShiftValue->ConnectExpression( *HueShiftExpression->GetInput(0) );
		ColorCorrectionExpression->ConnectExpression( *HueShiftExpression->GetInput(1) );

		ColorCorrectionExpression = HueShiftExpression;
	}

	// Saturation
	if ( !FMath::IsNearlyZero( ColorCorrectionParameters.Saturation ) )
	{
		IDatasmithMaterialExpressionGeneric* DesaturationExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		DesaturationExpression->SetExpressionName( TEXT("Desaturation") );

		IDatasmithMaterialExpressionScalar* DesaturationValue = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		DesaturationValue->SetName( TEXT("Desaturation") );
		DesaturationValue->GetScalar() = -ColorCorrectionParameters.Saturation;

		ColorCorrectionExpression->ConnectExpression( *DesaturationExpression->GetInput(0) );
		DesaturationValue->ConnectExpression( *DesaturationExpression->GetInput(1) );

		ColorCorrectionExpression = DesaturationExpression;
	}

	// Gamma
	if ( ColorCorrectionParameters.bAdvancedLightnessMode && !FMath::IsNearlyZero( ColorCorrectionParameters.GammaRGB ) )
	{
		IDatasmithMaterialExpressionGeneric* GammaExpression = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		GammaExpression->SetExpressionName( TEXT("Power") );

		IDatasmithMaterialExpressionScalar* GammaValue = MaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		GammaValue->SetName( TEXT("Gamma") );
		GammaValue->GetScalar() = 1.f / ColorCorrectionParameters.GammaRGB;

		ColorCorrectionExpression->ConnectExpression( *GammaExpression->GetInput(0) );
		GammaValue->ConnectExpression( *GammaExpression->GetInput(1) );

		ColorCorrectionExpression = GammaExpression;
	}

	return ColorCorrectionExpression;
}

bool FDatasmithMaxBakeableToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap && !FDatasmithMaxMatHelper::HasNonBakeableSubmap( InTexmap ) &&
		MaxMaterialToUEPbr->ConvertState.bCanBake )
	{
		if ( InTexmap->ClassID() == GRADIENTCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == GRADIENTRAMPCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == TILESMAPCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == CHECKERCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == COLORCORRECTCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == COMPOSITETEXCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == CELLULARCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == FORESTCOLORCLASS)
		{
			return true;
		}
	}

	return false;
}

class FBakeableToTextureElementConverter: public DatasmithMaxDirectLink::ITexmapToTextureElementConverter
{
public:
	virtual TSharedPtr<IDatasmithTextureElement> Convert(DatasmithMaxDirectLink::FMaterialsCollectionTracker& MaterialsTracker, const FString& ActualBitmapName) override
	{
		if ( !Tex )
		{
			return {};
		}

		FString TexmapName = FString(Tex->GetName().data());
		FString Path = FPaths::Combine(MaterialsTracker.SceneTracker.GetAssetsOutputPath(), FileName + FDatasmithMaxMatWriter::TextureBakeFormat);
		
		int BakeWidth = FDatasmithExportOptions::MaxTextureSize;
		int BakeHeight = FDatasmithExportOptions::MaxTextureSize;
		GetBakeableMaximumSize(Tex, BakeWidth, BakeHeight);

		DatasmithMaxDirectLink::FSceneUpdateStats& Stats = MaterialsTracker.Stats;
		SCENE_UPDATE_STAT_INC(UpdateTextures, Baked);

		SCENE_UPDATE_STAT_SET(UpdateTextures, BakedPixels, SCENE_UPDATE_STAT_GET(UpdateTextures, BakedPixels) + BakeWidth*BakeHeight);

		DatasmithMaxDirectLink::LogInfo(FString::Printf(TEXT("Rendering Texmap '%s' to %dx%d image, path '%s'..."), *TexmapName, BakeWidth, BakeHeight, *Path));
		FDateTime TimeStart = FDateTime::UtcNow();

		BitmapInfo BitmapInformation;
		BitmapInformation.SetType(BMM_TRUE_32);

		BitmapInformation.SetWidth(BakeWidth);
		BitmapInformation.SetHeight(BakeHeight);
		BitmapInformation.SetGamma(2.2f);

		BitmapInformation.SetName(*Path);

		Bitmap* NewBitmap = TheManager->Create(&BitmapInformation);
		Tex->RenderBitmap(GetCOREInterface()->GetTime(), NewBitmap, 1.0f, 1);
		
		NewBitmap->OpenOutput(&BitmapInformation);
		NewBitmap->Write(&BitmapInformation);
		NewBitmap->Close(&BitmapInformation);
		NewBitmap->DeleteThis();

		TSharedPtr< IDatasmithTextureElement > TextureElement = FDatasmithSceneFactory::CreateTexture(*ActualBitmapName);
		TextureElement->SetRGBCurve(1.0f);
		TextureElement->SetFile(*Path);

		FDateTime TimeFinish = FDateTime::UtcNow();
		
		DatasmithMaxDirectLink::LogInfo(FString::Printf(TEXT("done in %s"), *(TimeFinish-TimeStart).ToString()));

		return TextureElement;
	}

	static void GetBakeableMaximumSize(Texmap* InTexmap, int &Width, int &Height)
	{
		TArray<Texmap*> Texmaps;
		Texmaps.Add(InTexmap);

		// Limit baked texture size by subbitmap with largest area(number of pixels)
		int32 BiggestSubBitmapArea = 0;  // Biggest pixel count
		int32 BiggestSubBitmapWidth = 0;
		int32 BiggestSubBitmapHeight = 0;

		while (!Texmaps.IsEmpty())
		{
			Texmap* Texmap = Texmaps.Pop(EAllowShrinking::No);

			if (!Texmap)
			{
				continue;;
			}

			if (Texmap->ClassID() == RBITMAPCLASS)
			{
				BitmapTex* BitmapTexture = (BitmapTex*)Texmap;
				Bitmap* ActualBitmap = ((BitmapTex*)Texmap)->GetBitmap(GetCOREInterface()->GetTime());
				if (ActualBitmap != NULL)
				{
					int32 BitmapWidth = ActualBitmap->Width();
					int32 BitmapHeight = ActualBitmap->Height();

					int32 Area = BitmapWidth*BitmapHeight;
					if (Area > BiggestSubBitmapArea)
					{
						BiggestSubBitmapArea = Area;
						BiggestSubBitmapWidth = BitmapWidth;
						BiggestSubBitmapHeight = BitmapHeight;
					}

				}
			}

			for (int SubTexmap = 0; SubTexmap < Texmap->NumSubTexmaps(); SubTexmap++)
			{
				Texmaps.Add(Texmap->GetSubTexmap(SubTexmap));
			}
		}

		if (BiggestSubBitmapArea > 0)
		{
			int32 AreaLimit = Width*Height;

			Width = BiggestSubBitmapWidth;
			Height = BiggestSubBitmapHeight;
			int32 Area = BiggestSubBitmapArea;
			while (Area > AreaLimit)
			{
				Width = Width / 2;
				Height = Height / 2;
				Area = Width * Height;
			}
		}
	}

	Texmap* Tex;
	FString FileName;
};

IDatasmithMaterialExpression* FDatasmithMaxBakeableToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	if ( !InTexmap )
	{
		return nullptr;
	}

	MSTR ClassName;
	InTexmap->GetClassName(ClassName);
	FString TexmapName = FString(InTexmap->GetName().data());
	FString FileName = TexmapName + FString(ClassName.data()) + FString::FromInt(InTexmap->GetHandleByAnim(InTexmap));
	FileName = FDatasmithUtils::SanitizeFileName(FileName);

	FString Name = FDatasmithUtils::SanitizeObjectName(FileName + FDatasmithMaxMatWriter::TextureSuffix);

	TSharedRef<FBakeableToTextureElementConverter> Converter = MakeShared<FBakeableToTextureElementConverter>();
	Converter->Tex = InTexmap;
	Converter->FileName = FileName;

	MaxMaterialToUEPbr->AddTexmap(InTexmap, Name, Converter);

	IDatasmithMaterialExpression* BakedExpression = nullptr;

	IDatasmithMaterialExpression* MaterialExpression = MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Texture );

	if ( MaterialExpression )
	{
		IDatasmithMaterialExpressionTexture* MaterialExpressionTexture = static_cast< IDatasmithMaterialExpressionTexture* >( MaterialExpression );
		MaterialExpressionTexture->SetTexturePathName( *Name );

		FDatasmithMaxTexmapToUEPbrUtils::SetupTextureCoordinates( MaxMaterialToUEPbr, MaterialExpressionTexture->GetInputCoordinate(), InTexmap );

		BakedExpression = MaterialExpressionTexture;
	}

	return BakedExpression;
}

bool FDatasmithMaxPassthroughToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap && ( FDatasmithMaxMatHelper::HasNonBakeableSubmap( InTexmap ) ||
		!MaxMaterialToUEPbr->ConvertState.bCanBake ) )
	{
		if ( InTexmap->ClassID() == GRADIENTCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == GRADIENTRAMPCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == CHECKERCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == COLORMAPCLASS )
		{
			return true;
		}
		else if ( InTexmap->ClassID() == FORESTCOLORCLASS )
		{
			return true;
		}
	}

	return false;
}

const TCHAR* FDatasmithMaxPassthroughToUEPbr::GetColorParameterName(Texmap* InTexmap) const
{
	if ( InTexmap->ClassID() == FORESTCOLORCLASS )
	{
		return TEXT("colorbase");
	}
	return TEXT("color1");
}

IDatasmithMaterialExpression* FDatasmithMaxPassthroughToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	DatasmithMaxLogger::Get().AddPartialSupportedMap( InTexmap );

	DatasmithMaxTexmapParser::FMapParameter MapParameter;
	MapParameter.Map = InTexmap->GetSubTexmap(0);
	MapParameter.Weight = 1.f;
	MapParameter.bEnabled = ( InTexmap->SubTexmapOn(0) != 0 );

	FLinearColor PassthroughColor = GetColorParameter( InTexmap, GetColorParameterName(InTexmap) );
	IDatasmithMaterialExpression* Map1 = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( MaxMaterialToUEPbr, MapParameter, TEXT("Submap"), PassthroughColor, TOptional< float >() );

	return Map1;
}

FLinearColor FDatasmithMaxPassthroughToUEPbr::GetColorParameter( Texmap* InTexmap, const TCHAR* ParameterName )
{
	const int NumParamBlocks = InTexmap->NumParamBlocks();

	for (int j = 0; j < NumParamBlocks; j++)
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
		if (ParamBlock2 == nullptr)
		{
			return FLinearColor::Black;
		}

		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
		if (ParamBlockDesc == nullptr)
		{
			return FLinearColor::Black;
		}

		for ( int i = 0; i < ParamBlockDesc->count; i++ )
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if ( FCString::Stricmp( ParamDefinition.int_name, ParameterName ) == 0 )
			{
				BMM_Color_fl Color = (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, GetCOREInterface()->GetTime() );

				ParamBlock2->ReleaseDesc();
				return FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( Color );
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	return FLinearColor::Black;
}

const TCHAR* FDatasmithMaxCellularToUEPbr::GetColorParameterName(Texmap* InTexmap) const
{
	return TEXT("cellcolor");
}

bool FDatasmithMaxCellularToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap && ( FDatasmithMaxMatHelper::HasNonBakeableSubmap( InTexmap ) ||
		!MaxMaterialToUEPbr->ConvertState.bCanBake ) )
	{
		if ( InTexmap->ClassID() == CELLULARCLASS )
		{
			return true;
		}
	}

	return false;
}

const TCHAR* FDatasmithMaxThirdPartyMultiTexmapToUEPbr::GetColorParameterName(Texmap* InTexmap) const
{
	return TEXT("sub_color");
}

bool FDatasmithMaxThirdPartyMultiTexmapToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	if ( InTexmap )
	{
		if ( InTexmap->ClassID() == THIRDPARTYMULTITEXCLASS )
		{
			return true;
		}
	}

	return false;
}
