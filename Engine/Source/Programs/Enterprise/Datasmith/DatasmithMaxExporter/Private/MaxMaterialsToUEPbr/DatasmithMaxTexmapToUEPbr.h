// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithMaxMaterialsToUEPbr.h"
#include "DatasmithMaxTexmapParser.h"

class IDatasmithExpressionInput;
class IDatasmithMaterialExpression;

class FDatasmithMaxTexmapToUEPbrUtils
{
public:
	static IDatasmithMaterialExpression* ConvertTextureOutput( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, IDatasmithMaterialExpression* InputExpression, class TextureOutput* InTextureOutput );
	static void SetupTextureCoordinates( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, IDatasmithExpressionInput& UVCoordinatesInput, Texmap* InTexmap );
	static IDatasmithMaterialExpression* MapOrValue( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, const DatasmithMaxTexmapParser::FMapParameter& MapParameter, const TCHAR* ParameterName,
		TOptional< FLinearColor > Color, TOptional< float > Scalar );
	static IDatasmithMaterialExpression* ConvertBitMap(FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap, FString& ActualBitmapName, bool bUseAlphaAsMono, bool bIsSRGB);
};

class FDatasmithMaxBitmapToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxAutodeskBitmapToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported(const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap) const override;
	virtual IDatasmithMaterialExpression* Convert(FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap) override;
};

class FDatasmithMaxNormalToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
	virtual DatasmithMaxTexmapParser::FNormalMapParameters ParseMap( Texmap* InTexmap );
};

class FDatasmithMaxRGBMultiplyToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxRGBTintToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxMixToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxFalloffToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxNoiseToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxCompositeToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxTextureOutputToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

/**
 * Color Correction is baked when possible but we convert some parameters when we can't bake it
 */
class FDatasmithMaxColorCorrectionToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

/**
 * Maps for which the result is baked into a texture
 */
class FDatasmithMaxBakeableToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

/**
 * Maps for which we only export the first texmap or color
 */
class FDatasmithMaxPassthroughToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;

	virtual const TCHAR* GetColorParameterName(Texmap* InTexmap) const;

	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
	FLinearColor GetColorParameter( Texmap* InTexmap, const TCHAR* ParameterName );
};

class FDatasmithMaxCellularToUEPbr : public FDatasmithMaxPassthroughToUEPbr
{
	virtual const TCHAR* GetColorParameterName(Texmap* InTexmap) const override;
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
};

class FDatasmithMaxThirdPartyMultiTexmapToUEPbr : public FDatasmithMaxPassthroughToUEPbr
{
	virtual const TCHAR* GetColorParameterName(Texmap* InTexmap) const override;
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
};
