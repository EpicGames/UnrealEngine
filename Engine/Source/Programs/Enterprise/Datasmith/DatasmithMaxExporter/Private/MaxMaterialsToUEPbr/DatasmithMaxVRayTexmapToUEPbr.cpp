// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaxMaterialsToUEPbr/DatasmithMaxVRayTexmapToUEPbr.h"

#include "DatasmithMaterialElements.h"
#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxSceneExporter.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithMaxWriter.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxTexmapToUEPbr.h"

#include "DatasmithSceneFactory.h"
#include "Misc/Paths.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "bitmap.h"
	#include "gamma.h"
	#include "maxtypes.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"


namespace DatasmithMaxVRayTexmapToUEPbrImpl
{
	BMM_Color_fl ExtractVrayColor( Texmap* InTexmap, bool bForceInvert )
	{
		int NumParamBlocks = InTexmap->NumParamBlocks();

		int GammaCorrection = 1;
		BMM_Color_fl Color;
		float ColorGamma = 1.f;
		float GammaValue = 1.f;
		float RgbMultiplier = 1.f;

		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID((short)j);
			// The the descriptor to 'decode'
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
			// Loop through all the defined parameters therein
			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Color")) == 0)
				{
					Color = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("RgbMultiplier")) == 0)
				{
					RgbMultiplier = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("ColorGamma")) == 0)
				{
					ColorGamma = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("GammaCorrection")) == 0)
				{
					GammaCorrection = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("GammaValue")) == 0)
				{
					GammaValue = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		switch (GammaCorrection)
		{
		case 1:
			GammaValue = 1.0f / GammaValue;
			break;
		case 2:
			if (gammaMgr.IsEnabled())
			{
				GammaValue = 1.0f / gammaMgr.GetDisplayGamma();
			}
			else
			{
				GammaValue = 1.0f;
			}
			break;
		default:
			GammaValue = 1.0f;
			break;
		}

		Color.r = FMath::Pow( FMath::Pow(Color.r * RgbMultiplier, ColorGamma), GammaValue );
		Color.g = FMath::Pow( FMath::Pow(Color.g * RgbMultiplier, ColorGamma), GammaValue );
		Color.b = FMath::Pow( FMath::Pow(Color.b * RgbMultiplier, ColorGamma), GammaValue );

		return Color;
	}

	struct FMaxVRayDirtParameters
	{
		DatasmithMaxTexmapParser::FMapParameter UnoccludedMap;

		FLinearColor UnoccludedColor;
	};

	FMaxVRayDirtParameters ParseVRayDirtProperties( Texmap& InTexmap )
	{
		FMaxVRayDirtParameters VRayDirtParameters;

		const int NumParamBlocks = InTexmap.NumParamBlocks();
		for ( int j = 0; j < NumParamBlocks; j++ )
		{
			IParamBlock2* ParamBlock2 = InTexmap.GetParamBlockByID( (short)j );
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("unoccluded_color") ) == 0 )
				{
					VRayDirtParameters.UnoccludedColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, GetCOREInterface()->GetTime() ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_unoccluded_color")) == 0)
				{
					VRayDirtParameters.UnoccludedMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, GetCOREInterface()->GetTime() );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_unoccluded_color_on")) == 0)
				{
					VRayDirtParameters.UnoccludedMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, GetCOREInterface()->GetTime() ) != 0 );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_unoccluded_color_multiplier")) == 0)
				{
					VRayDirtParameters.UnoccludedMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() ) / 100.f;
				}
			}

			ParamBlock2->ReleaseDesc();
		}

		return VRayDirtParameters;
	}
}

bool FDatasmithMaxVRayColorTexmapToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	return InTexmap ? (bool)( InTexmap->ClassID() == VRAYCOLORCLASS ) : false;
}

IDatasmithMaterialExpression* FDatasmithMaxVRayColorTexmapToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	BMM_Color_fl VrayColor = DatasmithMaxVRayTexmapToUEPbrImpl::ExtractVrayColor( InTexmap, false );

	IDatasmithMaterialExpressionColor* ColorExpression = static_cast< IDatasmithMaterialExpressionColor* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantColor ) );
	ColorExpression->SetName( TEXT("Vray Color") );
	ColorExpression->GetColor() = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( VrayColor );

	return ColorExpression;
}

bool FDatasmithMaxVRayHDRITexmapToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	return InTexmap ? (bool)( InTexmap->ClassID() == VRAYHDRICLASS ) : false;
}

class FVRayHDRIToTextureElementConverter: public DatasmithMaxDirectLink::ITexmapToTextureElementConverter
{
public:
	virtual TSharedPtr<IDatasmithTextureElement> Convert(DatasmithMaxDirectLink::FMaterialsCollectionTracker& MaterialsTracker, const FString& ActualBitmapName) override
	{
		FString Path = TEXT("");

		int NumParamBlocks = Tex->NumParamBlocks();

		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = Tex->GetParamBlockByID((short)j);
			// The the descriptor to 'decode'
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
			// Loop through all the defined parameters therein
			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("HDRIMapName")) == 0)
				{
					Path = FDatasmithMaxSceneExporter::GetActualPath(ParamBlock2->GetStr(ParamDefinition.ID, GetCOREInterface()->GetTime()));
				}
			}
			ParamBlock2->ReleaseDesc();
		}


		if (Path.IsEmpty())
		{
			return {};
		}

		float Gamma =  FDatasmithMaxMatHelper::GetVrayHdriGamma(Tex);

		TSharedPtr< IDatasmithTextureElement > TextureElement = FDatasmithSceneFactory::CreateTexture(*ActualBitmapName);
		if (gammaMgr.IsEnabled())
		{
			TextureElement->SetRGBCurve(Gamma / 2.2f);
		}
		TextureElement->SetFile(*Path);
		return TextureElement;
	}
	BitmapTex* Tex;
};



IDatasmithMaterialExpression* FDatasmithMaxVRayHDRITexmapToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	FString TexturePath;
	float VrayMultiplier = 1.0f;

	const int NumParamBlocks = InTexmap->NumParamBlocks();

	for ( int j = 0; j < NumParamBlocks; j++ )
	{
		IParamBlock2* ParamBlock2 = InTexmap->GetParamBlockByID( (short)j );
		ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

		for ( int i = 0; i < ParamBlockDesc->count; i++ )
		{
			const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

			if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("HDRIMapName") ) == 0 )
			{
				TexturePath = FDatasmithMaxSceneExporter::GetActualPath( ParamBlock2->GetStr( ParamDefinition.ID, GetCOREInterface()->GetTime() ) );
			}
			else if (FCString::Stricmp( ParamDefinition.int_name, TEXT("RenderMultiplier") ) == 0)
			{
				VrayMultiplier = ParamBlock2->GetFloat( ParamDefinition.ID, GetCOREInterface()->GetTime() );
			}
		}
		ParamBlock2->ReleaseDesc();
	}

	IDatasmithMaterialExpressionTexture* TextureExpression = static_cast< IDatasmithMaterialExpressionTexture* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Texture ) );
	FString ActualBitmapName = FDatasmithMaxMatWriter::GetActualVRayBitmapName( (BitmapTex*)InTexmap );

	TSharedRef<FVRayHDRIToTextureElementConverter> Converter = MakeShared<FVRayHDRIToTextureElementConverter>();
	Converter->Tex = (BitmapTex*)InTexmap;

	MaxMaterialToUEPbr->AddTexmap(InTexmap, ActualBitmapName, Converter);

	TextureExpression->SetTexturePathName( *ActualBitmapName );
	FDatasmithMaxTexmapToUEPbrUtils::SetupTextureCoordinates( MaxMaterialToUEPbr, TextureExpression->GetInputCoordinate(), InTexmap );

	IDatasmithMaterialExpression* ResultExpression = TextureExpression;

	if ( !FMath::IsNearlyEqual( VrayMultiplier, 1.f ) )
	{
		IDatasmithMaterialExpressionGeneric* MultiplyExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
		MultiplyExpression->SetExpressionName( TEXT("Multiply") );

		IDatasmithMaterialExpressionScalar* VrayMultiplierScalar = static_cast< IDatasmithMaterialExpressionScalar* >( MaxMaterialToUEPbr->ConvertState.MaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::ConstantScalar ) );
		VrayMultiplierScalar->SetName( TEXT("Multiplier") );
		VrayMultiplierScalar->GetScalar() = VrayMultiplier;

		TextureExpression->ConnectExpression( *MultiplyExpression->GetInput(0), 0 );
		VrayMultiplierScalar->ConnectExpression( *MultiplyExpression->GetInput(1), 0 );

		ResultExpression = MultiplyExpression;
	}
	
	return ResultExpression;
}

bool FDatasmithMaxVRayDirtTexmapToUEPbr::IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const
{
	return InTexmap ? (bool)( InTexmap->ClassID() == VRAYDIRTCLASS ) : false;
}

IDatasmithMaterialExpression* FDatasmithMaxVRayDirtTexmapToUEPbr::Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap )
{
	if ( !InTexmap )
	{
		return nullptr;
	}

	DatasmithMaxVRayTexmapToUEPbrImpl::FMaxVRayDirtParameters VRayDirtParameters = DatasmithMaxVRayTexmapToUEPbrImpl::ParseVRayDirtProperties( *InTexmap );

	return FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( MaxMaterialToUEPbr, VRayDirtParameters.UnoccludedMap, TEXT("Vray Dirt Unoccluded Color"), VRayDirtParameters.UnoccludedColor, TOptional< float >() );
}
