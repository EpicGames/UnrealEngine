// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithMaxMaterialsToUEPbr.h"

class FDatasmithMaxVRayColorTexmapToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxVRayHDRITexmapToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};

class FDatasmithMaxVRayDirtTexmapToUEPbr : public IDatasmithMaxTexmapToUEPbr
{
public:
	virtual bool IsSupported( const FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) const override;
	virtual IDatasmithMaterialExpression* Convert( FDatasmithMaxMaterialsToUEPbr* MaxMaterialToUEPbr, Texmap* InTexmap ) override;
};
