// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaxMaterialsToUEPbr/DatasmithMaxVrayMaterialsToUEPbr.h"

#include "DatasmithMaxClassIDs.h"
#include "DatasmithMaxTexmapParser.h"
#include "DatasmithMaxWriter.h"
#include "DatasmithSceneFactory.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxTexmapToUEPbr.h"
#include "MaxMaterialsToUEPbr/DatasmithMaxVRayTexmapToUEPbr.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "max.h"
	#include "mtl.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

namespace DatasmithMaxVRayMaterialsToUEPbrImpl
{
	struct FMaxVRayMaterial
	{
		FMaxVRayMaterial()
			: bUseRoughness( false )
			, ReflectionGlossiness( 0.f )
			, bReflectionFresnel( false )
			, bLockReflectionIORToRefractionIOR( false )
			, ReflectionIOR( 0.f )
			, RefractionIOR( 0.f )
			, FogMultiplier( 0.f )
		{
		}

		// Diffuse
		DatasmithMaxTexmapParser::FWeightedColorParameter Diffuse;
		DatasmithMaxTexmapParser::FMapParameter DiffuseMap;

		// Reflection
		DatasmithMaxTexmapParser::FWeightedColorParameter Reflection;
		DatasmithMaxTexmapParser::FMapParameter ReflectionMap;

		bool bUseRoughness; // Reflection glossiness should be inverted
		float ReflectionGlossiness;
		DatasmithMaxTexmapParser::FMapParameter ReflectionGlossinessMap;

		// Reflection IOR
		bool bReflectionFresnel;
		bool bLockReflectionIORToRefractionIOR;

		float ReflectionIOR;
		DatasmithMaxTexmapParser::FMapParameter ReflectionIORMap;

		// Refraction
		DatasmithMaxTexmapParser::FWeightedColorParameter Refraction;
		DatasmithMaxTexmapParser::FMapParameter RefractionMap;

		// Refraction IOR
		float RefractionIOR;
		DatasmithMaxTexmapParser::FMapParameter RefractionIORMap;

		DatasmithMaxTexmapParser::FMapParameter OpacityMap;

		// Bump
		DatasmithMaxTexmapParser::FMapParameter BumpMap;

		// Displacement
		DatasmithMaxTexmapParser::FMapParameter DisplacementMap;

		// Fog
		DatasmithMaxTexmapParser::FWeightedColorParameter FogColor;
		DatasmithMaxTexmapParser::FMapParameter FogColorMap;
		float FogMultiplier;
	};

	FMaxVRayMaterial ParseVRayMaterialProperties( Mtl& Material )
	{
		FMaxVRayMaterial VRayMaterialProperties;

		const int NumParamBlocks = Material.NumParamBlocks();

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				// Diffuse
				if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("diffuse")) == 0 )
				{
					VRayMaterialProperties.Diffuse.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, CurrentTime ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_diffuse")) == 0)
				{
					VRayMaterialProperties.DiffuseMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("texmap_diffuse_on") ) == 0 )
				{
					VRayMaterialProperties.DiffuseMap.bEnabled = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_diffuse_multiplier")) == 0)
				{
					VRayMaterialProperties.DiffuseMap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime) / 100.f;
				}

				// Reflection
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection")) == 0 )
				{
					VRayMaterialProperties.Reflection.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, CurrentTime ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflection")) == 0)
				{
					VRayMaterialProperties.ReflectionMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflection_on")) == 0)
				{
					VRayMaterialProperties.ReflectionMap.bEnabled = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflection_multiplier")) == 0)
				{
					VRayMaterialProperties.ReflectionMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime ) / 100.0f;
				}

				// Reflection Glossiness
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("reflection_glossiness") ) == 0 )
				{
					VRayMaterialProperties.ReflectionGlossiness = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflectionGlossiness")) == 0)
				{
					VRayMaterialProperties.ReflectionGlossinessMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflectionGlossiness_on")) == 0)
				{
					VRayMaterialProperties.ReflectionGlossinessMap.bEnabled = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_reflectionGlossiness_multiplier")) == 0)
				{
					VRayMaterialProperties.ReflectionGlossinessMap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime) / 100.0f;
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("brdf_useRoughness") ) == 0 )
				{
					VRayMaterialProperties.bUseRoughness = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}

				// Reflection IOR
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection_fresnel")) == 0 )
				{
					VRayMaterialProperties.bReflectionFresnel = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection_ior")) == 0 )
				{
					VRayMaterialProperties.ReflectionIOR = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_ReflectionIOR")) == 0 )
				{
					VRayMaterialProperties.ReflectionIORMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_ReflectionIOR_on")) == 0 )
				{
					VRayMaterialProperties.ReflectionIORMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_ReflectionIOR_multiplier")) == 0 )
				{
					VRayMaterialProperties.ReflectionIORMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime ) * 0.01f;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("reflection_lockIOR")) == 0)
				{
					VRayMaterialProperties.bLockReflectionIORToRefractionIOR = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}

				// Refraction
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("refraction")) == 0 )
				{
					VRayMaterialProperties.Refraction.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, CurrentTime ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refraction")) == 0)
				{
					VRayMaterialProperties.RefractionMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refraction_on")) == 0)
				{
					VRayMaterialProperties.RefractionMap.bEnabled = ( ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime) != 0 );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refraction_multiplier")) == 0)
				{
					VRayMaterialProperties.RefractionMap.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime) / 100.0f;
				}

				// Refraction IOR
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("refraction_ior")) == 0 )
				{
					VRayMaterialProperties.RefractionIOR = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refractionIOR")) == 0 )
				{
					VRayMaterialProperties.RefractionIORMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refractionIOR_on")) == 0 )
				{
					VRayMaterialProperties.RefractionIORMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refractionIOR_multiplier")) == 0 )
				{
					VRayMaterialProperties.RefractionIORMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime ) * 0.01f;
				}

				// Opacity
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_opacity")) == 0)
				{
					VRayMaterialProperties.OpacityMap.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime);
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("texmap_opacity_on") ) == 0 )
				{
					VRayMaterialProperties.OpacityMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_opacity_multiplier")) == 0 )
				{
					VRayMaterialProperties.OpacityMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime ) * 0.01f;
				}

				// Bump
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_bump")) == 0 )
				{
					VRayMaterialProperties.BumpMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_bump_multiplier")) == 0)
				{
					VRayMaterialProperties.BumpMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime ) / 100.f;
				}
				else if ( FCString::Stricmp( ParamDefinition.int_name, TEXT("texmap_bump_on") ) == 0 )
				{
					VRayMaterialProperties.BumpMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
				}

				// Displacement
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_displacement")) == 0)
				{
					VRayMaterialProperties.DisplacementMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_displacement_multiplier")) == 0)
				{
					VRayMaterialProperties.DisplacementMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime ) / 100.f;
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_displacement_on")) == 0)
				{
					VRayMaterialProperties.DisplacementMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
				}

				// Fog
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("refraction_fogColor")) == 0)
				{
					VRayMaterialProperties.FogColor.Value = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor( (BMM_Color_fl)ParamBlock2->GetColor( ParamDefinition.ID, CurrentTime ) );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refraction_fog")) == 0)
				{
					VRayMaterialProperties.FogColorMap.Map = ParamBlock2->GetTexmap( ParamDefinition.ID, CurrentTime );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refraction_fog_on")) == 0)
				{
					VRayMaterialProperties.FogColorMap.bEnabled = ( ParamBlock2->GetInt( ParamDefinition.ID, CurrentTime ) != 0 );
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_refraction_fog_multiplier")) == 0)
				{
					VRayMaterialProperties.FogColorMap.Weight = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime ) / 100.f;
				}
				else if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("refraction_fogMult")) == 0 )
				{
					VRayMaterialProperties.FogMultiplier = ParamBlock2->GetFloat( ParamDefinition.ID, CurrentTime );
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		return VRayMaterialProperties;
	}

	struct FMaxVRay2SidedMaterial
	{
		FMaxVRay2SidedMaterial()
			: FrontMaterial( nullptr )
		{
		}

		// Front
		Mtl* FrontMaterial;
	};

	FMaxVRay2SidedMaterial ParseVRay2SidedMaterialProperties( Mtl& Material )
	{
		FMaxVRay2SidedMaterial VRay2SidedMaterialProperties;

		const int NumParamBlocks = Material.NumParamBlocks();

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				// Front
				if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("frontMtl")) == 0 )
				{
					VRay2SidedMaterialProperties.FrontMaterial = ParamBlock2->GetMtl( ParamDefinition.ID, CurrentTime );
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		return VRay2SidedMaterialProperties;
	}

	struct FMaxVRayWrapperMaterial
	{
		FMaxVRayWrapperMaterial()
			: BaseMaterial( nullptr )
		{
		}

		// Base
		Mtl* BaseMaterial;
	};

	FMaxVRayWrapperMaterial ParseVRayWrapperMaterialProperties( Mtl& Material )
	{
		FMaxVRayWrapperMaterial VRayWrapperMaterialProperties;

		const int NumParamBlocks = Material.NumParamBlocks();

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();

		for (int j = 0; j < NumParamBlocks; j++)
		{
			IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)j);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int i = 0; i < ParamBlockDesc->count; i++)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

				// Base
				if ( FCString::Stricmp(ParamDefinition.int_name, TEXT("baseMtl")) == 0 )
				{
					VRayWrapperMaterialProperties.BaseMaterial = ParamBlock2->GetMtl( ParamDefinition.ID, CurrentTime );
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		return VRayWrapperMaterialProperties;
	}

	struct FMaxVRayBlendMaterial
	{
		struct FVRayCoatMaterialProperties
		{
			Mtl* Material = nullptr;
			DatasmithMaxTexmapParser::FMapParameter MaterialBlendParameter;
			FLinearColor MixColor;
		};

		Mtl* BaseMaterial = nullptr;
		static const int32 MaximumNumberOfCoat = 10;
		FVRayCoatMaterialProperties CoatedMaterials[MaximumNumberOfCoat];
	};

	FMaxVRayBlendMaterial ParseVRayBlendMaterialProperties(Mtl& Material)
	{
		FMaxVRayBlendMaterial VRayBlendMaterialProperties;
		FMaxVRayBlendMaterial::FVRayCoatMaterialProperties* CoatedMaterials = VRayBlendMaterialProperties.CoatedMaterials;

		const TimeValue CurrentTime = GetCOREInterface()->GetTime();
		const int NumParamBlocks = Material.NumParamBlocks();

		for (int ParemBlockIndex = 0; ParemBlockIndex < NumParamBlocks; ++ParemBlockIndex)
		{
			IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)ParemBlockIndex);
			ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();

			for (int DescIndex = 0; DescIndex < ParamBlockDesc->count; ++DescIndex)
			{
				const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[DescIndex];

				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("baseMtl")) == 0)
				{
					VRayBlendMaterialProperties.BaseMaterial = ParamBlock2->GetMtl(ParamDefinition.ID, CurrentTime);
				}
				if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coatMtl")) == 0)
				{
					for (int CoatIndex = 0; CoatIndex < FMaxVRayBlendMaterial::MaximumNumberOfCoat; ++CoatIndex)
					{
						CoatedMaterials[CoatIndex].Material = ParamBlock2->GetMtl(ParamDefinition.ID, CurrentTime, CoatIndex);
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_blend")) == 0)
				{
					for (int CoatIndex = 0; CoatIndex < FMaxVRayBlendMaterial::MaximumNumberOfCoat; ++CoatIndex)
					{
						CoatedMaterials[CoatIndex].MaterialBlendParameter.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, CurrentTime, CoatIndex);
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("coatMtl_enable")) == 0)
				{
					for (int CoatIndex = 0; CoatIndex < FMaxVRayBlendMaterial::MaximumNumberOfCoat; ++CoatIndex)
					{
						CoatedMaterials[CoatIndex].MaterialBlendParameter.bEnabled = ParamBlock2->GetInt(ParamDefinition.ID, CurrentTime, CoatIndex) != 0;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_blend_multiplier")) == 0)
				{
					for (int CoatIndex = 0; CoatIndex < 9; ++CoatIndex)
					{
						CoatedMaterials[CoatIndex].MaterialBlendParameter.Weight = ParamBlock2->GetFloat(ParamDefinition.ID, CurrentTime, CoatIndex) / 100.f;
					}
				}
				else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Blend")) == 0)
				{
					for (int CoatIndex = 0; CoatIndex < FMaxVRayBlendMaterial::MaximumNumberOfCoat; ++CoatIndex)
					{
						CoatedMaterials[CoatIndex].MixColor = FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor((BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, CurrentTime, CoatIndex));
					}
				}
			}
			ParamBlock2->ReleaseDesc();
		}

		return VRayBlendMaterialProperties;
	}

	struct FVRayLightMaterial
	{
		DatasmithMaxTexmapParser::FMapParameter EmitTexture;
		DatasmithMaxTexmapParser::FMapParameter ClipTexture;
		BMM_Color_fl EmitColor;

		float Multiplier = 1.0f;

		void Parse(Mtl& Material)
		{
			int NumParamBlocks = Material.NumParamBlocks();

			for (int j = 0; j < NumParamBlocks; j++)
			{
				IParamBlock2* ParamBlock2 = Material.GetParamBlockByID((short)j);
				// The the descriptor to 'decode'
				ParamBlockDesc2* ParamBlockDesc = ParamBlock2->GetDesc();
				// Loop through all the defined parameters therein
				for (int i = 0; i < ParamBlockDesc->count; i++)
				{
					const ParamDef& ParamDefinition = ParamBlockDesc->paramdefs[i];

					if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap")) == 0)
					{
						EmitTexture.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacity_texmap")) == 0)
					{
						ClipTexture.Map = ParamBlock2->GetTexmap(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("texmap_on")) == 0)
					{
						if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
						{
							EmitTexture.bEnabled = false;
						}
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("opacity_texmap_on")) == 0)
					{
						if (ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime()) == 0)
						{
							ClipTexture.bEnabled = false;
						}
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Color")) == 0)
					{
						EmitColor = (BMM_Color_fl)ParamBlock2->GetColor(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("twoSided")) == 0)
					{
						// int twoSided = ParamBlock2->GetInt(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
					else if (FCString::Stricmp(ParamDefinition.int_name, TEXT("Multiplier")) == 0)
					{
						Multiplier = ParamBlock2->GetFloat(ParamDefinition.ID, GetCOREInterface()->GetTime());
					}
				}
				ParamBlock2->ReleaseDesc();
			}
		}

	};

}

FDatasmithMaxVRayMaterialsToUEPbr::FDatasmithMaxVRayMaterialsToUEPbr()
{
	TexmapConverters.Add( new FDatasmithMaxVRayColorTexmapToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxVRayHDRITexmapToUEPbr() );
	TexmapConverters.Add( new FDatasmithMaxVRayDirtTexmapToUEPbr() );
}

bool FDatasmithMaxVRayMaterialsToUEPbr::IsSupported( Mtl* Material )
{
	return true;
}

void FDatasmithMaxVRayMaterialsToUEPbr::Convert( TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithBaseMaterialElement >& MaterialElement, Mtl* Material, const TCHAR* AssetsPath )
{
	if ( !Material )
	{
		return;
	}

	TSharedRef< IDatasmithUEPbrMaterialElement > PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial( GetMaterialName(Material) );
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	DatasmithMaxVRayMaterialsToUEPbrImpl::FMaxVRayMaterial VRayMaterialProperties = DatasmithMaxVRayMaterialsToUEPbrImpl::ParseVRayMaterialProperties( *Material );

	ConvertState.DefaultTextureMode = EDatasmithTextureMode::Diffuse; // Both Diffuse and Reflection are considered diffuse maps

	// Diffuse
	IDatasmithMaterialExpression* DiffuseExpression = nullptr;
	{
		DiffuseExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, VRayMaterialProperties.DiffuseMap, TEXT("Diffuse Color"),
			VRayMaterialProperties.Diffuse.Value, TOptional< float >() );
	}

	if ( DiffuseExpression )
	{
		DiffuseExpression->SetName( TEXT("Diffuse") );
	}

	// Reflection
	IDatasmithMaterialExpression* ReflectionExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, VRayMaterialProperties.ReflectionMap, TEXT("Reflection Color"), VRayMaterialProperties.Reflection.Value, TOptional< float >() );

	if ( ReflectionExpression )
	{
		ReflectionExpression->SetName( TEXT("Reflection") );
	}

	IDatasmithMaterialExpressionGeneric* ReflectionIntensityExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();

	if ( VRayMaterialProperties.bReflectionFresnel )
	{
		ReflectionIntensityExpression->SetExpressionName( TEXT("Desaturation") );

		//TSharedRef< IDatasmithKeyValueProperty > LuminanceFactors = FDatasmithSceneFactory::CreateKeyValueProperty( TEXT("LuminanceFactors") );
		//LuminanceFactors->SetPropertyType( EDatasmithKeyValuePropertyType::Color );
		//LuminanceFactors->SetValue( *FLinearColor::White.ToString() );

		//ReflectionIntensityExpression->AddProperty( LuminanceFactors );

		ReflectionExpression->ConnectExpression( *ReflectionIntensityExpression->GetInput(0) );
	}
	else
	{
		ReflectionIntensityExpression->SetExpressionName( TEXT("Max") );

		IDatasmithMaterialExpressionGeneric* MaxRGExpression = static_cast< IDatasmithMaterialExpressionGeneric* >( PbrMaterialElement->AddMaterialExpression( EDatasmithMaterialExpressionType::Generic ) );
		MaxRGExpression->SetExpressionName( TEXT("Max") );

		ReflectionExpression->ConnectExpression( *MaxRGExpression->GetInput(0), 0 );
		ReflectionExpression->ConnectExpression( *MaxRGExpression->GetInput(1), 1 );

		ReflectionExpression->ConnectExpression( *ReflectionIntensityExpression->GetInput(1), 2 );

		MaxRGExpression->ConnectExpression( *ReflectionIntensityExpression->GetInput(0) );
	}

	// Glossiness
	IDatasmithMaterialExpression* GlossinessExpression = nullptr;
	{
		TGuardValue< bool > SetIsMonoChannel( ConvertState.bIsMonoChannel, true );

		ConvertState.DefaultTextureMode = EDatasmithTextureMode::Specular;

		GlossinessExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, VRayMaterialProperties.ReflectionGlossinessMap, TEXT("Reflection Glossiness"), TOptional< FLinearColor >(), VRayMaterialProperties.ReflectionGlossiness );

		if ( GlossinessExpression )
		{
			GlossinessExpression->SetName( TEXT("Reflection Glossiness") );

			if ( VRayMaterialProperties.bUseRoughness )
			{
				IDatasmithMaterialExpressionGeneric* InverseGlossinessExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				InverseGlossinessExpression->SetExpressionName( TEXT("OneMinus") );

				GlossinessExpression->ConnectExpression( *InverseGlossinessExpression->GetInput(0) );

				GlossinessExpression = InverseGlossinessExpression;
			}
		}
	}

	// Bump
	{
		ConvertState.DefaultTextureMode = EDatasmithTextureMode::Bump; // Will change to normal if we pass through a normal map texmap
		ConvertState.bCanBake = false; // Current baking fails to produce proper normal maps

		IDatasmithMaterialExpression* BumpExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, VRayMaterialProperties.BumpMap, TEXT("Bump Map"), TOptional< FLinearColor >(), TOptional< float >() );

		if ( BumpExpression )
		{
			BumpExpression->ConnectExpression( PbrMaterialElement->GetNormal() );
		}

		if ( BumpExpression )
		{
			BumpExpression->SetName( TEXT("Bump Map") );
		}

		ConvertState.bCanBake = true;
	}

	ConvertState.DefaultTextureMode = EDatasmithTextureMode::Specular; // At this point, all maps are considered specular maps

	// Opacity
	IDatasmithMaterialExpression* OpacityExpression = nullptr;
	{
		TGuardValue< bool > SetIsMonoChannel( ConvertState.bIsMonoChannel, true );
		OpacityExpression = ConvertTexmap( VRayMaterialProperties.OpacityMap );
	}

	// Refraction
	IDatasmithMaterialExpression* RefractionExpression = nullptr;
	IDatasmithMaterialExpression* RefractionIOR = nullptr;
	{
		TOptional< FLinearColor > OptionalRefractionColor;

		if ( !VRayMaterialProperties.Refraction.Value.IsAlmostBlack() )
		{
			OptionalRefractionColor = VRayMaterialProperties.Refraction.Value;
		}

		RefractionExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, VRayMaterialProperties.RefractionMap, TEXT("Refraction"), OptionalRefractionColor, TOptional< float >() );

		if ( OpacityExpression || RefractionExpression )
		{
			TGuardValue< bool > SetIsMonoChannel( ConvertState.bIsMonoChannel, true );
			RefractionIOR = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, VRayMaterialProperties.RefractionIORMap, TEXT("Refraction IOR"), TOptional< FLinearColor >(), VRayMaterialProperties.RefractionIOR );
		}
	}

	// UE Roughness
	{
		IDatasmithMaterialExpressionGeneric* MultiplyGlossiness = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		MultiplyGlossiness->SetExpressionName( TEXT("Multiply") );

		GlossinessExpression->ConnectExpression( *MultiplyGlossiness->GetInput(0) );
		GlossinessExpression->ConnectExpression( *MultiplyGlossiness->GetInput(1) );

		IDatasmithMaterialExpression* RoughnessOutput = MultiplyGlossiness;

		IDatasmithMaterialExpressionGeneric* OneMinusRougnessExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		OneMinusRougnessExpression->SetExpressionName( TEXT("OneMinus") );

		MultiplyGlossiness->ConnectExpression( *OneMinusRougnessExpression->GetInput(0) );

		RoughnessOutput = OneMinusRougnessExpression;

		IDatasmithMaterialExpressionGeneric* PowRoughnessExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		PowRoughnessExpression->SetExpressionName( TEXT("Power") );

		TSharedRef< IDatasmithKeyValueProperty > PowRoughnessValue = FDatasmithSceneFactory::CreateKeyValueProperty( TEXT("ConstExponent") );
		PowRoughnessValue->SetPropertyType( EDatasmithKeyValuePropertyType::Float );
		PowRoughnessValue->SetValue( *LexToString( 1.5f ) );

		PowRoughnessExpression->AddProperty( PowRoughnessValue );

		RoughnessOutput->ConnectExpression( *PowRoughnessExpression->GetInput(0) );
		PowRoughnessExpression->ConnectExpression( PbrMaterialElement->GetRoughness() );
	}

	IDatasmithMaterialExpressionGeneric* ReflectionFresnelExpression = nullptr;

	IDatasmithMaterialExpressionGeneric* IORFactor = nullptr;

	if ( VRayMaterialProperties.bReflectionFresnel )
	{
		DiffuseExpression->ConnectExpression( PbrMaterialElement->GetBaseColor() );

		IDatasmithMaterialExpressionGeneric* DiffuseLerpExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		DiffuseLerpExpression->SetExpressionName( TEXT("LinearInterpolate") );

		DiffuseLerpExpression->ConnectExpression( PbrMaterialElement->GetBaseColor() );

		IDatasmithMaterialExpression* ReflectionIOR = nullptr;

		{
			TGuardValue< bool > SetIsMonoChannel( ConvertState.bIsMonoChannel, true );

			if ( !VRayMaterialProperties.bLockReflectionIORToRefractionIOR )
			{
				ReflectionIOR = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, VRayMaterialProperties.ReflectionIORMap, TEXT("Fresnel IOR"), TOptional< FLinearColor >(), VRayMaterialProperties.ReflectionIOR );
			}
			else
			{
				ReflectionIOR = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, VRayMaterialProperties.RefractionIORMap, TEXT("Fresnel IOR"), TOptional< FLinearColor >(), VRayMaterialProperties.RefractionIOR );
			}
		}

		ReflectionIOR->SetName( TEXT("Fresnel IOR") );

		IDatasmithMaterialExpressionScalar* MinusOneFresnelIOR = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		MinusOneFresnelIOR->GetScalar() = -1.f;

		IDatasmithMaterialExpressionGeneric* AddAdjustFresnelIOR = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		AddAdjustFresnelIOR->SetExpressionName( TEXT("Add") );

		ReflectionIOR->ConnectExpression( *AddAdjustFresnelIOR->GetInput(0) );
		MinusOneFresnelIOR->ConnectExpression( *AddAdjustFresnelIOR->GetInput(1) );

		IORFactor = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		IORFactor->SetExpressionName( TEXT("Multiply") );

		IDatasmithMaterialExpressionScalar* ScaleIORScalar = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		ScaleIORScalar->GetScalar() = 0.02f;

		AddAdjustFresnelIOR->ConnectExpression( *IORFactor->GetInput(0) );
		ScaleIORScalar->ConnectExpression( *IORFactor->GetInput(1) );

		IDatasmithMaterialExpressionGeneric* BaseColorIORPow = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		BaseColorIORPow->SetExpressionName( TEXT("Power") );

		IDatasmithMaterialExpressionScalar* BaseColorIORPowScalar = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		BaseColorIORPowScalar->GetScalar() = 0.5f;

		IORFactor->ConnectExpression( *BaseColorIORPow->GetInput(0) );
		BaseColorIORPowScalar->ConnectExpression( *BaseColorIORPow->GetInput(1) );

		IDatasmithMaterialExpressionGeneric* DiffuseIORLerpExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		DiffuseIORLerpExpression->SetExpressionName( TEXT("LinearInterpolate") );

		DiffuseExpression->ConnectExpression( *DiffuseIORLerpExpression->GetInput(0) );
		ReflectionExpression->ConnectExpression( *DiffuseIORLerpExpression->GetInput(1) );
		BaseColorIORPow->ConnectExpression( *DiffuseIORLerpExpression->GetInput(2) );

		DiffuseExpression->ConnectExpression( *DiffuseLerpExpression->GetInput(0) );
		DiffuseIORLerpExpression->ConnectExpression( *DiffuseLerpExpression->GetInput(1) );
		ReflectionIntensityExpression->ConnectExpression( *DiffuseLerpExpression->GetInput(2) );
	}
	else
	{
		IDatasmithMaterialExpressionGeneric* MultiplyExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		MultiplyExpression->SetExpressionName( TEXT("Multiply") );

		IDatasmithMaterialExpressionGeneric* OneMinusExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		OneMinusExpression->SetExpressionName( TEXT("OneMinus") );

		DiffuseExpression->ConnectExpression( *MultiplyExpression->GetInput(0) );

		ReflectionIntensityExpression->ConnectExpression( *OneMinusExpression->GetInput( 0 ) );

		OneMinusExpression->ConnectExpression( *MultiplyExpression->GetInput(1) );

		IDatasmithMaterialExpressionGeneric* AddExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		AddExpression->SetExpressionName( TEXT("Add") );

		MultiplyExpression->ConnectExpression( *AddExpression->GetInput( 0 ) );

		IDatasmithMaterialExpressionGeneric* MultiplyReflectionExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		MultiplyReflectionExpression->SetExpressionName( TEXT("Multiply") );

		ReflectionExpression->ConnectExpression( *MultiplyReflectionExpression->GetInput(0) );
		ReflectionIntensityExpression->ConnectExpression( *MultiplyReflectionExpression->GetInput(1) );

		MultiplyReflectionExpression->ConnectExpression( *AddExpression->GetInput( 1 ) );

		AddExpression->ConnectExpression( PbrMaterialElement->GetBaseColor() );
	}

	// UE Metallic
	IDatasmithMaterialExpression* MetallicExpression = nullptr;
	if ( VRayMaterialProperties.bReflectionFresnel )
	{
		IDatasmithMaterialExpressionGeneric* MetallicIORPow = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		MetallicIORPow->SetExpressionName( TEXT("Power") );

		IDatasmithMaterialExpressionScalar* MetallicIORPowScalar = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
		MetallicIORPowScalar->GetScalar() = 0.2f;

		IORFactor->ConnectExpression( *MetallicIORPow->GetInput(0) );
		MetallicIORPowScalar->ConnectExpression( *MetallicIORPow->GetInput(1) );

		IDatasmithMaterialExpressionGeneric* MultiplyIORExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
		MultiplyIORExpression->SetExpressionName( TEXT("Multiply") );

		ReflectionIntensityExpression->ConnectExpression( *MultiplyIORExpression->GetInput(0) );
		MetallicIORPow->ConnectExpression( *MultiplyIORExpression->GetInput(1) );

		MetallicExpression = MultiplyIORExpression;
	}
	else
	{

		MetallicExpression = ReflectionIntensityExpression;
	}

	if ( MetallicExpression )
	{
		MetallicExpression->ConnectExpression( PbrMaterialElement->GetMetallic() );
	}

	// UE Specular
	if ( MetallicExpression )
	{
		MetallicExpression->ConnectExpression( PbrMaterialElement->GetSpecular() );
	}

	// UE Opacity & Refraction
	if ( OpacityExpression || RefractionExpression )
	{
		IDatasmithMaterialExpression* UEOpacityExpression = nullptr;

		if ( RefractionExpression )
		{
			IDatasmithMaterialExpressionGeneric* RefractionIntensity = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
			RefractionIntensity->SetExpressionName( TEXT("Desaturation") );

			RefractionExpression->ConnectExpression( *RefractionIntensity->GetInput(0) );

			IDatasmithMaterialExpressionGeneric* OneMinusRefraction = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
			OneMinusRefraction->SetExpressionName( TEXT("OneMinus") );

			RefractionIntensity->ConnectExpression( *OneMinusRefraction->GetInput(0) );

			if ( OpacityExpression )
			{
				IDatasmithMaterialExpressionGeneric* LerpOpacityRefraction = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				LerpOpacityRefraction->SetExpressionName( TEXT("LinearInterpolate") );

				OpacityExpression->ConnectExpression( *LerpOpacityRefraction->GetInput(0) );
				OneMinusRefraction->ConnectExpression( *LerpOpacityRefraction->GetInput(1) );
				OpacityExpression->ConnectExpression( *LerpOpacityRefraction->GetInput(2) );

				UEOpacityExpression = LerpOpacityRefraction;
			}
			else
			{
				UEOpacityExpression = OneMinusRefraction;
			}
		}
		else
		{
			UEOpacityExpression = OpacityExpression;
		}

		if ( UEOpacityExpression )
		{
			UEOpacityExpression->ConnectExpression( PbrMaterialElement->GetOpacity() );

			IDatasmithMaterialExpressionGeneric* ThinTranslucencyMaterialOutput = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
			ThinTranslucencyMaterialOutput->SetExpressionName( TEXT("ThinTranslucentMaterialOutput") );

			// Fog
			IDatasmithMaterialExpression* FogExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue( this, VRayMaterialProperties.FogColorMap, TEXT("Fog"), VRayMaterialProperties.FogColor.Value, TOptional< float >() );

			if ( FogExpression )
			{
				FogExpression->SetName( TEXT("Fog") );

				IDatasmithMaterialExpressionScalar* FogMultiplier = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
				FogMultiplier->SetName( TEXT("Fog Multiplier") );
				FogMultiplier->GetScalar() = VRayMaterialProperties.FogMultiplier;

				IDatasmithMaterialExpressionGeneric* MultiplyFog = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
				MultiplyFog->SetExpressionName( TEXT("Multiply") );

				FogExpression->ConnectExpression( *MultiplyFog->GetInput(0) );
				FogMultiplier->ConnectExpression( *MultiplyFog->GetInput(1) );

				FogExpression = MultiplyFog;

				FogExpression->ConnectExpression( *ThinTranslucencyMaterialOutput->GetInput(0) );
			}

			PbrMaterialElement->SetShadingModel( EDatasmithShadingModel::ThinTranslucent );
		}

		if ( RefractionIOR )
		{
			IDatasmithMaterialExpressionGeneric* RefractionFresnel = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
			RefractionFresnel->SetExpressionName( TEXT("Fresnel") );

			IDatasmithMaterialExpressionGeneric* RefractionLerp = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionGeneric >();
			RefractionLerp->SetExpressionName( TEXT("LinearInterpolate") );

			IDatasmithMaterialExpressionScalar* RefractionIOROne = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
			RefractionIOROne->GetScalar() = 1.f;

			RefractionIOROne->ConnectExpression( *RefractionLerp->GetInput(0) );
			RefractionIOR->ConnectExpression( *RefractionLerp->GetInput(1) );
			RefractionFresnel->ConnectExpression( *RefractionLerp->GetInput(2) );

			RefractionLerp->ConnectExpression( PbrMaterialElement->GetRefraction() );
		}
	}

	MaterialElement = PbrMaterialElement;
}

bool FDatasmithMaxVRay2SidedMaterialsToUEPbr::IsSupported( Mtl* Material )
{
	if ( !Material )
	{
		return false;
	}

	DatasmithMaxVRayMaterialsToUEPbrImpl::FMaxVRay2SidedMaterial VRay2SidedMaterialProperties = DatasmithMaxVRayMaterialsToUEPbrImpl::ParseVRay2SidedMaterialProperties( *Material );

	return FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter( VRay2SidedMaterialProperties.FrontMaterial ) != nullptr;
}

void FDatasmithMaxVRay2SidedMaterialsToUEPbr::Convert( TSharedRef<IDatasmithScene> DatasmithScene, TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement, Mtl* Material, const TCHAR* AssetsPath )
{
	if ( !Material )
	{
		return;
	}

	DatasmithMaxVRayMaterialsToUEPbrImpl::FMaxVRay2SidedMaterial VRay2SidedMaterialProperties = DatasmithMaxVRayMaterialsToUEPbrImpl::ParseVRay2SidedMaterialProperties( *Material );

	if ( VRay2SidedMaterialProperties.FrontMaterial )
	{
		if ( FDatasmithMaxMaterialsToUEPbr* MaterialConverter = FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter( VRay2SidedMaterialProperties.FrontMaterial ) )
		{
			MaterialConverter->Convert( DatasmithScene, MaterialElement, VRay2SidedMaterialProperties.FrontMaterial, AssetsPath );

			if ( MaterialElement )
			{
				MaterialElement->SetName( GetMaterialName(Material) ); // Name it with the main material not the front material

				if ( MaterialElement->IsA( EDatasmithElementType::UEPbrMaterial ) )
				{
					TSharedRef< IDatasmithUEPbrMaterialElement > UEPbrMaterialElement = StaticCastSharedRef< IDatasmithUEPbrMaterialElement >( MaterialElement.ToSharedRef() );
					UEPbrMaterialElement->SetTwoSided( true );
				}
			}
		}
	}
}

bool FDatasmithMaxVRayWrapperMaterialsToUEPbr::IsSupported( Mtl* Material )
{
	if ( !Material )
	{
		return false;
	}

	DatasmithMaxVRayMaterialsToUEPbrImpl::FMaxVRayWrapperMaterial VRayWrapperMaterialProperties = DatasmithMaxVRayMaterialsToUEPbrImpl::ParseVRayWrapperMaterialProperties( *Material );
	return FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter( VRayWrapperMaterialProperties.BaseMaterial ) != nullptr;
}

void FDatasmithMaxVRayWrapperMaterialsToUEPbr::Convert( TSharedRef<IDatasmithScene> DatasmithScene, TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement, Mtl* Material, const TCHAR* AssetsPath )
{
	if ( !Material )
	{
		return;
	}

	DatasmithMaxVRayMaterialsToUEPbrImpl::FMaxVRayWrapperMaterial VRayWrapperMaterialProperties = DatasmithMaxVRayMaterialsToUEPbrImpl::ParseVRayWrapperMaterialProperties( *Material );

	if ( VRayWrapperMaterialProperties.BaseMaterial )
	{
		if ( FDatasmithMaxMaterialsToUEPbr* MaterialConverter = FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter( VRayWrapperMaterialProperties.BaseMaterial ) )
		{
			MaterialConverter->Convert( DatasmithScene, MaterialElement, VRayWrapperMaterialProperties.BaseMaterial, AssetsPath );

			if ( MaterialElement )
			{
				MaterialElement->SetName( GetMaterialName(Material) ); // Name it with the main material not the base material
			}
		}
	}
}

bool FDatasmithMaxVRayBlendMaterialToUEPbr::IsSupported( Mtl* Material )
{
	using namespace DatasmithMaxVRayMaterialsToUEPbrImpl;

	if (!Material)
	{
		return false;
	}

	FMaxVRayBlendMaterial VRayBlendMaterialProperties = ParseVRayBlendMaterialProperties(*Material);
	bool bAllMaterialsSupported = true;

	if (VRayBlendMaterialProperties.BaseMaterial)
	{
		FDatasmithMaxMaterialsToUEPbr* MaterialConverter = FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter(VRayBlendMaterialProperties.BaseMaterial);
		bAllMaterialsSupported &= MaterialConverter != nullptr;
	}

	for (int CoatIndex = 0; bAllMaterialsSupported && CoatIndex < FMaxVRayBlendMaterial::MaximumNumberOfCoat; ++CoatIndex)
	{
		const FMaxVRayBlendMaterial::FVRayCoatMaterialProperties& CoatedMaterial = VRayBlendMaterialProperties.CoatedMaterials[CoatIndex];

		if (CoatedMaterial.Material != nullptr && CoatedMaterial.MaterialBlendParameter.bEnabled)
		{
			//Only support if all the blended materials are UEPbr materials.
			FDatasmithMaxMaterialsToUEPbr* MaterialConverter = FDatasmithMaxMaterialsToUEPbrManager::GetMaterialConverter(CoatedMaterial.Material);
			bAllMaterialsSupported &= MaterialConverter != nullptr;
		}
	}

	return bAllMaterialsSupported;
}

void FDatasmithMaxVRayBlendMaterialToUEPbr::Convert( TSharedRef<IDatasmithScene> DatasmithScene, TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement, Mtl* Material, const TCHAR* AssetsPath )
{
	using namespace DatasmithMaxVRayMaterialsToUEPbrImpl;

	TSharedRef< IDatasmithUEPbrMaterialElement > PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial(GetMaterialName(Material));
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	FMaxVRayBlendMaterial VRayBlendMaterialProperties = ParseVRayBlendMaterialProperties( *Material );

	IDatasmithMaterialExpression* PreviousExpression = nullptr;

	//Exporting the base material.
	if (VRayBlendMaterialProperties.BaseMaterial) 
	{
		IDatasmithMaterialExpressionFunctionCall* BaseMaterialFunctionCall = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
		if (TSharedPtr<IDatasmithBaseMaterialElement> ExportedMaterial = FDatasmithMaxMatExport::ExportUniqueMaterial(DatasmithScene, VRayBlendMaterialProperties.BaseMaterial, AssetsPath))
		{
			BaseMaterialFunctionCall->SetFunctionPathName(ExportedMaterial->GetName());
		}
		PreviousExpression = BaseMaterialFunctionCall;
	}

	//Exporting the blended materials.
	for (int CoatIndex = 0; CoatIndex < FMaxVRayBlendMaterial::MaximumNumberOfCoat; ++CoatIndex)
	{
		const FMaxVRayBlendMaterial::FVRayCoatMaterialProperties& CoatedMaterial = VRayBlendMaterialProperties.CoatedMaterials[CoatIndex];

		if (CoatedMaterial.Material != nullptr && CoatedMaterial.MaterialBlendParameter.bEnabled)
		{
			TSharedPtr<IDatasmithBaseMaterialElement> ExportedCoatedMaterial = FDatasmithMaxMatExport::ExportUniqueMaterial(DatasmithScene, CoatedMaterial.Material, AssetsPath);
			if (PreviousExpression)
			{
				IDatasmithMaterialExpressionFunctionCall* BlendFunctionCall = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
				BlendFunctionCall->SetFunctionPathName(TEXT("/Engine/Functions/MaterialLayerFunctions/MatLayerBlend_Standard.MatLayerBlend_Standard"));
				PreviousExpression->ConnectExpression(*BlendFunctionCall->GetInput(0));
				PreviousExpression = BlendFunctionCall;

				IDatasmithMaterialExpressionFunctionCall* CoatedMaterialFunctionCall = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
				if (ExportedCoatedMaterial)
				{
					CoatedMaterialFunctionCall->SetFunctionPathName(ExportedCoatedMaterial->GetName());
				}
				CoatedMaterialFunctionCall->ConnectExpression(*BlendFunctionCall->GetInput(1));

				IDatasmithMaterialExpression* AlphaExpression = FDatasmithMaxTexmapToUEPbrUtils::MapOrValue(this, CoatedMaterial.MaterialBlendParameter, TEXT("MixAmount"),
					CoatedMaterial.MixColor, TOptional< float >());

				//AlphaExpression is nullptr only when there is no mask and the mask weight is ~100% so we add scalar 0 instead.
				if(!AlphaExpression)
				{
					IDatasmithMaterialExpressionScalar* WeightExpression = PbrMaterialElement->AddMaterialExpression< IDatasmithMaterialExpressionScalar >();
					WeightExpression->SetName(TEXT("MixAmount"));
					WeightExpression->GetScalar() = 0.f;
					AlphaExpression = WeightExpression;
				}

				AlphaExpression->ConnectExpression(*BlendFunctionCall->GetInput(2));
			}
			else
			{
				IDatasmithMaterialExpressionFunctionCall* CoatedMaterialFunctionCall = PbrMaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
				if (ExportedCoatedMaterial)
				{
					CoatedMaterialFunctionCall->SetFunctionPathName(ExportedCoatedMaterial->GetName());
				}
				PreviousExpression = CoatedMaterialFunctionCall;
			}
		}
	}

	PbrMaterialElement->SetUseMaterialAttributes(true);
	if (PreviousExpression)
	{
		PreviousExpression->ConnectExpression(PbrMaterialElement->GetMaterialAttributes());
	}
	MaterialElement = PbrMaterialElement;
}

bool FDatasmithMaxVRayLightMaterialToUEPbr::IsSupported(Mtl* Material)
{
	return true;
}

void FDatasmithMaxVRayLightMaterialToUEPbr::Convert(TSharedRef<IDatasmithScene> DatasmithScene,
	TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement, Mtl* Material, const TCHAR* AssetsPath)
{
	if ( !Material )
	{
		return;
	}

	TSharedRef<IDatasmithUEPbrMaterialElement> PbrMaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial(GetMaterialName(Material));
	FScopedConvertState ScopedConvertState(ConvertState);
	ConvertState.DatasmithScene = DatasmithScene;
	ConvertState.MaterialElement = PbrMaterialElement;
	ConvertState.AssetsPath = AssetsPath;

	DatasmithMaxVRayMaterialsToUEPbrImpl::FVRayLightMaterial MaterialProperties;
	MaterialProperties.Parse(*Material);

	Connect(PbrMaterialElement->GetEmissiveColor(),
		COMPOSE_OR_DEFAULT2(nullptr, Multiply,
			TextureOrColor(TEXT("Emissive Color"), MaterialProperties.EmitTexture, FDatasmithMaxMatHelper::MaxLinearColorToFLinearColor(MaterialProperties.EmitColor)),
			&Scalar(MaterialProperties.Multiplier))
	);

	if (IDatasmithMaterialExpression* OpacictyExpression = ConvertTexmap(MaterialProperties.ClipTexture))
	{
		PbrMaterialElement->SetBlendMode(/*EBlendMode::BLEND_Masked*/1); // Set blend mode when there's opacity mask
		Connect(PbrMaterialElement->GetOpacity(), OpacictyExpression);
	}

	PbrMaterialElement->SetShadingModel(EDatasmithShadingModel::Unlit);

	MaterialElement = PbrMaterialElement;
}
