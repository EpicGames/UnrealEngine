// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "IDatasmith3dsMaxUI.h"

#include "Containers/Queue.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"

// Slate
#include "Framework/Application/SWindowTitleBar.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/CoreStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/SListView.h"

#include "HAL/PlatformApplicationMisc.h"  // For ClipboardCopy

#include "DatasmithExporterManager.h"  // Using FDatasmithExporterManager::PushCommandIntoGameThread
#include "DatasmithUtils.h"


#define LOCTEXT_NAMESPACE "Datasmith3dsMaxUI"

namespace DatasmithMaxDirectLink
{
namespace Ui
{

struct FMessageData
{
	enum ECategory
	{
		Default,
		Info,
		Debug,
		Error,
		Warning,
		Completion
	};

	FString MessageText;
	ECategory Category;
};

class SMessagesWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagesWidget) {}
	SLATE_END_ARGS()

	virtual bool SupportsKeyboardFocus() const override
	{
		// Handle keys
		return true;
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		// Copy queued messages
		if (!MessagesQueue->IsEmpty())
		{
			TSharedPtr<FMessageData> Message;
			while (MessagesQueue->Dequeue(Message))
			{
				Messages.Add(Message);
			}
			MessagesView->RequestListRefresh();
			MessagesView->ScrollToBottom();
		}

		// Always call base Widget Tick
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}

	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
	{
		if (InKeyEvent.IsControlDown())
		{
			if (InKeyEvent.GetKey() == EKeys::C)
			{
				CopyLog();
			}
		}
		return FReply::Unhandled();
	}

	FReply OnCopyMessagesClicked()
	{
		CopyLog();
		return FReply::Handled();
	}

	FReply OnClearMessagesClicked()
	{
		ClearMessages();
		return FReply::Handled();
	}

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FMessageData> Item, const TSharedRef<STableViewBase>& Owner)
	{
		FSlateColor Color = FLinearColor::Gray;

		switch (Item->Category)
		{
		case FMessageData::Default: break;
		case FMessageData::Info: break;
		case FMessageData::Debug: break;
		case FMessageData::Error: 
			Color = FLinearColor::Red;
			break;
		case FMessageData::Warning: 
			Color = FLinearColor::Yellow;
			break;
		case FMessageData::Completion: 
			Color = FLinearColor::Green;
			break;
		default: ;
		}

		return SNew(STableRow<TSharedPtr<FMessageData> >, Owner)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Item->MessageText))
				.ColorAndOpacity(Color)
			]
		];

	}

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew( SVerticalBox )

			// Messages list view
			+ SVerticalBox::Slot()
			.Padding( 2.f )
			.VAlign( VAlign_Fill )
			[
				SAssignNew( MessagesView, SListView<TSharedPtr<FMessageData>> )
				.ListItemsSource( &Messages )
				.OnGenerateRow( this, &SMessagesWidget::OnGenerateRow )
				.SelectionMode(ESelectionMode::Multi)
			]

			// Buttons
			+ SVerticalBox::Slot()
			.Padding( 2.f )
			.AutoHeight()
			.VAlign( VAlign_Bottom )
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.Padding( 2.f )
				.AutoWidth()
				.HAlign( HAlign_Right )
				[
					SNew( SButton )
					.OnClicked( this, &SMessagesWidget::OnCopyMessagesClicked )
					.ToolTipText( LOCTEXT("CopyMessages_Tooltip", "Copy Messages To Clipboard") )
					[
						SNew( SVerticalBox )
						+ SVerticalBox::Slot()
						.VAlign( VAlign_Center )
						[
							SNew( STextBlock )
							.Text( LOCTEXT("CopyMessages_Label", "Copy") )
						]
					]
				]
				+ SHorizontalBox::Slot()
				.Padding( 2.f )
				.AutoWidth()
				.HAlign( HAlign_Right )
				[
					SNew( SButton )
					.OnClicked( this, &SMessagesWidget::OnClearMessagesClicked )
					.ToolTipText( LOCTEXT("ClearMessages_Tooltip", "Clear Messages") )
					[
						SNew( SVerticalBox )
						+ SVerticalBox::Slot()
						.VAlign( VAlign_Center )
						[
							SNew( STextBlock )
							.Text( LOCTEXT("ClearMessages_Label", "Clear") )
						]
					]
				]
			]

		];
	}

	void CopyLog() const
	{
		const TArray<TSharedPtr<FMessageData>> SelectedItems = MessagesView->GetSelectedItems();

		const TArray<TSharedPtr<FMessageData>>& Items = SelectedItems.IsEmpty() ? Messages: SelectedItems;

		FString Text;
		for( int32 Index = 0; Index < Items.Num(); ++Index )
		{
			Text += Items[Index]->MessageText;
			Text += LINE_TERMINATOR;
		}
	
		FPlatformApplicationMisc::ClipboardCopy(*Text);
	}

	void ClearMessages()
	{
		check(IsInGameThread());
		Messages.Empty();
		MessagesView->RequestListRefresh();
	}

	TSharedPtr<SListView<TSharedPtr<FMessageData>>> MessagesView;

	TArray<TSharedPtr<FMessageData>> Messages;
	TQueue<TSharedPtr<FMessageData>>* MessagesQueue;
};



class FMessagesWindow : public IMessagesWindow
{
public:
	FText GetWindowTitle()
	{
		FFormatNamedArguments WindowTitleArgs;
		WindowTitleArgs.Add(TEXT("DatasmithSDKVersion"), FText::FromString(FDatasmithUtils::GetEnterpriseVersionAsString(true)));
		return FText::Format(LOCTEXT("DatasmithMessagesWindowTitle", "Datasmith Messages v.{DatasmithSDKVersion}"), WindowTitleArgs);
	}

	virtual void OpenWindow() override
	{
		FSimpleDelegate RunOnUIThread;

		RunOnUIThread.BindLambda( [this]()
		{
			if ( TSharedPtr<SWindow> SlateWindow = SlateWindowPtr.Pin() )
			{
				// Cancel the minimize
				SlateWindow->BringToFront();

				SlateWindow->HACK_ForceToFront();
				SlateWindow->FlashWindow();
			}
			else
			{
				TSharedRef<SWindow> Window = SNew( SWindow )
					.CreateTitleBar( false )
					.ClientSize( FVector2D( 640, 480 ) )
					.AutoCenter( EAutoCenter::PrimaryWorkArea )
					.SizingRule( ESizingRule::UserSized )
					.FocusWhenFirstShown( true )
					.Title( GetWindowTitle() );

				TSharedRef<SWindowTitleBar> WindowTitleBar = SNew( SWindowTitleBar, Window, nullptr, GetWindowTitleAlignement() )
					.Visibility( EVisibility::Visible )
					.ShowAppIcon( false );

				Window->SetTitleBar( WindowTitleBar );

				Window->SetContent(
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							WindowTitleBar
						]
						+ SVerticalBox::Slot()
						.FillHeight( 1.f )
						[
							SNew( SBorder )
							.BorderImage( FCoreStyle::Get().GetBrush( "ToolPanel.GroupBorder" ) )
							[
								SAssignNew(MessagesWidget, SMessagesWidget)
							]
						]
					);

				FSlateApplication::Get().AddWindow( Window, true );
				Window->HACK_ForceToFront();
				MessagesWidget->MessagesQueue = &MessagesQueue;
				SlateWindowPtr = Window;
			}
		});

#if IS_PROGRAM
		FDatasmithExporterManager::PushCommandIntoGameThread(MoveTemp(RunOnUIThread));
#else
		RunOnUIThread.Execute();
#endif
	}

	TQueue<TSharedPtr<FMessageData>> MessagesQueue;
	void AddMessage(const FString& Message, FMessageData::ECategory Category)
	{
		MessagesQueue.Enqueue(MakeShareable(new FMessageData{Message, Category}));
	}

	virtual void AddError(const FString& Message) override
	{
		AddMessage(Message, FMessageData::Error);
	}

	virtual void AddWarning(const FString& Message) override
	{
		AddMessage(Message, FMessageData::Warning);
	}

	virtual void AddInfo(const FString& Message) override
	{
		AddMessage(Message, FMessageData::Info);
	}

	virtual void AddCompletion(const FString& Message) override
	{
		AddMessage(Message, FMessageData::Completion);
	}

	virtual void ClearMessages() override
	{
		MessagesWidget->ClearMessages();
	}

	EHorizontalAlignment GetWindowTitleAlignement()
	{
		EWindowTitleAlignment::Type TitleAlignment = FSlateApplicationBase::Get().GetPlatformApplication()->GetWindowTitleAlignment();
		EHorizontalAlignment TitleContentAlignment;

		if ( TitleAlignment == EWindowTitleAlignment::Left )
		{
			TitleContentAlignment = HAlign_Left;
		}
		else if ( TitleAlignment == EWindowTitleAlignment::Center )
		{
			TitleContentAlignment = HAlign_Center;
		}
		else
		{
			TitleContentAlignment = HAlign_Right;
		}
		return TitleContentAlignment;
	}

	TWeakPtr<SWindow> SlateWindowPtr;

	TSharedPtr<SMessagesWidget> MessagesWidget;
};

IMessagesWindow* CreateMessagesWindow()
{
	static FMessagesWindow MessagesWindow;
	return &MessagesWindow;
}

}
}
#undef LOCTEXT_NAMESPACE
