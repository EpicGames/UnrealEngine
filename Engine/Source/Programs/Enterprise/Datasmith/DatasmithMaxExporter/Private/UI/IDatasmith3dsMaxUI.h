// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace DatasmithMaxDirectLink
{
namespace Ui
{

class IMessagesWindow
{
public:

	virtual void OpenWindow() = 0;

	virtual void AddError(const FString& Message) = 0;
	virtual void AddWarning(const FString& Message) = 0;
	virtual void AddInfo(const FString& Message) = 0;
	virtual void AddCompletion(const FString& Message) = 0;
	virtual void ClearMessages() = 0;

	virtual ~IMessagesWindow() = default;
};

IMessagesWindow* CreateMessagesWindow();

}
}
