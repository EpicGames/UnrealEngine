// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

enum class EVRayIESLightShapes
{
	Point,
	Rectangle,
	Circle,
	Sphere,
	VerticalCylinder
};

enum class EVRayLightParamBlocks : short
{
	Params
};

// Based on VRay SDK pb2enum.h
enum class EVrayLightsParams : short
{
	LightOn,
	Type,
	Size0,
	Size1,
	Size2,
	MinSamples,
	Samples,
	Color,
	Multiplier,
	DoubleSided,
	Transparent,
	IgnoreNormals,
	NormalizeColor,
	NoDecay,
	StoreWithIrradMap,
	DegradeDepth,
	LightPortal,
	SmoothShadows,
	Texmap,
	TexmapOn,
	TexmapResolution,
	DomeEmitRadius,
	DomeTargetRadius,
	ShadowBias,
	DomeVisibleOriginal,
	AffectDiffuse,
	AffectSpecular,
	DomeSpherical,
	CutoffThresh,
	UseToneOpNotUsed,
	AffectReflections,
	SimplePortal,
	CastShadows,
	TexmapAdaptiveness,
	MeshSource,
	MeshFlip,
	LightDistribution,
	MeshMoveLight,
	Temperature,	
	ColorMode,
	DomeRayDistance,
	DomeRayDistanceMode,
	UseLightscape,
	Targeted,
	TargetDistance,
	RectPreview,
	UseMis,
	WireColor,
	WireColorOn,
	TexmapLocktodome,
	DomeAffectAlpha,
	TexmapPreview,
	TextOn,
	AffectDiffuseAmount,
	AffectSpecularAmount
};
