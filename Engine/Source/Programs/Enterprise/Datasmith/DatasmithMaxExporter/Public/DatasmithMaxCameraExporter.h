// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"


#include "DatasmithMaxHelper.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "maxtypes.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

class CameraObject;
class IDatasmithCameraActorElement;
class IDatasmithPostProcessVolumeElement;
class INode;
class ToneOperator;

/**
 * Export both the cameras and the tone operator as they share a lot of parameters.
 */
class FDatasmithMaxCameraExporter
{
public:
	static bool ExportCamera( TimeValue CurrentTime, INode& Node, TSharedRef< IDatasmithCameraActorElement > CameraElement );
	static bool ExportToneOperator( ToneOperator& ToneOp, TSharedRef< IDatasmithPostProcessVolumeElement > PostProcessVolumeElement );

private:
	static bool ExportPhysicalCamera( CameraObject& Camera, TSharedRef< IDatasmithCameraActorElement > CameraElement );
	static bool ExportVRayPhysicalCamera( CameraObject& Camera, TSharedRef< IDatasmithCameraActorElement > CameraElement );

	static bool ExportPhysicalExposureControl( ToneOperator& ToneOp, TSharedRef< IDatasmithPostProcessVolumeElement > PostProcessVolumeElement );
	static bool ExportVRayExposureControl( ToneOperator& ToneOp, TSharedRef< IDatasmithPostProcessVolumeElement > PostProcessVolumeElement );
};
