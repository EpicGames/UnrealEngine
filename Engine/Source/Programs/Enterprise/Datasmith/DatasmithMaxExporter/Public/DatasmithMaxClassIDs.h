// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#define DATASMITHUNREALEXPORTER_CLASS_ID			Class_ID(0x49ad677c, 0x3c9edc59)

//MAXSCRIPT MODIFIER
#define DATASMITHUNREALATTRIBUTEMODIFIER_CLASS_ID	Class_ID(0x60178e47, 0x2feafe95)


//MAPS CLASS IDS
#define  RBITMAPCLASS				Class_ID(0x240, 0x0)
#define  AUTODESKBITMAPCLASS		Class_ID(0x7ed417e4, 0x273deef8)
#define  RGBMULTIPLYCLASS			Class_ID(0x290, 0x0)
#define  RGBTINTCLASS				Class_ID(0x224, 0x0)
#define  OUTPUTMAPCLASS				Class_ID(0x2b0, 0x0)
#define  THEABITMAPCLASS			Class_ID(0x30260307, 0x35535b85)
#define  REGULARNORMALCLASS			Class_ID(0x243e22C6, 0x63f6a014)
#define  CORONANORMALCLASS			Class_ID(0x2870ceaa, 0xcc18437c)
#define  COLORCORRECTCLASS			Class_ID(0x2d0, 0x0)
#define  FALLOFFCLASS				Class_ID(0x6ec3730c, 0x0)
#define  MIXCLASS					Class_ID(0x230, 0x0)
#define  NOISECLASS					Class_ID(0x234, 0x0)
#define  GRADIENTCLASS				Class_ID(0x270, 0x0)
#define  GRADIENTRAMPCLASS			Class_ID(0x1dec5b86, 0x43383a51)
#define  CHECKERCLASS				Class_ID(0x00000200, 0x00000000)
#define  CELLULARCLASS				Class_ID(0xc90017a5, 0x111940bb)
#define  COMPOSITETEXCLASS			Class_ID(0x280, 0x0)
#define  CORONABITMAPCLASS			Class_ID(0xabba5784, 0x65484244)
#define  CORONACOLORCLASS			Class_ID(0x68426dca, 0x372362d4)
#define  CORONAMIXCLASS				Class_ID(0x6847286f, 0x3fffaab4)
#define  CORONAMULTITEXCLASS		Class_ID(0xbaba3487, 0x30782624)
#define  CORONAAOCLASS				Class_ID(0x26ea4249, 0x404b4b59)
#define  CORONAPHYSICALSKYCLASS		Class_ID(0x59577962, 0x4cab7ac1)
#define  MRPHYSICALSKYCLASS			Class_ID(0x628e269e, 0x72386abf)
#define  MRPHYSICALSKYBCLASS		Class_ID(0x223349b4, 0x562a7d8b)
#define  THIRDPARTYMULTITEXCLASS	Class_ID(0x55c6d763, 0xc637bb2e)
#define  TILESMAPCLASS				Class_ID(0x64035fb9, 0x69664cdc)
#define  COLORMAPCLASS				Class_ID(0x139f22c6, 0x13f6a914)


#define  VRAYNORMALCLASS			Class_ID(0x71fa6e51, 0x72057c2f)
#define  VRAYCOLORCLASS				Class_ID(0x58f82b74, 0x73b75d7f)
#define  VRAYDIRTCLASS				Class_ID(0x2f567899, 0x90d5ea4)
#define  VRAYPHYSICALSKYCLASS		Class_ID(0x48480b4e, 0x62ff5a90)
#define  VRAYCOLOR2BUMP				Class_ID(0x31e15d1d, 0x7d7e2810)
#define  VRAYHDRICLASS				Class_ID(0x6769144b, 0x2c1017d)

#define FORESTCOLORCLASS			Class_ID(0x602f7917, 0x06354410)

//MATERIAL CLASS IDS
#define  MULTIMAT_CLASS_ID		Class_ID(0x200, 0x0)
#define  THEAMATERIALCLASS      Class_ID(0x11dcacdd, 0x446400f0)	
#define  THEAMATOLDCLASS		Class_ID(0x372e9654, 0x1bdffef0)	
#define  THEARANDOMCLASS        Class_ID(0x544ff1de, 0x21ab5ea6)	
#define  THEABASICCLASS			Class_ID(0x587ff1de, 0x21ec5ea6)
#define  THEAGLOSSYCLASS		Class_ID(0x39c95ac3, 0x1db33879)
#define  THEASSSCLASS			Class_ID(0x65572bd9, 0x7137c4b)
#define  THEAFILMCLASS			Class_ID(0x51a048bf, 0x3420ada8)
#define  THEACOATINGCLASS		Class_ID(0x2ca1f05e, 0x224ea918)
#define  STANDARDMATCLASS		Class_ID(0x2,0x0)
#define  SHELLCLASS				Class_ID(0x255,0x0)
#define  BLENDMATCLASS			Class_ID(0x250, 0x0)
#define  XREFMATCLASS			Class_ID(0x272c0d4b,0x432a414b)
#define  ARCHDESIGNMATCLASS		Class_ID(0x70b05735, 0x4a163654)
#define  CORONAMATCLASS			Class_ID(0x70be6506,0x448931dd)
#define  CORONALAYERMATCLASS	Class_ID(0x65486584,0x8425554e)
#define  CORONALIGHTMATCLASS	Class_ID(0x53b7717a,0x500d5093)
#define  CORONAPHYSICALMTLCLASS	Class_ID(0x6912ab89,0x87151720) 
#define	 PHYSICALMATCLASS		Class_ID(0x3d6b1cec,0xdeadc001)



#define  VRAYMATCLASS			Class_ID(0x37bf3f2f, 0x7034695c)
#define  VRAYBLENDMATCLASS		Class_ID(0x3da0041b, 0xf436e31)
#define  VRAYLIGHTMATCLASS		Class_ID(0x7ccf263e, 0x3f5b39b9)
#define  VRAYFASTSSSCLASS		Class_ID(0x3f2c5faa, 0x750008bc)
#define  VRAY2SIDEDMATCLASS		Class_ID(0x6066686a, 0x11731b4b)
#define  VRAYWRAPPERMATCLASS	Class_ID(0x409517df, 0x3a5f44b6)
#define  VRAYOVERRIDEMATCLASS	Class_ID(0x15d20e6b, 0x54e217eb)

//LIGHTS
#define  THEAOMNICLASS			Class_ID(0x1a486759, 0x69269430)
#define  THEASPOTCLASS			Class_ID(0x6fe6bc2a, 0x4c04b705)
#define  THEAIESCLASS			Class_ID(0x29b64382, 0x31882eb0)
#define  THEAPLANECLASS			Class_ID(0x7c40f14e, 0x172d9b94)
#define  REGULARSPOTACLASS		Class_ID(0x1012, 0x0)
#define  REGULARSPOTBCLASS		Class_ID(0x1014, 0x0)
#define  REGULARDIRECTACLASS	Class_ID(0x1013, 0x0)
#define  REGULARDIRECTBCLASS	Class_ID(0x1015, 0x0)
#define  REGULAROMNICLASS		Class_ID(0x1011, 0x0)
#define  PHOTOPLANEACLASS		Class_ID(0x36507d92, 0x105a1a47)
#define  PHOTOPLANEBCLASS		Class_ID(0x71794f9d, 0x70ae52f2)
#define  VRAYLIGHTIESCLASS		Class_ID(0x2EAF2F07, 0x7EC44F31)
#define  VRAYLIGHTCLASS			Class_ID(0x3C5575A1, 0x5FD602DF)
#define  VRAYSUNCLASS			Class_ID(0x732c0383, 0x7d993776)
#define  SUNLIGHTACLASS			Class_ID(0x23fe7f4f, 0x6cf2fe9)
#define  SUNLIGHTBCLASS			Class_ID(0x71503bbb, 0x25e21124)
#define  SUNPOSITIONERCLASS		Class_ID(0x27fe329c, 0x716f5b25)


#define  CORONASUNCLASS			Class_ID(0x7c3a3c80, 0x294c5879)
#define  CORONASUNCLASSB		Class_ID(0x7c3a3c80, 0x9cf576f)
#define  CORONALIGHTCLASS		Class_ID(0x423045e5, 0x294c5879)
#define  ARNOLDLIGHTCLASS		Class_ID(0x6705f00d, 0xca131d05)
#define  MRPORTALLIGHTCLASS		Class_ID(0x335c655c, 0x382151b1)

#define  DAYLIGHTASSEMBLYCLASS  Class_ID(0x4a1e6deb, 0x31c77d57)
#define  SKYLIGHTCLASS			Class_ID(0x7bf61478, 0x522e4705)
#define  IESSKYLIGHTCLASS		Class_ID(0x4b241b11, 0x64e8527d)
#define  MRSKYLIGHTCLASS		Class_ID(0x26ee3350, 0x799b5b56)


//SPECIAL CASE GEOMETRIES
#define VRAYPROXY_CLASS_ID		Class_ID(0x6CF53873, 0x1FF85498)
#define VRAYPLANE_CLASS_ID		Class_ID(0x628140F6, 0x3BDB0E0C)
#define VRAYFUR_CLASS_ID		Class_ID(0x527763C6, 0x26B35193)
#define VRAYSPHERE_CLASS_ID		Class_ID(0xF7B1CAC,  0x1DF87067)
#define MENTALPROXY_CLASS_ID	Class_ID(0x2E013BD5, 0x13C00AFB)
#define ITOOFOREST_CLASS_ID		Class_ID(0x79fE41C2, 0x1D7B4FB1)
#define RAILCLONE_CLASS_ID		Class_ID(0x39712DEF, 0x10A72959)
#define PFLOWEVENT_CLASS_ID		Class_ID(0X74F93B02, 0x1EB34300)
#define PFLOWVIEW_CLASS_ID		Class_ID(0X74F93B07, 0x1EB34300)
#define THEAPROXY_CLASS_ID		Class_ID(0x600846F6, 0x1D8C7F97)
#define POPULATE_CLASS_ID		Class_ID(0x418219c7, 0x180f6d1e)
#define BODYOBJECT_CLASS_ID		Class_ID(0x25292BC6, 0x17f40588)


//EXPOSURE CONTROLS
#define PHYSICAL_CRTL_CLASSID	Class_ID(0x27964a9b, 0x2ea5376a)
#define VRAY_CRTL_CLASSID		Class_ID(0x565a20b6, 0x10480513) 
#define MR_CRTL_CLASSID			Class_ID(0x73317331, 0x49154663) 

//CAMERAS
#define PHYSICALCAMERA_CLASS	Class_ID(0x46697218, 0x28e8008d) // MaxSDK::IPhysicalCamera::GetClassID() isn't supported in Max 2016
#define VRAY_PHYSICALCAMERA_CLASS Class_ID(0x405e3fee, 0x4f6513be)
