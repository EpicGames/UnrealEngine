// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithMaxExporterDefines.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "Animatable.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

class IDatasmithActorElement;
class IDatasmithMeshActorElement;
class IDatasmithMeshElement;

namespace DatasmithMaxDirectLink
{

class ISceneTracker;
class FNodeTracker;

// Identifies data for Max node converted to Datasmith
class FNodeConverted: FNoncopyable
{
public:
	TSharedPtr<IDatasmithActorElement> DatasmithActorElement;
	TSharedPtr<IDatasmithMeshActorElement> DatasmithMeshActor;
};

class FMeshConverted
{
public:
	TSharedPtr<IDatasmithMeshElement> DatasmithMeshElement;

	[[nodiscard]] TSharedPtr<IDatasmithMeshElement> GetDatasmithMeshElement() const
	{
		return DatasmithMeshElement;
	}

	TSet<uint16> SupportedChannels;
	TMap<int32, int32> UVChannelsMap;

	void ReleaseMeshConverted()
	{
		DatasmithMeshElement.Reset();
		SupportedChannels.Reset();
		UVChannelsMap.Reset();
	}
};

class FNodeConverter: FNoncopyable
{
public:
	enum EType
	{
		Unknown,
		MeshNode,
		HismNode,
		LightNode,
		CameraNode,
		HelperNode,
	};

	FNodeConverter(EType InConverterType=EType::Unknown): ConverterType(InConverterType)
	{
	}

	virtual ~FNodeConverter(){}

	virtual void Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) = 0;
	virtual void ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) = 0;
	virtual void RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) = 0;

	const EType ConverterType;
};

class FMeshNodeConverter: public FNodeConverter
{
public:
	FMeshNodeConverter(): FNodeConverter(EType::MeshNode)
	{
	}
	virtual void Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	virtual void ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	virtual void RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;

	AnimHandle InstanceHandle = 0; // todo: rename - this is handle for object this node is instance of
	bool bMaterialsAssignedToStaticMesh; // This node will used materials assigned to static mesh, otherwise actor's override
};

class FHismNodeConverter: public FNodeConverter
{
public:
	FHismNodeConverter(): FNodeConverter(EType::HismNode)
	{
	}
	virtual void RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	
	TArray<FMeshConverted> Meshes;
};

class FRailCloneNodeConverter: public FHismNodeConverter
{
public:
	virtual void Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	virtual void ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
};

class FForestNodeConverter: public FHismNodeConverter
{
public:
	virtual void Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	virtual void ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
};

class FHelperNodeConverter: public FNodeConverter
{
public:
	FHelperNodeConverter(): FNodeConverter(EType::HelperNode)
	{
	}

	virtual void Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	virtual void ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	virtual void RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
};

class FLightNodeConverter: public FNodeConverter
{
public:
	FLightNodeConverter(): FNodeConverter(EType::LightNode)
	{
	}

	virtual void Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	virtual void ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	virtual void RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;

	void ApplyIesProfile(const TCHAR* InIesFilePath);
	const TCHAR* GetIesProfile();
	bool IsIesProfileValid();

private:
	FString IesFilePath;

};

class FCameraNodeConverter: public FNodeConverter
{
public:
	FCameraNodeConverter(): FNodeConverter(EType::CameraNode)
	{
	}
	virtual void Parse(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	virtual void ConvertToDatasmith(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
	virtual void RemoveFromTracked(ISceneTracker& SceneTracker, FNodeTracker& NodeTracker) override;
};

}

