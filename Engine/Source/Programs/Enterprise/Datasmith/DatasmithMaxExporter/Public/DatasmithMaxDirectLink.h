// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithMaxExporterDefines.h"

#include "DatasmithMaxConverters.h"
#include "DatasmithMaxGeomUtils.h"

#include "IDatasmithSceneElements.h"
#include "DatasmithMesh.h"
#include "DatasmithUtils.h"

#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDatasmithMaxExporter, Log, All);

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "impexp.h"
	#include "max.h"

	#include "decomp.h" // for matrix
	#include "ISceneEventManager.h"

MAX_INCLUDES_END



// Log all messages
// #define LOG_DEBUG_HEAVY_ENABLE 1
// #define LOG_DEBUG_ENABLE 1
// #define CANCEL_DEBUG_ENABLE 1



void AssignMeshMaterials(TSharedPtr<IDatasmithMeshElement>&MeshElement, Mtl * Material, const TSet<uint16>&SupportedChannels);

class FDatasmithMaxStaticMeshAttributes;

namespace DatasmithMaxDirectLink
{

class ISceneTracker;
class FLayerTracker;
class FMaterialsCollectionTracker;


/** Global export options, stored in preferences */
class IPersistentExportOptions
{
public:

	/** Whether to export animation or not */
	virtual void SetAnimatedTransforms(bool) = 0;
	virtual bool GetAnimatedTransforms() = 0;

	/** Whether to output export statistics to listener/log */
	virtual void SetStatSync(bool) = 0;
	virtual bool GetStatSync() = 0;

	virtual void SetTextureResolution(int32 Bool) = 0;
	virtual int32 GetTextureResolution() = 0;

	virtual void SetXRefScenes(bool) = 0;
	virtual bool GetXRefScenes() = 0;

};

/** Main class for export/change tracking */
class IExporter
{
	public:
	virtual ~IExporter() {}

	virtual void Shutdown() = 0;

	virtual void SetOutputPath(const TCHAR* Path) = 0;
	virtual void SetName(const TCHAR* Name) = 0;

	virtual ISceneTracker& GetSceneTracker() = 0;

	virtual void InitializeScene() = 0;

	/** Scene update */
	virtual bool UpdateScene(bool bQuiet) = 0;

	virtual void ResetSceneTracking() = 0;

	/** ChangeTracking */
	virtual void StartSceneChangeTracking() = 0;

	/** DirectLink */
	virtual void InitializeDirectLinkForScene() = 0;
	virtual void UpdateDirectLinkScene() = 0;
	virtual bool ToggleAutoSync() = 0;
	virtual bool IsAutoSyncEnabled() = 0;
	virtual void SetAutoSyncDelay(float Seconds) = 0;
	virtual void SetAutoSyncIdleDelay(float Seconds) = 0;

	virtual void PerformSync(bool bQuiet) = 0;
};

/** Create exporter with ability for DirectLink change tracking */
bool CreateExporter(bool bEnableUI, const TCHAR* EnginePath); 
IExporter* GetExporter();
void ShutdownExporter();
bool Export(const TCHAR* Name, const TCHAR* OutputPath, bool bQuiet, bool bSelected);
void ShutdownScripts();


IPersistentExportOptions& GetPersistentExportOptions();

// Max nodes are matched by these keys
typedef NodeEventNamespace::NodeKey FNodeKey;
typedef MtlBase* FMaterialKey;


struct FXRefScene
{
	INode* Tree = nullptr;
	int32 XRefFileIndex = -1;

	operator bool(){ return XRefFileIndex != -1; }
};

/** Incapsulating time slider value to distinguish usage of TimeValue specifically for point on timeslider where we syncing */
struct FSyncPoint
{
	TimeValue Time;
};

/** Incapsulating time interval value to distinguish usage of Interval specifically for validity of synced entity */
struct FValidity
{
	explicit FValidity()
	{
		Invalidate(); // Invalid by default(to essentially force update(validate explicitly)
	}

	bool IsInvalidated() const
	{
		return bIsInvalidated;
	}

	void SetValid()
	{
		bIsInvalidated = false;
	}

	void Invalidate()
	{
		bIsInvalidated = true;
		ValidityInterval.SetEmpty();
	}

	bool IsValidForSyncPoint(const FSyncPoint& State) const
	{
		return ValidityInterval.InInterval(State.Time) != 0;
	}

	/** Maximize validity interval before updating invalidated */
	void ResetValidityInterval()
	{
		check(bIsInvalidated);
		ValidityInterval.SetInfinite();
	}

	/** Intersect current validity interval with the param*/
	void NarrowValidityToInterval(const Interval& InValidityInterval)
	{
		ValidityInterval &= InValidityInterval;
	}

	void NarrowValidityToInterval(const FValidity& Validity)
	{
		NarrowValidityToInterval(Validity.ValidityInterval);
	}

	/**
	This Validity contains another validity interval fully.
	Used to determine if this validity doesn't need to be updated
	@return True is this validity doesn't need to be updated
	*/
	bool Overlaps(const FValidity& Validity) const
	{
		return ValidityInterval.IsSubset(Validity.ValidityInterval);
	}

private:
	Interval ValidityInterval;
	// todo: this flag indicates that ValidityInterval is determined to be recalculated
	//  this is done in order to distinguish from Empty Interval that might happen somehow when updated
	//  Until ValidityInterval is recalculated bIsInvalidated stays - this supports cancelling Update
	//  at any point. So on the next Update nodes with bIsInvalidated are updated(again, if their update wasn't finished)
	bool bIsInvalidated; 
};

/**
NodeTracker - associated with INode, plus validity
Converter - holds convertion data for SPECIFIC node object type, e.g. geometry, railclone, light, dummy
Converted - holds Datasmith converted data for node
*/

/** Identifies Max node to track its changes, including tracked dependencies */
class FNodeTracker: FNoncopyable
{
public:
	explicit FNodeTracker(FNodeKey InNodeKey, INode* InNode) : NodeKey(InNodeKey), Node(InNode) {}

	void SetXRefIndex(FXRefScene InXRefScene)
	{
		XRefScene = InXRefScene;
	}

	INode* GetXRefParent()
	{
		INode* Result = XRefScene ? XRefScene.Tree->GetXRefParent(XRefScene.XRefFileIndex): nullptr;
		return Result;
	}

	FNodeConverted& CreateConverted()
	{
		check(!Converted);
		Converted.Reset(new FNodeConverted);
		return *Converted;
	}

	FNodeConverted& GetConverted()
	{
		check(Converted);
		return *Converted;
	}

	bool HasConverted()
	{
		return Converted.IsValid();
	}

	void ReleaseConverted()
	{
		Converted.Reset();
	}

	template<typename ConverterType>
	ConverterType& CreateConverter()
	{
		check(!Converter);
		ConverterType* Result = new ConverterType;
		Converter.Reset(Result);
		return *Result;
	}

	FNodeConverter& GetConverter()
	{
		check(Converter);
		return *Converter;
	}

	bool HasConverter()
	{
		return Converter.IsValid();
	}

	FNodeConverter::EType GetConverterType()
	{
		if (Converter.IsValid())
		{
			return Converter->ConverterType;
		}
		return FNodeConverter::Unknown;
	}

	void ReleaseConverter()
	{
		Converter.Reset();
	}

	/** Node entity identification data */
	FNodeKey NodeKey;
	INode* const Node;

	/**
	Keep root node and xref index when this node is a direct child of an XRef scene
	This is needed to retrieve parent node(e.g. when updated)
	Keeping parent node itself doesn't work - it can change and the only way to get it when it's changed is to call INode::GetXRefParent
	*/
	FXRefScene XRefScene;
	FString Name;

	/** Node validity */
	bool bParsed = false;
	bool bDeleted = false;
	FValidity Validity;
	FValidity SubtreeValidity;

	/** Other related tracked entities */
	FNodeTracker* Collision = nullptr;
	FLayerTracker* Layer = nullptr;
	TSet<class FMaterialTracker*> MaterialTrackers; 

	/** Node conversion state */

	/** Datasmith element that this Node is converted to */
	TUniquePtr<FNodeConverted> Converted = nullptr;
	/** Converter for specific Max object type */
	TUniquePtr<FNodeConverter> Converter = nullptr; 
};

class FMeshConverterSource
{
public:
	/**
	Node this mesh instantiates. When this is a 'regular' node it's  just instantiates mesh for Params
	but it's possible that this Node wants mesh to be bounding-box(when DatasmithAttributes spicifies it)
	*/
	INode* Node;

	/** Suggested Mesh name */
	FString MeshName;

	/*Extracted render mesh*/
	GeomUtils::FRenderMeshForConversion RenderMesh;

	/** Whether to join all materials id into single material slot for render mesh (used when a geometry doesn't have a multimaterial assigned) */
	bool bConsolidateMaterialIds; 

	GeomUtils::FRenderMeshForConversion CollisionMesh;

	Interval GetValidityInterval()
	{
		return RenderMesh.GetValidityInterval() & CollisionMesh.GetValidityInterval();
	}

	// Whether this will make a mesh
	bool IsValid()
	{
		return RenderMesh.IsValid();
	}
};



struct FSceneUpdateStats
{
	void Reset()
	{
		*this = FSceneUpdateStats();
	}

	// Explicitly initialize every stat
	int32 ParseSceneXRefFileEncountered = 0;
	int32 ParseSceneXRefFileDisabled = 0;
	int32 ParseSceneXRefFileMissing = 0;
	int32 ParseSceneXRefFileToParse = 0;
	int32 ParseNodeNodesEncountered = 0;
	int32 RemoveDeletedNodesNodes = 0;
	int32 RefreshCollisionsChangedNodes = 0;
	int32 UpdateNodeNodesUpdated = 0;
	int32 UpdateNodeSkippedAsCollisionNode = 0;
	int32 UpdateNodeSkippedAsHiddenNode = 0;
	int32 UpdateNodeSkippedAsUnselected = 0;
	int32 UpdateNodeGeomObjEncontered = 0;
	int32 UpdateNodeHelpersEncontered = 0;
	int32 UpdateNodeCamerasEncontered = 0;
	int32 UpdateNodeLightsEncontered = 0;
	int32 UpdateNodeLightsSkippedAsUnknown = 0;
	int32 UpdateNodeGeomObjSkippedAsNonRenderable = 0;
	int32 UpdateNodeGeomObjConverted = 0;
	int32 ReparentActorsSkippedWithoutDatasmithActor = 0;
	int32 ReparentActorsAttached = 0;
	int32 ReparentActorsAttachedToRoot = 0;
	int32 ProcessInvalidatedMaterialsInvalidated = 0;
	int32 ProcessInvalidatedMaterialsActualToUpdate = 0;
	int32 UpdateMaterialsTotal = 0;
	int32 UpdateMaterialsSkippedAsAlreadyConverted =0;
	int32 UpdateMaterialsConverted = 0;
	int32 UpdateTexturesTotal = 0;
	int32 CheckTimeSliderTotalChecks = 0;
	int32 CheckTimeSliderSkippedAsAlreadyInvalidated = 0;
	int32 CheckTimeSliderSkippedAsSubtreeValid = 0;
	int32 CheckTimeSliderInvalidated = 0;
	int32 ConvertNodesConverted = 0;
	int32 UpdateInstancesGeometryUpdated = 0;
	int32 UpdateTexturesBaked = 0;
	int64 UpdateTexturesBakedPixels = 0;
};

#define SCENE_UPDATE_STAT_INC(Category, Name) {Stats.Category##Name++;}
#define SCENE_UPDATE_STAT_SET(Category, Name, Value) Stats.Category##Name = Value
#define SCENE_UPDATE_STAT_GET(Category, Name) Stats.Category##Name


/**
Modifies Datasmith scene in responce to change notification calls
Subscription to various Max notification systems is done separately, see  FNotifications
*/
class ISceneTracker
{
public:
	virtual ~ISceneTracker(){}

	/** Change notifications */
	virtual void NodeAdded(INode* Node) = 0;
	virtual void NodeDeleted(INode* Node) = 0;
	virtual void NodeGeometryChanged(INode* Node) = 0;
	virtual void NodeHideChanged(INode* Node) = 0;
	virtual void NodeNameChanged(FNodeKey NodeKey) = 0;
	virtual void NodePropertiesChanged(INode* Node) = 0;
	virtual void NodeLinkChanged(FNodeKey NodeKey) = 0;
	virtual void NodeTransformChanged(INode* Node) = 0;
	virtual void NodeMaterialAssignmentChanged(FNodeKey NodeKey) = 0;
	virtual void NodeMaterialAssignmentChanged(INode* Node) = 0;
	virtual void NodeMaterialGraphModified(FNodeKey NodeKey) = 0;
	virtual void NodeMaterialGraphModified(INode* NodeKey) = 0;

	virtual void MaterialGraphModified(Mtl* Material) = 0;

	virtual void HideByCategoryChanged() = 0;

	virtual bool IsUpdateInProgress() = 0;

	/** Scene modification*/
	virtual void AddMeshElement(TSharedPtr<IDatasmithMeshElement>& Mesh, FDatasmithMesh& DatasmithMesh, FDatasmithMesh* CollisionMesh) = 0;
	virtual void ReleaseMeshElement(FMeshConverted& Converted) = 0;
	virtual void SetupActor(FNodeTracker& NodeTracker) = 0;
	virtual void SetupDatasmithHISMForNode(FNodeTracker& NodeTracker, FMeshConverterSource& MeshSource, Mtl* Material, int32 MeshIndex, const TArray<Matrix3>& Transforms) = 0;
	virtual void RemoveMaterial(const TSharedPtr<IDatasmithBaseMaterialElement>& DatasmithMaterial) = 0;
	virtual void RemoveTexture(const TSharedPtr<IDatasmithTextureElement>&) = 0;
	virtual void NodeXRefMerged(INode* Node) = 0;
	virtual void RemapConvertedMaterialUVChannels(Mtl* ActualMaterial, const TSharedPtr<IDatasmithBaseMaterialElement>& DatasmithMaterial) = 0;
	
	/** Sync/Update */
	FSyncPoint CurrentSyncPoint;

	virtual void AddGeometryNodeInstance(FNodeTracker& NodeTracker, FMeshNodeConverter& MeshConverter, Object* Obj) = 0;
	virtual void RemoveGeometryNodeInstance(FNodeTracker& NodeTracker) = 0;
	virtual void ConvertGeometryNodeToDatasmith(FNodeTracker& NodeTracker, FMeshNodeConverter& MeshConverter) = 0;

	virtual void UnregisterNodeForMaterial(FNodeTracker& NodeTracker) = 0;

	virtual const TCHAR* AcquireIesTexture(const FString& IesFilePath) = 0;
	virtual void ReleaseIesTexture(const FString& IesFilePath) = 0;

	virtual TSharedRef<IDatasmithScene> GetDatasmithSceneRef() = 0;
	virtual const TCHAR* GetAssetsOutputPath() = 0;

	/** Utility */
	virtual FNodeTracker* GetNodeTrackerByNodeName(const TCHAR* Name) = 0;
	virtual FSceneUpdateStats& GetStats() = 0;
};

/** Input data for mesh conversion*/
struct MeshConversionParams
{
	/** Node, this geom object created from */
	INode* Node;

	/** Extracted render mesh */
	const GeomUtils::FRenderMeshForConversion& RenderMesh;

	/** Whether to join all materials id into single material slot (used when a geometry doesn't have a multimaterial assigned) */
	bool bConsolidateMaterialIds;
};

/** Holds all the data to export mesh to DatasmithMeshElement */
class FDatasmithMeshConverter
{
public:
	FMD5Hash ComputeHash();

	FDatasmithMesh RenderMesh;
	TSet<uint16> SupportedChannels;
	TMap<int32, int32> UVChannelsMap;
	int32 SelectedLightmapUVChannel = -1;

	FDatasmithMesh CollisionMesh;
	bool bHasCollision = false;

	FMeshConverted Converted;
};

/** Creates Mesh element and converts max mesh into it */
bool ConvertMaxMeshToDatasmith(TimeValue CurrentTime, FMeshConverterSource& MeshSource, FDatasmithMeshConverter& MeshConverter);

bool OpenDirectLinkUI();
const TCHAR* GetDirectlinkCacheDirectory();


/** Max Notifications/Events/Callbacks handling */
class FNodeEventCallback;
class FNodeObserver;
class FMaterialObserver;

class FNotifications
{
public:

	FNotifications(IExporter& InExporter);
	~FNotifications();

	void RegisterForSystemNotifications();

	void StartSceneChangeTracking();
	void StopSceneChangeTracking();

	void AddNode(INode*);
	void AddMaterial(Mtl*);

	FString ConvertNotificationCodeToString(int code);
	void PrepareForUpdate();

	static void On3dsMaxNotification(void* param, NotifyInfo* info);

	bool bSceneChangeTracking = false;

	IExporter& Exporter;
	TMap<int, FString> NotificationCodetoString; // todo: remove, just for debug to output strings for notification codes
	TArray<int> NotificationCodesRegistered;
	TUniquePtr<FNodeEventCallback> NodeEventCallback;
	TUniquePtr<FNodeObserver> NodeObserver;
	TUniquePtr<FMaterialObserver> MaterialObserver;
};


class FDatasmithConverter
{
public:
	const float UnitToCentimeter;

	FDatasmithConverter();

	FVector toDatasmithVector(Point3 Point) const
	{
		return FVector(UnitToCentimeter * Point.x,
			UnitToCentimeter * -Point.y,
			UnitToCentimeter * Point.z);
	}

	FVector toDatasmithNormal(Point3 Point) const
	{
		return FVector(Point.x, -Point.y, Point.z);
	}

	FColor toDatasmithColor(Point3& Point)
	{
		//The 3ds Max vertex colors are encoded from 0 to 1 in float
		return FColor(Point.x * MAX_uint8, Point.y * MAX_uint8, Point.z * MAX_uint8);
	}

	void MaxToUnrealCoordinates(Matrix3 Matrix, FVector& Translation, FQuat& Rotation, FVector& Scale)
	{
		Point3 Pos = Matrix.GetTrans();
		Translation.X = Pos.x * UnitToCentimeter;
		Translation.Y = -Pos.y * UnitToCentimeter;
		Translation.Z = Pos.z * UnitToCentimeter;

		// Clear the transform on the matrix
		Matrix.NoTrans();

		// We're only doing Scale - save out the
		// rotation so we can put it back
		AffineParts Parts;
		decomp_affine(Matrix, &Parts);
		ScaleValue ScaleVal = ScaleValue(Parts.k * Parts.f, Parts.u);
		Scale = FVector(ScaleVal.s.x, ScaleVal.s.y, ScaleVal.s.z);

		Rotation = FQuat(Parts.q.x, -Parts.q.y, Parts.q.z, Parts.q.w);
	}
};

/** Material change tracking */
class FMaterialTracker
{
public:
	explicit FMaterialTracker(Mtl* InMaterial) : Material(InMaterial) {}


	Mtl* Material;

	/*Actual materials used for this assigned material*/
	TArray<Mtl*> Materials;
	TArray<Texmap*> Textures;

	bool bInvalidated = true;

	TArray<Mtl*>& GetActualMaterials()
	{
		return Materials;
	}

	TArray<Texmap*>& GetActualTexmaps()
	{
		return Textures;
	}

	void ResetActualMaterialAndTextures()
	{
		Materials.Reset();
		Textures.Reset();
	}

	void AddActualMaterial(Mtl* ActualMaterial)
	{
		Materials.AddUnique(ActualMaterial);
	}

	void AddActualTexture(Texmap* Texture)
	{
		Textures.AddUnique(Texture);
	}
};

class FMaterialTrackerHandle
{
public:

	explicit FMaterialTrackerHandle(Mtl* InMaterial) : Impl(MakeShared<FMaterialTracker>(InMaterial)) {}

	FMaterialTracker* GetMaterialTracker()
	{
		return Impl.Get();
	}

private:
	TSharedPtr<FMaterialTracker> Impl; // todo: reuse material tracker objects(e.g. make a pool)

	FMaterialTracker& GetImpl()
	{
		return *Impl;
	}
};

/**
Converts texmap to texture element
This is different from IDatasmithMaxTexmapToUEPbr which performs texmap conversion to full MaterialExpression which is used in material graph
IDatasmithMaxTexmapToUEPbr uses ToTextureElementConverter when it needs a texture element
*/
class ITexmapToTextureElementConverter
{
public:
	virtual ~ITexmapToTextureElementConverter(){}
	virtual TSharedPtr<IDatasmithTextureElement> Convert(FMaterialsCollectionTracker& MaterialsTracker, const FString& ActualBitmapName) = 0;

	FString TextureElementName;
};

class FMaterialsCollectionTracker
{
public:

	FMaterialsCollectionTracker(ISceneTracker& InSceneTracker) : SceneTracker(InSceneTracker), Stats(InSceneTracker.GetStats()) {}

	void Reset();

	/** Add material to track its changes*/
	FMaterialTracker* AddMaterial(Mtl* Material);
	/** When Material is not used in scene anymore - stop tracking it*/
	void ReleaseMaterial(FMaterialTracker& MaterialTracker);
	/** Mark changed*/
	void InvalidateMaterial(Mtl* Material);
	/** Reparse source material*/
	void UpdateMaterial(FMaterialTracker* MaterialTracker);
	/** Convert to Datasmith */
	void ConvertMaterial(Mtl* Material, TSharedRef<IDatasmithScene> DatasmithScene, const TCHAR* AssetsPath, TSet<Texmap*>& TexmapsConverted);

	/** Clean all converted data, remove from datasmith scene(e.g. before rebuilding material) */
	void RemoveConvertedMaterial(FMaterialTracker& MaterialTracker);

	/** Add material used in a specific tracked material's graph */
	void AddActualMaterial(FMaterialTracker& MaterialTracker, Mtl* Material);
	/** Reverse of AddActualMaterial*/
	void ReleaseActualMaterial(FMaterialTracker& MaterialTracker, Mtl* Material);

	/** Get name used for Datasmith material. Datasmith material names must be unique(used to identify elements) */
	const TCHAR* GetMaterialName(Mtl* SubMaterial);

	/** If material is used on a mesh/actor to create a datamsith element */
	bool IsMaterialUsed(Mtl* Material);
	/** Set datasmith materials to Datasmith static mesh according to assigned max material */
	void SetMaterialsForMeshElement(FMeshConverted& MeshConverted, Mtl* Material);
	void SetMaterialsForMeshElement(const TSharedPtr<IDatasmithMeshElement>& MeshElement, Mtl* Material, const TSet<uint16>& SupportedChannels);
	void SetMaterialForMeshElement(const TSharedPtr<IDatasmithMeshElement>& MeshElement, Mtl* MaterialAssignedToNode, Mtl* ActualMaterial, uint16 SlotId);
	void UnSetMaterialsForMeshElement(const TSharedPtr<IDatasmithMeshElement>& MeshElement);
	/** Set datasmith materials for mesh actor for assigned max material */
	void SetMaterialsForMeshActor(const TSharedPtr<IDatasmithMeshActorElement>& MeshActor, Mtl* Material, TSet<uint16>& SupportedChannels, const FVector3f& RandomSeed);
	void SetMaterialForMeshActorElement(const TSharedPtr<IDatasmithMeshActorElement>& MeshActor, Mtl* MaterialAssignedToNode, Mtl* Material, int32 SlotId);
	void UnSetMaterialsForMeshActor(const TSharedPtr<IDatasmithMeshActorElement>&);

	/** Record which Datasmith material was created for a Max material, not only for tracked(assigned) materials */
	void AddDatasmithMaterialForUsedMaterial(TSharedRef<IDatasmithScene> DatasmithScene, Mtl* Material, TSharedPtr<IDatasmithBaseMaterialElement> DatasmithMaterial);
	void RemoveDatasmithMaterialForUsedMaterial(Mtl* Material);

	void ResetInvalidatedMaterials()
	{
		InvalidatedMaterialTrackers.Reset();		
	}

	TSet<FMaterialTracker*> GetInvalidatedMaterials()
	{
		return InvalidatedMaterialTrackers;
	}

	void UpdateTexmap(Texmap* Texmap);

	/** Adds a texmap that need to be converted to texture element */
	void AddTexmapForConversion(Texmap* Texmap, const FString& DesiredTextureElementName, const TSharedPtr<ITexmapToTextureElementConverter>& Converter);

	void AddTextureElement(const TSharedPtr<IDatasmithTextureElement>& );
	void RemoveTextureElement(const TSharedPtr<IDatasmithTextureElement>& TextureElement);

	ISceneTracker& SceneTracker;

	/** Tracks all assigned materials */
	TMap<FMaterialKey, FMaterialTrackerHandle> MaterialTrackers;
	/** Materials needing update */
	TSet<FMaterialTracker*> InvalidatedMaterialTrackers;

	/**
	3ds Max Materials can be:
	- Tracked. When a material is assigned to a 3ds Max node. Assigned materials are tracked for changes by the plugin.
	- Used. When a material is used to create a DatasmithMaterialElement associated with it and this element is used by a DatasmithStaticMesh or MeshActor
	- Actual. Tracked + Used + <all unused submaterials>  - 
	 Tracked material won't necessarily be Used, for various reasons, e.g. it can be a composite material(in this case only its submaterials are be Used). Or material is assigned but all geometry is invisible.
	 Used materials can be missing in Tracked - a submaterial of an MultiSubobj material assigned to node is not assigned directly.
	 */

	/** Tracked material for every Actual material (i.e. set of material which use an Mtl as a submaterial ) */
	TMap<Mtl*, TSet<FMaterialTracker*>> ActualMaterialToMaterialTracker;
	/** All materials that have a corresponding Datasmith material created */
	TMap<Mtl*, TSharedPtr<IDatasmithBaseMaterialElement>> UsedMaterialToDatasmithMaterial;

	/** Cached unique name for each used material for Datasmith Element */
	TMap<Mtl*, FString> UsedMaterialToDatasmithMaterialName;

	/** Track materials used for static meshes */
	TMap<Mtl*, TMap<IDatasmithMeshElement*, TSet<int32>>> UsedMaterialToMeshElementSlot;
	TMap<IDatasmithMeshElement*, TMap<Mtl*, TSet<int32>>> MeshElementToUsedMaterialSlot;

	/** Track materials used for overrides on mesh actors */
	TMap<Mtl*, TMap<IDatasmithMeshActorElement*, TSet<int32>>> UsedMaterialToMeshActorSlot;
	TMap<IDatasmithMeshActorElement*, TMap<Mtl*, TSet<int32>>> MeshActorToUsedMaterialSlot;

	/** Texmaps tracking */
	/** Materials uses by nodes keep set of assigned textures they are used for */
	TMap<Texmap*, TSet<FMaterialTracker*>> UsedTextureToMaterialTracker;
	/** Keep track of Datasmith Element created for texmap to simplify update/removal(no need to search DatasmithScene) */
	TMap<Texmap*, TSet<TSharedPtr<IDatasmithTextureElement>>> UsedTextureToDatasmithElement;
	/** Back reference - which texture elements correspond to which texmaps(different texmaps can make same element) */
	TMap<TSharedPtr<IDatasmithTextureElement>, TSet<Texmap*>> TextureElementToTexmap;

	TMap<Texmap*, TSharedPtr<ITexmapToTextureElementConverter>> TexmapConverters;

	/** Bitmap textures using the same file become same texture element */
	TMap<FString, TSharedPtr<IDatasmithTextureElement>>  BitmapTextureElements;

	TSet<TSharedPtr<IDatasmithTextureElement>> TextureElementsAddedToScene;

	FSceneUpdateStats& Stats;

	FDatasmithUniqueNameProvider MaterialNameProvider;
};

/** Logging */

void LogFlush();

void LogError(const TCHAR* Msg);
void LogWarning(const TCHAR* Msg);
void LogCompletion(const TCHAR* Msg);
void LogInfo(const TCHAR* Msg);

void LogError(const FString& Msg);
void LogWarning(const FString& Msg);
void LogCompletion(const FString& Msg);
void LogInfo(const FString& Msg);

void LogErrorDialog(const FString& Msg);
void LogWarningDialog(const FString& Msg);
void LogCompletionDialog(const FString& Msg);
void LogInfoDialog(const FString& Msg);

/** Debug logging */
void LogDebugImpl(const FString& Msg);
void LogDebugDialog(const FString& Msg);

#ifdef LOG_DEBUG_HEAVY_ENABLE
	#define LOG_DEBUG_HEAVY(message) LogDebug(message)
#else
	#define LOG_DEBUG_HEAVY(message) 
#endif

void LogDebugNode(const FString& Name, class INode* Node);
void LogNodeEvent(const MCHAR* Name, INodeEventCallback::NodeKeyTab& Nodes);
#ifdef LOG_DEBUG_ENABLE
	#define LogDebug(Msg) LogDebugImpl(Msg)
#else
	#define LogDebug(Msg)
#endif 

}

#include "Windows/HideWindowsPlatformTypes.h"
