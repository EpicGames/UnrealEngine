// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifdef __clang__
#ifndef IGNORE_CLANG_WARNINGS
#define IGNORE_CLANG_WARNINGS
#pragma clang error "TODO: replace deprecated feature"
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wnon-pod-varargs"
#pragma clang diagnostic ignored "-Waddress-of-temporary"
#endif // IGNORE_CLANG_WARNINGS
#endif // __clang__


#ifndef MAX_INCLUDES_START
#define MAX_INCLUDES_START \
	THIRD_PARTY_INCLUDES_START \
	__pragma(warning(push)) \
	__pragma(warning(disable: 4005))  /*  macro redefinition */ \
	__pragma(warning(disable: 4263))  /*  member function does not override any base class virtual member function */ \
	__pragma(warning(disable: 4264))  /* no override available for virtual member function from base 'Symbol'; function is hidden */ \
	__pragma(warning(disable: 4495))  /* nonstandard extension used, 3dsMax 2018 SDK uses __super */ \
	__pragma(warning(disable: 4840))  /* non-portable use of class 'WStr' as an argument to a variadic function */ \
	__pragma(warning(disable: 4596))  /* illegal qualified name in member declaration */ \
	__pragma(warning(disable: 4535))  /* calling _set_se_translator() requires /EHa */ \
	__pragma(push_macro("NoExport"))
#endif // MAX_INCLUDE_STARTS

#ifndef MAX_INCLUDES_END
#define MAX_INCLUDES_END \
	__pragma(warning(pop)) \
	__pragma(pop_macro("NoExport")) \
	THIRD_PARTY_INCLUDES_END
#endif // MAX_INCLUDE_END
