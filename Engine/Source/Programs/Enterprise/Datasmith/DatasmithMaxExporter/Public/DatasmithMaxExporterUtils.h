// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "DatasmithMaxExporterDefines.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "Max.h"
	#include "maxscript/maxwrapper/mxsobjects.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

class IDatasmithActorElement;

namespace DatasmithMaxExporterUtils
{
	void ExportMaxTagsForDatasmithActor(const TSharedPtr<IDatasmithActorElement>& ActorElement, INode* Node, INode* ParentNode, TMap<TPair<uint32, TPair<uint32, uint32>>, MAXClass*>& KnownMaxClass, TMap<uint32, MAXSuperClass*>& KnownMaxSuperClass);
}

