// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithMaxExporterDefines.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "max.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"


class FDatasmithMaxStaticMeshAttributes;
class FDatasmithMesh;

namespace DatasmithMaxDirectLink
{

class ISceneTracker;
class FNodeTracker;

namespace GeomUtils
{

class FRenderMeshForConversion: FNoncopyable
{
public:
	explicit FRenderMeshForConversion()
		: Node(nullptr)
		, MaxMesh(nullptr)
		, bNeedsDelete(false)
		, ObjectToDelete(nullptr)
		, Pivot(FTransform::Identity)
	{
		ValidityInterval.SetInfinite();
	}

	explicit FRenderMeshForConversion(INode* InNode, Mesh* InMaxMesh, bool bInNeedsDelete, TriObject* InObjectToDelete=nullptr)
		: Node(InNode)
		, MaxMesh(InMaxMesh)
		, bNeedsDelete(bInNeedsDelete)
		, ObjectToDelete(InObjectToDelete)
		, Pivot(FTransform::Identity)
	{
		ValidityInterval.SetInfinite();
	}

	FRenderMeshForConversion(FRenderMeshForConversion&& Other)
	{
		Node = Other.Node;
		Other.Node = nullptr;

		MaxMesh = Other.MaxMesh;
		Other.MaxMesh = nullptr;

		Pivot = Other.Pivot;
		Other.Pivot = FTransform::Identity;

		ValidityInterval = Other.ValidityInterval;
		Other.ValidityInterval.SetInfinite();

		//Transfer ownership
		bNeedsDelete = Other.bNeedsDelete;
		Other.bNeedsDelete = false;

		ObjectToDelete = Other.ObjectToDelete;
		Other.ObjectToDelete = nullptr;
	}

	~FRenderMeshForConversion()
	{
		if (bNeedsDelete)
		{
			MaxMesh->DeleteThis();
		}
		if (ObjectToDelete)
		{
			ObjectToDelete->DeleteMe();
		}
	}

	bool IsValid() const
	{
		return MaxMesh != nullptr;
	}

	INode* GetNode() const
	{
		return Node;
	}

	Mesh* GetMesh() const
	{
		return MaxMesh;
	}

	const FTransform& GetPivot() const
	{
		return Pivot;
	}

	void SetPivot(const FTransform& InPivot)
	{
		Pivot = InPivot;
	}

	void SetValidityInterval(const Interval& Interval)
	{
		ValidityInterval = Interval;
	}

	Interval GetValidityInterval()
	{
		return ValidityInterval;
	}
private:
	INode* Node; // Scene node mesh is taken from 
	Mesh* MaxMesh; // the 3ds Max Mesh which holds the geometry
	bool bNeedsDelete; // Whether Mesh lifetime is in our hands
	TriObject* ObjectToDelete; // When the Mesh is retrieved from some temporary TriObject this TriObject needs to be deleted(after Mesh is used)

	FTransform Pivot; 
	Interval ValidityInterval;
};

bool ConvertRailClone(DatasmithMaxDirectLink::ISceneTracker& SceneTracker, DatasmithMaxDirectLink::FNodeTracker& NodeTracker);
bool ConvertForest(DatasmithMaxDirectLink::ISceneTracker& Scene, DatasmithMaxDirectLink::FNodeTracker& NodeTracker);

FRenderMeshForConversion GetMeshForGeomObject(TimeValue CurrentTime, INode* Node, const FTransform& Pivot = FTransform::Identity); // Extract mesh using already evaluated object

FRenderMeshForConversion GetMeshForCollision(TimeValue CurrentTime, DatasmithMaxDirectLink::ISceneTracker& SceneTracker, INode* Node, bool bBakePivot);
INode* GetCollisionNode(DatasmithMaxDirectLink::ISceneTracker& SceneTracker, INode* OriginalNode, const FDatasmithMaxStaticMeshAttributes* DatasmithAttributes, bool& bOutFromDatasmithAttribute);

void FillDatasmithMeshFromMaxMesh(TimeValue CurrentTime, FDatasmithMesh& DatasmithMesh, Mesh& MaxMesh, INode* ExportedNode, bool bForceSingleMat, TSet<uint16>& SupportedChannels, TMap<int32, int32>& UVChannelsMap, FTransform Pivot);

}
}
