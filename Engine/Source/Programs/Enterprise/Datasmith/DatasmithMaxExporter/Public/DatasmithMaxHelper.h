// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "DatasmithMaxExporterDefines.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
#include "bitmap.h"
#include "maxversion.h"
#include "units.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

// In 3ds max 2022 SDK, GetMasterScale is deprecated and replaced with GetSystemUnitScale
#if MAX_PRODUCT_YEAR_NUMBER < 2022
inline double GetSystemUnitScale(int type) { return GetMasterScale(type); }
#endif

// Helper structure to help manage loading and deleting bitmaps when we only have its BitmapInfo
struct FScopedBitMapPtr
{
public:
	BitmapInfo MapInfo;
	Bitmap* Map = nullptr;

	FScopedBitMapPtr(const BitmapInfo& InMapInfo, Bitmap* InMap);
	~FScopedBitMapPtr();

private:
	bool bNeedsDelete = false;
};


namespace DatasmithMaxHelper
{
	void FilterInvertedScaleTransforms(const TArray<FTransform>& Transforms, TArray<FTransform>& OutNormal, TArray<FTransform>& OutInverted);
}