// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Resources/Windows/resource.h"
#include "DatasmithMaxClassIDs.h"
#include "DatasmithLogger.h"

#include "Templates/SharedPointer.h"

#include "Windows/AllowWindowsPlatformTypes.h"

class INode;
class Mtl;
class Texmap;

class DatasmithMaxLogger :public FDatasmithLogger, public TSharedFromThis<DatasmithMaxLogger>
{
public:
	static DatasmithMaxLogger& Get();

	void Purge();

	void AddPartialSupportedMat(Mtl* Mat);
	void AddUnsupportedMat(Mtl* Mat);
	void AddPartialSupportedMap(Texmap* Map);
	void AddUnsupportedMap(Texmap* Map);
	void AddOtherMapWarning(FString Warning);
	void AddUnsupportedLight(INode* Light);
	void AddUnsupportedUV(INode* Node);
	
	void AddInvalidObj(INode* Node);
	TArray<INode*>GetInvalidObjects() const;

	void AddInvalidTransform(INode* Node);
	
	bool HasWarnings();
	void Show(HWND hDlg);
	bool CopyToClipBoard();

	FString GetLightDescription(INode* LightNode);

private:
	void AddItem(const TCHAR* Msg, HWND Handle, FString& FullMsg);
	void AddObjectList(TArray< INode* > ObjectList, HWND Handle, const TCHAR* Header, const TCHAR* Description = nullptr);

	TArray<Mtl*> PartialSupportedMats;
	TArray<Mtl*> UnsupportedMats;
	TArray<Texmap*> PartialSupportedMaps;
	TArray<Texmap*> UnsupportedMaps;
	TArray<INode*> UnsupportedLight;
	TArray<INode*> FailUVs;
	TArray<INode*> FailObjs;
	TArray<INode*> InvalidTransforms;

	FString ShowMessage;
};

INT_PTR CALLBACK MsgListDlgProc(HWND hDlg, UINT iMsg, WPARAM wParam, LPARAM lParam);

#include "Windows/HideWindowsPlatformTypes.h"
