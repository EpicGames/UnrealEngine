// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithMaxExporterDefines.h"

class FDatasmithMesh;
class FDatasmithMaxStaticMeshAttributes;
class IDatasmithMeshElement;
class INode;
class Matrix3;
class Mesh;
class MeshNormalSpec;
class Point3;

struct FMaxExportMeshArgs
{
	bool bForceSingleMat = false;
	INode* Node = nullptr;

	//Pass along a mesh if the mesh can't be simply extracted from the node
	Mesh* MaxMesh = nullptr;

	const FDatasmithMaxStaticMeshAttributes* DatasmithAttributes;

	const TCHAR* ExportPath = nullptr;
	const TCHAR* ExportName = nullptr;

	float UnitMultiplier = 1.f;
	FTransform Pivot = FTransform::Identity;
	bool bBakePivot = true;
};

class FDatasmithMaxMeshExporter
{
public:
	FDatasmithMaxMeshExporter();
	~FDatasmithMaxMeshExporter();

	bool CalcSupportedChannelsOnly(INode* Node, TSet<uint16>& SupportedChannels, bool bForceSingleMat) const;

	//export unreal mesh from actual node
	TSharedPtr< IDatasmithMeshElement > ExportMesh(FMaxExportMeshArgs& ExportMeshArgs, TSet<uint16>& SupportedChannels, FString& Error);
	
	//export dummy unreal mesh it could be used for helpers etc, currently a void mesh is exported
	TSharedPtr< IDatasmithMeshElement > ExportDummyMesh( const TCHAR* ExportPath, const TCHAR* ExportName );

	typedef TMap<int32, int32> FUVChannelsMap;
	const FUVChannelsMap& GetUVChannelsMapForMesh(const TCHAR* MeshName) const;

	static INode* GetCollisionNode(INode* OriginalNode, const FDatasmithMaxStaticMeshAttributes* DatasmithAttributes, bool& bOutFromDatasmithAttribute);

private:
	/**
	 * Export a scene node to a datasmith mesh
	 *
	 * @param DSMesh	        Destination mesh
	 * @param bForceSingleMat   Ignore Material from source mesh
	 * @param ExportNode        Source scene node
	 * @param SupportedChannels
	 * @param MeshName          UVChannel name
	 * @param Pivot             Transformation to bake into the mesh
	 */
	bool CreateDatasmithMesh(FDatasmithMesh& DSMesh, FMaxExportMeshArgs& ExportMeshArgs, TSet<uint16>& SupportedChannels);

	/**
	 * Actual data extraction from Max mesh
	 *
	 * @param DatasmithMesh      Destination mesh
	 * @param MaxMesh            Source mesh
	 * @param bForceSingleMat    Ignore Material from source mesh
	 * @param SupportedChannels
	 * @param MeshName           UVChannel name
	 * @param Pivot              Transformation to bake into the mesh
	 */
	void FillDatasmithMeshFromMaxMesh(FDatasmithMesh& DatasmithMesh, Mesh& MaxMesh, INode* ExportedNode, bool bForceSingleMat, TSet<uint16>& SupportedChannels,
		const TCHAR* MeshName, FTransform Pivot);

	/**
	 * Fill the mesh with a bounding box
	 * 
	 * @param DatasmithMesh		Destination mesh
	 * @param ExportNode		The node from which the bounding box will be made
	 * @param Pivot				Transformation to bake into the mesh
	 * @return True if the mesh was filled with a bounding box
	 */
	bool FillDatasmithMeshFromBoundingBox(FDatasmithMesh& DatasmithMesh, INode* ExportNode, FTransform Pivot) const;

	// Convert Max to UE axis
	FVector Point3ToFVector(Point3 p3) const;

	// Convert Max to UE axis, and handle scene unit scaling
	FVector Unit3ToFVector(Point3 p3) const;

	MeshNormalSpec* GetSpecifiedNormalsFromMesh(Mesh* MaxMesh) const;
	INode* GetTemporalNode() const;
	bool DeleteTemporalNode() const;

	TMap<FString, FUVChannelsMap> MeshNamesToUVChannels;

	float UnitMultiplier;
};