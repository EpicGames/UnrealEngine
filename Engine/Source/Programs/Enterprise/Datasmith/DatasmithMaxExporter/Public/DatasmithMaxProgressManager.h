// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithProgressManager.h"

class FDatasmithMaxProgressManager : public IDatasmithProgressManager
{
public:
	FDatasmithMaxProgressManager();

	virtual ~FDatasmithMaxProgressManager() override;

	virtual void ProgressEvent(float InProgressRatio, const TCHAR* InProgressString) override;

	void SetMainMessage(const TCHAR* InProgressMessage);

	void SetProgressStart(float InProgressStart)
	{
		ProgressStart = InProgressStart;
	}

	void SetProgressEnd(float InProgressEnd)
	{
		ProgressEnd = InProgressEnd;
	}

private:
	float ProgressStart;
	float ProgressEnd;
	int32 ProgressBarCounter;
};