// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DatasmithDefinitions.h"

class Animatable;
enum class EMaxLightClass;
class IDatasmithElement;
class IDatasmithActorElement;
class IDatasmithAreaLightElement;
class IDatasmithCameraActorElement;
class IDatasmithLevelSequenceElement;
class IDatasmithLightActorElement;
class IDatasmithLightmassPortalElement;
class IDatasmithMeshActorElement;
class IDatasmithPointLightElement;
class IDatasmithPostProcessElement;
class IDatasmithPostProcessVolumeElement;
class IDatasmithScene;
class IDatasmithTransformAnimationElement;
class IDatasmithMetaDataElement;
class INode;
class INodeTab;
class CameraObject;
class LightObject;
class Matrix3;
class Mtl;
class Point3;
class Quat;
class Texmap;
class ToneOperator;

enum EStaticMeshExportMode : uint8;


namespace DatasmithMaxDirectLink
{
	class FLightNodeConverter;
}

/**
 * Lights may have multiple default orientation depending on their shape and type. 
 * This is a structure used when converting actor's coordinate in MaxToUnrealCoordinates to make sure to apply right correction to the light objects.
 */
struct FMaxLightCoordinateConversionParams
{
public:
	bool bIsLight = false;
	bool bIsCoronaLight = false;
	EDatasmithLightShape LightShape = EDatasmithLightShape::None;

	FMaxLightCoordinateConversionParams() {}
	FMaxLightCoordinateConversionParams(INode* LightNode, EDatasmithLightShape Shape = EDatasmithLightShape::None);
};

class FDatasmithMaxSceneExporter
{
public:
	static bool ExportActor(TSharedRef< IDatasmithScene > DatasmithScene, INode* Node, const TCHAR* Name, float UnitMultiplier);
	static void ExportMeshActor(TSharedRef< IDatasmithScene > DatasmithScene,  TSet<uint16>& SupportedChannels, INode* Node, const TCHAR* MeshName,
		float UnitMultiplier, bool bPivotIsBakedInGeometry, Mtl* StaticMeshMtl, const EStaticMeshExportMode& ExportMode);
	static bool ExportCameraActor(TSharedRef< IDatasmithScene > DatasmithScene, INode* Parent, INodeTab Instances, int InstanceIndex, const TCHAR* Name, float UnitMultiplier);
	static void WriteEnvironment(TSharedRef< IDatasmithScene > DatasmithScene, bool bOnlySelection);
	static void ExportToneOperator(TSharedRef< IDatasmithScene > DatasmithScene);
	static void ExportAnimation(TSharedRef< IDatasmithLevelSequenceElement > LevelSequence, INode* ParentNode, INode* Node, const TCHAR* ActorName, float UnitMultiplier, const FMaxLightCoordinateConversionParams& LightParams = FMaxLightCoordinateConversionParams());

	static FString GetActualPath(const TCHAR* OriginalPath);

	static Mtl* GetRandomSubMaterial(Mtl* Material, FVector3f RandomSeed);

	/**
	 * Extract the Node to Object transform in UE coordinates
	 * @note aka. Object-Offset Transform Matrix in Autodesk language
	 * @see https://help.autodesk.com/view/3DSMAX/2018/ENU/?guid=__files_GUID_3B001F21_8FE9_4663_A972_E648682A0ACD_htm
	 *
	 * @param Node           node to extract Pivot from
	 * @param UnitMultiplier scene master scale
	 */
	static FTransform GetPivotTransform(INode* Node, float UnitMultiplier);
	static void MaxToUnrealCoordinates(Matrix3 Matrix, FVector& Translation, FQuat& Rotation, FVector& Scale, float UnitMultiplier, const FMaxLightCoordinateConversionParams& LightParams = FMaxLightCoordinateConversionParams());

	static TSharedPtr< IDatasmithLightActorElement > CreateLightElementForNode(INode* Node, const TCHAR* Name);
	static bool ParseLight(DatasmithMaxDirectLink::FLightNodeConverter&, INode* Node, TSharedRef< IDatasmithLightActorElement > LightElement, TSharedRef< IDatasmithScene > DatasmithScene);

	static TSharedPtr<IDatasmithMetaDataElement> ParseUserProperties(INode* Node, TSharedRef< IDatasmithActorElement > ActorElement, TSharedRef< IDatasmithScene > DatasmithScene);

private:
	/* Return the physical scale for the unitless light */
	static float GetLightPhysicalScale();

	static int GetSeedFromMaterial(Mtl* Material);

	static bool ParseActor(INode* Node, TSharedRef< IDatasmithActorElement > ActorElement, float UnitMultiplier, TSharedRef< IDatasmithScene > DatasmithScene);
	static bool ParseLightObject(LightObject& Light, TSharedRef< IDatasmithLightActorElement > LightElement, TSharedRef< IDatasmithScene > DatasmithScene);
	static bool ParseCoronaLight(DatasmithMaxDirectLink::FLightNodeConverter&, LightObject& Light, TSharedRef< IDatasmithAreaLightElement > AreaLightElement, TSharedRef< IDatasmithScene > DatasmithScene);
	static bool ParsePhotometricLight(DatasmithMaxDirectLink::FLightNodeConverter& Converter, LightObject& Light, TSharedRef< IDatasmithPointLightElement > PointLightElement, TSharedRef< IDatasmithScene > DatasmithScene);
	static bool ParseVRayLight(LightObject& Light, TSharedRef< IDatasmithAreaLightElement > AreaLightElement, TSharedRef< IDatasmithScene > DatasmithScene);
	static bool ParseVRayLightPortal(LightObject& Light, TSharedRef< IDatasmithLightmassPortalElement > LightPortalElement, TSharedRef< IDatasmithScene > DatasmithScene);
	static bool ParseVRayLightIES(DatasmithMaxDirectLink::FLightNodeConverter&, LightObject& Light, TSharedRef< IDatasmithPointLightElement > PointLightElement, TSharedRef< IDatasmithScene > DatasmithScene);
	static bool ParseLightParameters(DatasmithMaxDirectLink::FLightNodeConverter&, EMaxLightClass LightClass, LightObject& Light, TSharedRef< IDatasmithLightActorElement > LightElement, TSharedRef< IDatasmithScene > DatasmithScene);
	static bool ProcessLightTexture(TSharedRef< IDatasmithLightActorElement > LightElement, Texmap* LightTexture, TSharedRef< IDatasmithScene > DatasmithScene);
	static bool ParseTransformAnimation(INode* ParentNode, INode* Node, TSharedRef< IDatasmithTransformAnimationElement > AnimationElement, float UnitMultiplier, const FMaxLightCoordinateConversionParams& LightParams);
	static void ParseSun(INode* Node, TSharedRef<IDatasmithLightActorElement> LightElement);
};
