// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DatasmithMaxExporterDefines.h"
#include "DatasmithSceneExporter.h"

#include "Math/Color.h"
#include "Templates/SharedPointer.h"

#include "Windows/AllowWindowsPlatformTypes.h"
MAX_INCLUDES_START
	#include "iparamb2.h"
	#include "max.h"
	#include "stdmat.h"
	#include "gamma.h"
MAX_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

enum class EDSBitmapType
{
	NotSupported,
	RegularBitmap,
	AutodeskBitmap,
	TheaBitmap,
	CoronaBitmap,
	VRayHRDI,
	NormalMap,
	ColorCorrector,
	FallOff,
	Mix,
	Gradient,
	GradientRamp,
	Checker,
	Noise,
	Cellular,
	VRayDirt,
	VRayColor,
	CoronaColor,
	CoronaMix,
	CoronaMultiTex,
	CoronaAO,
	CompositeTex,
	PhysicalSky,
	ThirdPartyMultiTex,
	BakeableMap
};

enum class EDSMaterialType
{
	NotSupported,
	TheaMaterial,
	TheaMaterialDeprecated,
	TheaBasic,
	TheaGlossy,
	TheaSSS,
	TheaFilm,
	TheaCoating,
	TheaRandom,
	MultiMat,
	StandardMat,
	BlendMat,
	XRefMat,
	ArchDesignMat,
	ShellMat,
	VRayMat,
	VRayLightMat,
	VRayBlendMat,
	VRayFastSSSMat,
	CoronaMat,
	CoronaLightMat,
	CoronaLayerMat,
	PhysicalMat
};

class FDatasmithMaxMatHelper
{
public:
	// CLASS RELATED FUNCTIONS
	static EDSBitmapType GetTextureClass(Texmap* mTexMap);
	static EDSMaterialType GetMaterialClass(Mtl* Material);
	/**
	 * Gets a XRefMaterial and returns the rendered material which is either the SubMaterial or SourceMaterial depending on if the source material is overridden.
	 * @param XRefMaterial	The XRefMaterial we want the rendered material of.
	 * @return              The rendered material, can be nullptr.
	*/
	static Mtl* GetRenderedXRefMaterial(Mtl* XRefMaterial);

	static bool HasNonBakeableSubmap(Texmap* InTexmap);

	static FLinearColor MaxColorToFLinearColor(BMM_Color_fl Color, float Multiplier = 1.0f); // todo: rename, although result type is FLinearColor this function does pow(rgb, 1/gamma), i.e. converting to gamma-space
	static FLinearColor MaxLinearColorToFLinearColor(BMM_Color_fl Color, float Multiplier = 1.0f);
	static float GetBitmapGamma(BitmapTex* InBitmapTex);
	static float GetBitmapGamma(BitmapInfo* InBitmapInfo);
	static float GetVrayHdriGamma(BitmapTex* InBitmapTex);
	static bool IsSRGB(Bitmap& InBitmap);
	static bool IsSRGB(BitmapTex& InBitmapTex);
};

// EXPORT FUNCTIONS THAT NEEDS TO BE CALLED FROM OTHER PARTS OF CODE
class FDatasmithMaxMatExport
{
public:
	// stores the textures without allowing dupe in uniqueTextures
	static void GetXMLTexture(TSharedRef< IDatasmithScene > DatasmithScene, Texmap* InTexmap, const TCHAR* AssetsPath, TArray<TSharedPtr< IDatasmithTextureElement >>* OutTextureElements=nullptr);
		
	// Exports the given material if it hasn't been exported yet and returns the created IDatasmithBaseMaterialElement, else returns an invalid TSharedPtr.
	static TSharedPtr< IDatasmithBaseMaterialElement > ExportUniqueMaterial(TSharedRef< IDatasmithScene > DatasmithScene, Mtl* Material, const TCHAR* AssetsPath);
	static void WriteXMLMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement > MaterialElement, Mtl* Material);

private:
	static bool UseFirstSubMapOnly(EDSMaterialType MaterialType, Mtl* Material);
};

// ACTUAL EXPORT FUNCTIONS
class FDatasmithMaxMatWriter
{
public:
	// String suffix to be added to the textures identifier
	static FString TextureSuffix;
	// File suffix for texture bakes including its dot
	static FString TextureBakeFormat;
	// selects from any kind of texture
	static FString DumpTexture(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, const TCHAR* ColorPrefix, bool bForceInvert, bool bIsGrayscale);
	// selects from any kind of texture
	static FString DumpNormalTexture(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, const TCHAR* ColorPrefix, bool bForceInvert, bool bIsGrayscale);
	// selects from any kind of texture the texture is mixed with the given color using Weight
	static void DumpWeightedTexture(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, BMM_Color_fl Color, float Weight, const TCHAR* Prefix, const TCHAR* ColorPrefix, bool bForceInvert, bool bIsGrayscale);
	// selects from any kind of texture the ColorA is mixed with the Color B using Weight
	static void DumpWeightedColor(TSharedPtr<IDatasmithCompositeTexture>& CompTex, BMM_Color_fl ColorA, BMM_Color_fl ColorB, float Weight, const TCHAR* Prefix);
	// if we want to export sub-texture but it is void
	static FString DumpColorOfTexmap(TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* ColorPrefix, const TCHAR* property);

	//---------------------------------------------------------------
	// SCANLINE DEPENDANT
	//---------------------------------------------------------------
	static FString GetActualBitmapPath(BitmapTex* InBitmapTex);
	static FString GetActualBitmapPath(BitmapInfo* InBitmapInfo);
	static FString GetActualBitmapName(BitmapTex* InBitmapTex);
	static FString GetActualBitmapName(BitmapInfo* InBitmapInfo);
	static FString CropBitmap(BitmapTex* InBitmapTex);
	static FString DumpBitmap(TSharedPtr<IDatasmithCompositeTexture>& CompTex, BitmapTex* InBitmapTex, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale);
	static FString DumpAutodeskBitmap(TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale);
	static void ExportStandardMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);
	static void ExportBlendMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);
	static FString DumpColorCorrect(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale);
	static FString DumpMix(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale);
	static FString DumpCompositetex(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale);
	static FString DumpFalloff(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale);
	static FString DumpBakeable(TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale);

	//---------------------------------------------------------------
	// THEA DEPENDANT
	//---------------------------------------------------------------
	static FString DumpBitmapThea(TSharedPtr<IDatasmithCompositeTexture>& CompTex, BitmapTex* InBitmapTex, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale);
	static void GetTheaTexmap(TSharedRef< IDatasmithScene > DatasmithScene, BitmapTex* InBitmapTex, TArray<TSharedPtr< IDatasmithTextureElement >>* OutTextureElements);
	static void ExportTheaSubmaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithShaderElement >& MaterialShader, Mtl* Material, EDSMaterialType MaterialType);
	static void ExportTheaMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);

	//---------------------------------------------------------------
	// VRAY DEPENDANT
	//---------------------------------------------------------------
	static FString DumpVrayColor(TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert);
	static FString DumpVrayHdri(TSharedPtr<IDatasmithCompositeTexture>& CompTex, BitmapTex* InBitmapTex, const TCHAR* Prefix, bool bForceInvert);
	static void ExportVRayMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);
	static void ExportVRayLightMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);
	static void ExportVrayBlendMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);
	static FString GetActualVRayBitmapName(BitmapTex* InBitmapTex);

	//---------------------------------------------------------------
	// MENTAL RAY DEPENDANT
	//---------------------------------------------------------------
	static void ExportArchDesignMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);

	//---------------------------------------------------------------
	// CORONA DEPENDANT
	//---------------------------------------------------------------
	static FString DumpBitmapCorona(TSharedPtr<IDatasmithCompositeTexture>& CompTex, BitmapTex* InBitmapTex, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale);
	static FString DumpCoronaColor(TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert);
	static void GetCoronaTexmap(TSharedRef< IDatasmithScene > DatasmithScene, BitmapTex* InBitmapTex, TArray<TSharedPtr< IDatasmithTextureElement >>* OutTextureElements);
	static void ExportCoronaMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);
	static void ExportCoronaLightMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);
	static void ExportCoronaBlendMaterial(TSharedRef< IDatasmithScene > DatasmithScene,  TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);
	static FString DumpCoronaMix(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, bool bForceInvert, bool bIsGrayscale);
	static FString DumpCoronaMultitex(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr<IDatasmithCompositeTexture>& CompTex, Texmap* InTexmap, const TCHAR* Prefix, const TCHAR* ColorPrefix, bool bForceInvert, bool bIsGrayscale);
	static bool GetCoronaFixNormal(Texmap* InTexmap);
	
	//---------------------------------------------------------------
	// ARNOLD DEPENDANT
	//---------------------------------------------------------------
	static void ExportPhysicalMaterial(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);
	static void ExportPhysicalMaterialCoat(TSharedRef< IDatasmithScene > DatasmithScene, TSharedPtr< IDatasmithMaterialElement >& MaterialElement, Mtl* Material);
	static void ExportPhysicalMaterialProperty(TSharedRef< IDatasmithScene > DatasmithScene, Texmap* Texture, bool bTextureEnabled, Texmap* TextureWeight, bool bTextureWeightEnabled, BMM_Color_fl Color, float Weight, TSharedPtr<IDatasmithCompositeTexture>& CompTex, FString TextureAliasName, FString ColorAliasName, bool bForceInvert, bool bIsGrayscale);

};
