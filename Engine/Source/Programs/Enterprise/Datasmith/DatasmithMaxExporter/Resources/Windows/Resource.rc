// Microsoft Visual C++ generated resource script.
//
#include "resource.h"

#define APSTUDIO_READONLY_SYMBOLS
/////////////////////////////////////////////////////////////////////////////
//
// Generated from the TEXTINCLUDE 2 resource.
//
#include "windows.h"
#include "../../../../../../../../Engine/Source/Runtime/Launch/Resources/Version.h"


/////////////////////////////////////////////////////////////////////////////
#undef APSTUDIO_READONLY_SYMBOLS

/////////////////////////////////////////////////////////////////////////////
// English (United States) resources

#if !defined(AFX_RESOURCE_DLL) || defined(AFX_TARG_ENU)
LANGUAGE LANG_ENGLISH, SUBLANG_ENGLISH_US
#pragma code_page(1252)

/////////////////////////////////////////////////////////////////////////////
//
// Dialog
//

IDD_EXPORTOPTIONS DIALOGEX 0, 0, 199, 113
STYLE DS_SETFONT | DS_MODALFRAME | WS_POPUP | WS_CAPTION | WS_SYSMENU
CAPTION "Datasmith Export Options"
FONT 8, "MS Sans Serif", 0, 0, 0x0
BEGIN
    DEFPUSHBUTTON   "OK",IDOK,6,92,88,14
    PUSHBUTTON      "Cancel",IDCANCEL,98,92,94,14
    COMBOBOX        IDC_CBInclude,98,49,94,30,CBS_DROPDOWNLIST | WS_VSCROLL | WS_TABSTOP
    LTEXT           "Include",IDC_LBInclude,8,51,85,8
    CONTROL         "v0.14",IDC_LBVersion,"Static",SS_SIMPLE | WS_DISABLED | WS_GROUP,118,25,74,8,WS_EX_RIGHT
    ICON            IDI_ICONUE,IDC_STATIC_LOGO,88,16,21,20,SS_CENTERIMAGE
    COMBOBOX        IDC_CBAnimated,98,70,94,30,CBS_DROPDOWNLIST | WS_VSCROLL | WS_TABSTOP
    LTEXT           "Animated Transforms",IDC_LBAnimated,8,72,85,8
END

IDD_ERROR_MSGS DIALOGEX 0, 0, 310, 182
STYLE DS_SETFONT | DS_MODALFRAME | WS_POPUP | WS_VISIBLE | WS_CAPTION | WS_SYSMENU | WS_THICKFRAME
CAPTION "Output Warnings:"
FONT 8, "MS Sans Serif", 0, 0, 0x0
BEGIN
    DEFPUSHBUTTON   "Copy log to clipboard",IDCOPY,16,163,140,16
    DEFPUSHBUTTON   "Select invalid objects",IDSELECTINVALID,157,163,140,16
    LISTBOX         IDC_ERROR_MSG_LIST,4,4,302,156,LBS_USETABSTOPS | LBS_NOINTEGRALHEIGHT | WS_VSCROLL | WS_TABSTOP
END


#ifdef APSTUDIO_INVOKED
/////////////////////////////////////////////////////////////////////////////
//
// TEXTINCLUDE
//

1 TEXTINCLUDE 
BEGIN
    "resource.h\0"
END

2 TEXTINCLUDE 
BEGIN
    "#include ""windows.h""\r\n"
    "#include ""../../../../../../../../Engine/Source/Runtime/Launch/Resources/Version.h""\r\n"
    "\r\n"
    "\0"
END

3 TEXTINCLUDE 
BEGIN
    "\0"
END

#endif    // APSTUDIO_INVOKED


/////////////////////////////////////////////////////////////////////////////
//
// DESIGNINFO
//

#ifdef APSTUDIO_INVOKED
GUIDELINES DESIGNINFO
BEGIN
    IDD_EXPORTOPTIONS, DIALOG
    BEGIN
        LEFTMARGIN, 7
        RIGHTMARGIN, 192
        VERTGUIDE, 98
        TOPMARGIN, 7
        BOTTOMMARGIN, 106
    END

    IDD_ERROR_MSGS, DIALOG
    BEGIN
        BOTTOMMARGIN, 181
    END
END
#endif    // APSTUDIO_INVOKED


/////////////////////////////////////////////////////////////////////////////
//
// AFX_DIALOG_LAYOUT
//

IDD_EXPORTOPTIONS AFX_DIALOG_LAYOUT
BEGIN
    0
END

IDD_ERROR_MSGS AFX_DIALOG_LAYOUT
BEGIN
    0
END


/////////////////////////////////////////////////////////////////////////////
//
// Icon
//

// Icon with lowest ID value placed first to ensure application icon
// remains consistent on all systems.
IDI_ICONUE              ICON                    "UE.ico"


/////////////////////////////////////////////////////////////////////////////
//
// String Table
//

STRINGTABLE
BEGIN
    IDS_SYNC_NAME    "Sync"
    IDS_SYNC_DESC    "Synchronize with Direct Link"
    IDS_CONNECTIONS_NAME    "Connections"
    IDS_CONNECTIONS_DESC    "Manage Connections"
    IDS_AUTOSYNC_NAME    "AutoSync"
    IDS_AUTOSYNC_DESC    "Toggle Direct Link Auto Sync"
    IDS_EXPORT_NAME    "Export"
    IDS_EXPORT_DESC    "Export to Datasmith file"
    IDS_SHOWLOG_NAME    "Log"
    IDS_SHOWLOG_DESC    "Show Messages"
    IDS_DATASMITH_CATEGORY "Datasmith"
    IDS_EXPORT_SELECTED_NAME    "Export Selected"
    IDS_EXPORT_SELECTED_DESC    "Export Selected objects to Datasmith file"
END

#endif    // English (United States) resources
/////////////////////////////////////////////////////////////////////////////


/////////////////////////////////////////////////////////////////////////////
// English (Canada) resources

#if !defined(AFX_RESOURCE_DLL) || defined(AFX_TARG_ENC)
LANGUAGE LANG_ENGLISH, SUBLANG_ENGLISH_CAN
#pragma code_page(1252)

/////////////////////////////////////////////////////////////////////////////
//
// Version
//

VS_VERSION_INFO VERSIONINFO
 FILEVERSION 1,0,0,1
 PRODUCTVERSION 1,0,0,1
 FILEFLAGSMASK 0x3fL
#ifdef _DEBUG
 FILEFLAGS 0x1L
#else
 FILEFLAGS 0x0L
#endif
 FILEOS 0x40004L
 FILETYPE 0x0L
 FILESUBTYPE 0x0L
BEGIN
    BLOCK "StringFileInfo"
    BEGIN
        BLOCK "100904b0"
        BEGIN
            VALUE "CompanyName", "Epic Games, Inc."
            VALUE "FileDescription", "DatasmithMaxExporter"
            VALUE "FileVersion", "1.0.0.1"
            VALUE "InternalName", "DatasmithMaxExporter"
            VALUE "LegalCopyright", "Copyright Epic Games, Inc. All Rights Reserved."
            VALUE "OriginalFilename", "DatasmithMaxExporter.dle"
            VALUE "ProductName", "Unreal Engine"
            VALUE "ProductVersion", "5.0"
        END
    END
    BLOCK "VarFileInfo"
    BEGIN
        VALUE "Translation", 0x1009, 1200
    END
END

#endif    // English (Canada) resources
/////////////////////////////////////////////////////////////////////////////


