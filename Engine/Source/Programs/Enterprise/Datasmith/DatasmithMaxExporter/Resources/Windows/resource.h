// Copyright Epic Games, Inc. All Rights Reserved.

//{{NO_DEPENDENCIES}}
// Microsoft Visual C++ generated include file.
// Used by Resource.rc
//
#define IDS_DXFEXP                      1
#define IDCOPY                          4
#define IDSELECTINVALID                 5
#define IDD_EXPORTOPTIONS               101
#define IDB_BITMAP1                     105
#define IDI_ICON1                       107
#define IDI_ICONUE                      107
#define IDR_ACCELERATOR1                108
#define IDS_SYNC_NAME                   109
#define IDS_SYNC_DESC                   110
#define IDS_CONNECTIONS_NAME            111
#define IDS_CONNECTIONS_DESC            112
#define IDS_AUTOSYNC_NAME               113
#define IDS_AUTOSYNC_DESC               114
#define IDS_EXPORT_NAME                 115
#define IDS_EXPORT_DESC                 116
#define IDS_SHOWLOG_NAME                117
#define IDS_SHOWLOG_DESC                118
#define IDS_DATASMITH_CATEGORY          119
#define IDS_EXPORT_SELECTED_NAME        120
#define IDS_EXPORT_SELECTED_DESC        121
#define IDD_ERROR_MSGS                  181
#define IDC_BYOBJECT                    1016
#define IDC_BYMATERIAL                  1017
#define IDC_1LAYER                      1018
#define IDC_ACTORS                      1019
#define IDC_EDITPATH                    1020
#define IDC_CBTextures                  1020
#define IDC_MATERIALS                   1021
#define IDC_LIGHTS                      1022
#define IDC_GEOMETRY                    1023
#define IDC_LBTextures                  1024
#define IDC_CBColorGamma                1025
#define IDC_LBColorGamma                1026
#define IDC_CBLightmap                  1027
#define IDC_LBLightmap                  1028
#define IDC_CBMaxSize                   1029
#define IDC_LBMaxSize                   1030
#define IDC_CBPath                      1031
#define IDC_CBInclude                   1031
#define IDC_LBPath                      1032
#define IDC_LBInclude                   1032
#define IDC_STATIC_LOGO                 1033
#define IDC_LBMaxSize2                  1034
#define IDC_LBVersion                   1034
#define IDC_CHECK1                      1035
#define IDC_CHECKNEWVRAY                1035
#define IDC_CBAnimated                  1035
#define IDC_LBAnimated                  1036
#define IDC_ERROR_MSG_LIST              1168
#define IDC_STATIC                      -1

// Next default values for new objects
// 
#ifdef APSTUDIO_INVOKED
#ifndef APSTUDIO_READONLY_SYMBOLS
#define _APS_NEXT_RESOURCE_VALUE        120
#define _APS_NEXT_COMMAND_VALUE         40002
#define _APS_NEXT_CONTROL_VALUE         1036
#define _APS_NEXT_SYMED_VALUE           101
#endif
#endif
