@echo off

robocopy %1 %2 %3 /J /Z /S /R:1 /W:0 /NS /NC /NFL /NDL /NP > nul

exit /B 0