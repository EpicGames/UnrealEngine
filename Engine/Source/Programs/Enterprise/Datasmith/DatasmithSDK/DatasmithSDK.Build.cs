// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DatasmithSDK : ModuleRules
	{
		public DatasmithSDK(ReadOnlyTargetRules Target)
			: base(Target)
		{
			CppStandard = CppStandardVersion.Cpp20;

			PublicIncludePathModuleNames.Add("Launch");

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DatasmithCore",
					"DatasmithExporter",
					"DatasmithExporterUI",
					// Network layer
					"UdpMessaging",
				}
			);
		}
	}
}