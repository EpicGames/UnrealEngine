// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;
using System.IO;
using System;
using UnrealBuildTool.Rules;

public delegate void PostCopyFilesFunc(string SrcPath, string DestPath, string Files);
public delegate void PostCopyDirFunc(string SrcPath, string DestPath);

[SupportedPlatforms("Win64", "Mac", "Linux")]
public class DatasmithSDKTarget : TargetRules
{
	public DatasmithSDKTarget(TargetInfo Target)
		: base(Target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		DefaultBuildSettings = BuildSettingsVersion.Latest;
		SolutionDirectory = "Programs/Datasmith";
		bBuildInSolutionByDefault = false;

		LaunchModuleName = "DatasmithSDK";
		ExeBinariesSubFolder = "DatasmithSDK";

		ExtraModuleNames.AddRange( new string[] { "DatasmithCore", "DatasmithExporter" } );

		LinkType = TargetLinkType.Monolithic;
		bShouldCompileAsDLL = true;

		bBuildDeveloperTools = false;
		bBuildWithEditorOnlyData = true;
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = true;
		bCompileICU = false;
		bUsesSlate = false;
		DebugInfo = DebugInfoMode.Full;
		bUsePDBFiles = true;
		bHasExports = true;
		bIsBuildingConsoleApplication = true;

		KeyValuePair<string, string>[] FilesToCopy = new KeyValuePair<string, string>[]
		{

		};

		PostCopyFilesFunc PostBuildCopy = PostBuildCopyWin64;
		if (Platform != UnrealTargetPlatform.Win64)
		{
			List<string> Directories = new List<string>(
				new string[]
				{
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Documentation/",
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Public/",
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Private/",
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Engine/",
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Engine/Shaders/StandaloneRenderer/D3D/",
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Engine/Content/",
				}
			);

			PostBuildSteps.Add(string.Format("echo Creating directories to export to\n"));
			foreach (string Directory in Directories)
			{
				PostBuildSteps.Add(string.Format("mkdir -p {0}\n", Directory));
			}

			PostBuildCopy = PostBuildCopyUnix;
		}

		AddPostBuildSteps();

		// Enable UDP in shipping (used by DirectLink)
		if (BuildEnvironment == TargetBuildEnvironment.Unique)
		{
			GlobalDefinitions.Add("ALLOW_UDP_MESSAGING_SHIPPING=1"); // bypasses the 'if shipping' of UdpMessagingModule.cpp
			GlobalDefinitions.Add("PLATFORM_SUPPORTS_MESSAGEBUS=1"); // required to enable the default MessageBus in MessagingModule.cpp
		}
	}
	public void PostBuildCopyWin64(string SrcPath, string DestPath, string Files)
	{
		PostBuildSteps.Add(string.Format("echo Copying {0}{2} to {1}", SrcPath, DestPath, Files));
		PostBuildSteps.Add(string.Format(@"call $(EngineDir)\Source\Programs\Enterprise\Datasmith\DatasmithSDK\CopyCmd.bat {0} {1} {2}", SrcPath, DestPath, Files));
	}

	public void PostBuildCopyUnix(string SrcPath, string DestPath, string Files)
	{
		SrcPath = SrcPath.Replace("\\", "/");
		DestPath = DestPath.Replace("\\", "/");
		PostBuildSteps.Add(string.Format("echo Copying \"{0}\" to {1}\n", SrcPath, DestPath));
		// || true to clear error code when source not found
		PostBuildSteps.Add(string.Format("cp -R -f {0}/{2} {1} || true\n", SrcPath, DestPath, Files));
	}

	public void PostBuildCopyUnix(string SrcPath, string DestPath)
	{
		SrcPath = SrcPath.Replace("\\", "/");
		DestPath = DestPath.Replace("\\", "/");
		PostBuildSteps.Add(string.Format("echo Copying \"{0}\" to {1}\n", SrcPath, DestPath));
		// || true to clear error code when source not found
		PostBuildSteps.Add(string.Format("cp -R -f {0} {1} || true\n", SrcPath, DestPath));
	}

	public void AddPostBuildSteps()
	{
		PostCopyFilesFunc PostBuildCopy = PostBuildCopyWin64;

		string SrcDatasmithSDK = @"$(EngineDir)\Source\Programs\Enterprise\Datasmith\DatasmithSDK";
		string DstDatasmithSDK = @"$(EngineDir)\Binaries\$(TargetPlatform)\DatasmithSDK";

		if (Platform != UnrealTargetPlatform.Win64)
		{
			List<string> Directories = new List<string>(
				new string[]
				{
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Documentation/",
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Public/",
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Private/",
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Engine/Binary",
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Engine/Content/",
					@"$(EngineDir)/Binaries/$(TargetPlatform)/DatasmithSDK/Engine/Shaders/StandaloneRenderer/",
				}
			);

			PostBuildSteps.Add(string.Format("echo Creating directories to export to\n"));
			foreach (string Directory in Directories)
			{
				PostBuildSteps.Add(string.Format("mkdir -p {0}\n", Directory));
			}

			PostBuildCopy = PostBuildCopyUnix;

			PostBuildCopyUnix(@"$(EngineDir)/Content/Slate", Path.Combine(DstDatasmithSDK, "Engine/Content/"));
			PostBuildCopyUnix(@"$(EngineDir)/Shaders/StandaloneRenderer/OpenGL", Path.Combine(DstDatasmithSDK, "Engine/Shaders/StandaloneRenderer"));
		}
		else
		{
			PostBuildSteps.Add(string.Format("mkdir {0}\n", Path.Combine(DstDatasmithSDK, "Engine\\Content\\Binary")));
			PostBuildCopyWin64(@"$(EngineDir)\Content\Slate\", Path.Combine(DstDatasmithSDK, "Engine\\Content\\"), "");
			PostBuildCopyWin64(@"$(EngineDir)\Shaders\StandaloneRenderer\D3D\", Path.Combine(DstDatasmithSDK, "Engine\\Shaders\\StandaloneRenderer\\D3D\\"), "");
		}

		(string, string, string)[] FilesToCopy = new (string, string, string)[]
		{
			(Path.Combine(SrcDatasmithSDK,"Documentation\\"), Path.Combine(DstDatasmithSDK,"Documentation\\"), "*.*"),
			(@"$(EngineDir)\Source\Runtime\Datasmith\DatasmithCore\Public\", Path.Combine(DstDatasmithSDK,"Public\\"), "*.h"),
			(@"$(EngineDir)\Source\Runtime\Datasmith\DirectLink\Public\", Path.Combine(DstDatasmithSDK,"Public\\"), "*.h"),
			(@"$(EngineDir)\Source\Developer\Datasmith\DatasmithExporter\Public\", Path.Combine(DstDatasmithSDK,"Public\\"), "*.h"),
			(@"$(EngineDir)\Source\Developer\Datasmith\DatasmithExporterUI\Public\", Path.Combine(DstDatasmithSDK,"Public\\"), "*.h"),
			(@"$(EngineDir)\Intermediate\Build\Win64\DatasmithSDK\Inc\DatasmithCore\UHT\", Path.Combine(DstDatasmithSDK,"Public\\"), "*.generated.h"),
			(@"$(EngineDir)\Extras\VisualStudioDebugging\", DstDatasmithSDK, "Unreal.natvis"),
			(@"$(EngineDir)\Source\Runtime\TraceLog\Public\", Path.Combine(DstDatasmithSDK,"Private\\"), "*.h"),
			(@"$(EngineDir)\Source\Runtime\TraceLog\Public\", Path.Combine(DstDatasmithSDK,"Private\\"), "*.inl"),
			(@"$(EngineDir)\Source\Runtime\Messaging\Public\", Path.Combine(DstDatasmithSDK,"Private\\"), "*.h"),
			(@"$(EngineDir)\Source\Runtime\Messaging\Public\", Path.Combine(DstDatasmithSDK,"Private\\"), "*.inl"),
			(@"$(EngineDir)\Source\Runtime\AutoRTFM\Public\", Path.Combine(DstDatasmithSDK,"Private\\"), "*.h"),
			(@"$(EngineDir)\Source\Runtime\Core\Public\", Path.Combine(DstDatasmithSDK,"Private\\"), "*.h"),
			(@"$(EngineDir)\Source\Runtime\Core\Public\", Path.Combine(DstDatasmithSDK,"Private\\"), "*.inl"),
			(@"$(EngineDir)\Source\Runtime\CoreUObject\Public\", Path.Combine(DstDatasmithSDK,"Private\\"), "*.h"),
			(@"$(EngineDir)\Source\Runtime\CoreUObject\Public\", Path.Combine(DstDatasmithSDK,"Private\\"), "*.inl"),
			(@"$(EngineDir)\Intermediate\Build\Win64\DatasmithSDK\Inc\CoreUObject\UHT\", Path.Combine(DstDatasmithSDK,"Private\\UObject\\"), "*.generated.h"),
		};

		foreach ((string, string, string) ToCopy in FilesToCopy)
		{
			PostBuildCopy(ToCopy.Item1, ToCopy.Item2, ToCopy.Item3);
		}
	}
}
