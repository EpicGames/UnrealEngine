# Welcome to the Datasmith SDK                                                      {#mainpage}

The Datasmith SDK helps C++ programmers generate files that are compatible with the [Datasmith](https://docs.unrealengine.com/en-US/Studio/Datasmith/index.html) scene import tools built into Unreal Engine.

Epic Games uses this SDK as a basis for the [exporter plugins](https://www.unrealengine.com/en-US/studio/downloads) that we ship for Autodesk 3ds Max, Autodesk Revit, and Trimble Sketchup Pro. You can use this SDK to make a different source of 3D scene data capable of exporting data in the same format.

> **Note:**
> 
> This SDK is not final. The APIs are subject to change in future releases, and we do not guarantee backward compatibility for any `.udatasmith` files that you generate using this version. You should feel free to use this SDK if it helps you to take advantage of the Datasmith import pipeline for bringing data into the Unreal Engine. However, keep in mind that you may need to change your code in future in order for the application you build to continue to work with newer versions of the Unreal Engine.
>
> The Datasmith file format is not intended to be a generic format for manipulating and translating 3D data between any applications. Its internal scene structure is tightly coupled to the way the Unreal Engine represents 3D data internally. Therefore, it is primarily intended as a means to express scene data and structure in a way that makes it easy for the downstream Datasmith importer to recreate the same structure inside the Unreal Engine. If you need a generic scene representation for interoperability between applications, consider other options such as [USD](https://graphics.pixar.com/usd/docs/index.html), [FBX](https://help.autodesk.com/view/FBX/2019/ENU/), or [glTF](https://www.khronos.org/gltf/).

### Using this documentation

This documentation contains step-by-step guides to getting started with the SDK:

-   \ref projectsetup "Setting Up your Project": How to set up your C++ compiler and linker to work with the SDK header files and library.
-   \ref exporting "Creating a Datasmith File": How to use the classes and functions exposed by the SDK to generate a `.udatasmith` file that contains the scene data you want to bring into the Unreal Engine.

It also contains code examples that illustrate the process described in the step-by-step guides:

-   \ref DatasmithExportBasicSample.cpp : A relatively simple illustration of the process described in \ref 3.ExportToDatasmith.md "Creating a Datasmith File", that shows the entire workflow of creating a `.udatasmith` file from beginning to end.
-   \ref DatasmithExportHelper.cpp : Shows how to use the `FDatasmithMesh` class to pass the Datasmith SDK the vertices, faces, and other information about the geometric objets in your scene.

Finally, it contains a complete reference to all the classes and functions offered by the SDK. You can use the navigational pages like the [Class Hierarchy](hierarchy.html) to browse the API elements, or use the search bar at the top right to find a specific API element by name.