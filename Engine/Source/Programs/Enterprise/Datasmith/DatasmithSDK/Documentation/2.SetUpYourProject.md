# Setting Up your Project                                                      {#projectsetup}

In order to use the classes and functions in the Datasmith SDK, you'll need to tell your compiler and linker where to find the SDK's header files and compiled library, and you'll need to set up some pre-processor definitions.

>   **Tip:** If you create a new project in Visual Studio that uses the Datasmith SDK inside the same solution that contains the Unreal Engine source code (`UE4.sln`), the include directories and pre-processor definitions below are taken care of for you automatically by the Unreal Build Tool. However, most customers prefer to integrate the Datasmith SDK library into a standalone application outside of the Unreal Engine solution. In this case, you'll need to set these values yourself in your own Visual Studio or other IDE project.

1.  Install Unreal Engine to a stable location on your computer.

    You can use the binary version that you install from the Epic Games Launcher, or you can download the source code from GitHub. Both have the header files you'll need. However, if you choose the source code distribution, you'll need to generate the `UE4.sln` file and build the `Programs/Enterprise/DatasmithSDK` project.

2.  Add the following additional include directories for your project. You'll find all these folders within your Unreal Engine installation directory.

        DatasmithSDK\Public
        Engine\Source\Runtime\Core\Public
        Engine\Source\Runtime\CoreUObject\Public
        Engine\Source\Runtime\Projects\Public
        Engine\Source\Runtime\TraceLog\Public

3.  Add the following pre-processor definitions for your project:

        UE_BUILD_DEVELOPMENT=1
        UE_BUILD_MINIMAL=1
        WITH_EDITOR=0
        WITH_EDITORONLY_DATA=1
        WITH_SERVER_CODE=1
        WITH_ENGINE=0
        WITH_UNREAL_DEVELOPER_TOOLS=0
        WITH_PLUGIN_SUPPORT=0
        IS_MONOLITHIC=1
        IS_PROGRAM=1
        PLATFORM_WINDOWS=1
        CORE_API=DLLIMPORT
        COREUOBJECT_API=DLLIMPORT
        DATASMITHEXPORTER_API=DLLIMPORT
        DATASMITHCORE_API=DLLIMPORT
        DIRECTLINK_API=DLLIMPORT
        WIN32=1
        _WIN32_WINNT=0x0601
        WINVER=0x0601
        UNICODE
        _UNICODE
        UE_BUILD_DEVELOPMENT_WITH_DEBUGGAME=0
        UBT_COMPILED_PLATFORM=Windows

4.  Add the following additional library directories for your linker:

        Engine\Binaries\Win64\DatasmithSDK

5.  Add the following additional library dependency for your linker:

        DatasmithSDK.lib

6.  When you run your compiled application, it will need to load the `Engine\Binaries\Win64\DatasmithSDK\DatasmithSDK.dll` file. Either copy this file to the directory that contains your application’s executable file, or configure your application to load the `.dll` file from a different location.



