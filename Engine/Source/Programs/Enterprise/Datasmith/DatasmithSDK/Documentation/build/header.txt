<!-- HTML header for doxygen 1.8.16-->
<!DOCTYPE html PUBLIC "-//W3C//DTD XHTML 1.0 Transitional//EN" "https://www.w3.org/TR/xhtml1/DTD/xhtml1-transitional.dtd">
<html xmlns="http://www.w3.org/1999/xhtml">
<head>
<meta http-equiv="Content-Type" content="text/xhtml;charset=UTF-8"/>
<meta http-equiv="X-UA-Compatible" content="IE=9"/>
<meta name="generator" content="Doxygen $doxygenversion"/>
<meta name="viewport" content="width=device-width, initial-scale=1"/>
<!--BEGIN PROJECT_NAME--><title>$projectname: $title</title><!--END PROJECT_NAME-->
<!--BEGIN !PROJECT_NAME--><title>$title</title><!--END !PROJECT_NAME-->
<link href="$relpath^tabs.css" rel="stylesheet" type="text/css"/>
<script type="text/javascript" src="$relpath^jquery.js"></script>
<script type="text/javascript" src="$relpath^dynsections.js"></script>
$treeview
$search
$mathjax
<link href="$relpath^$stylesheet" rel="stylesheet" type="text/css" />
$extrastylesheet
</head>
<body>
<div id="top"><!-- do not remove this div, it is closed by doxygen! -->

<!--BEGIN TITLEAREA-->
<div id="titlearea">

<div id="head"></div>
<script>
$(document).ready(function () {
    $.ajax({
        url: "https://www.unrealengine.com/epic-games-header?useServerUrl=true&lang=en-US",
        context: document.body
    }).done(function (data) {
        $("#head").html(data);
    }).fail(function( jqXHR, textStatus, errorThrown ) {
	});
});

window._egNavAfterConfigSet = function() {
	_epicGamesNav.hideSignIn = true;
	_epicGamesNav.hideLocale = true;
};
</script>

</div>
<!--END TITLEAREA-->
<!-- end header part -->
