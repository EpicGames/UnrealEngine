<html>
<head>
<meta http-equiv="Refresh" content="0; url=./html/index.html" />
<title>Welcome to the Datasmith SDK documentation</title>
</head>
<body>
<p>Welcome to the Datasmith SDK documentation.</p>
<p>Click <a href="./html/index.html">here</a> if you are not redirected automatically.
</body>