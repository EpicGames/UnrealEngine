// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSDK.h"

int32 RunDatasmithSDK(const TCHAR* /*Commandline*/)
{
	return 0;
}



#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"

#include "DatasmithExporterManager.h"
#include "CoreGlobals.h"

BOOL WINAPI DllMain(
	HINSTANCE HinstDLL,
	DWORD     FdwReason,
	LPVOID    LpvReserved
)
{
	if (FdwReason == DLL_PROCESS_DETACH)
	{
		// When a process is unloading the DLL, shut down the Datasmith exporter module.
		FDatasmithExporterManager::Shutdown();

		// Previous Shutdown call is a no-op when the corresponding Initialize was never called (eg. dll loaded but SDK not really used).
		// We have to make sure IsEngineExitRequested() is true before the dll is unloaded as some static destructors rely on that flag to behave properly.
		//
		// The specific case handled here:
		//     FSparseDelegateStorage::SparseDelegateObjectListener dtr uses a maybe-invalid critical-section,
		//     and rely on IsEngineExitRequested() to behave correctly. We enforce that when the dll unloads
		if (!IsEngineExitRequested())
		{
			RequestEngineExit(TEXT("DLL_PROCESS_DETACH received"));
		}
	}

	return TRUE;
}

// End Datasmith platform include guard.
#include "Windows/HideWindowsPlatformTypes.h"
#endif

