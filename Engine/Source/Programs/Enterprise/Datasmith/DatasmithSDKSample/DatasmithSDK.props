<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="4.0" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <ImportGroup Label="PropertySheets" />
  <PropertyGroup Label="UserMacros">
    <DatasmithSDKPath>$(ProjectDir)..\..\..\..\..\Binaries\Win64\DatasmithSDK\</DatasmithSDKPath>
    <GuidelinesSupportLibrary>$(ProjectDir)..\..\..\..\..\Source\ThirdParty\GuidelinesSupportLibrary\GSL-1144\include</GuidelinesSupportLibrary>
  </PropertyGroup>
  <ItemDefinitionGroup>
    <ClCompile>
      <LanguageStandard>stdcpp20</LanguageStandard>
      <PreprocessorDefinitions>UE_SERVER=0;UE_GAME=0;WITH_SERVER_CODE=0;UE_MERGED_MODULES=0;WITH_FIXED_TIME_STEP_SUPPORT=1;UE_ALLOW_EXEC_COMMANDS=1;UE_BUILD_DEVELOPMENT=1;UE_BUILD_MINIMAL=1;WITH_EDITOR=0;WITH_EDITORONLY_DATA=1;WITH_SERVER_CODE=1;WITH_ENGINE=0;WITH_UNREAL_DEVELOPER_TOOLS=0;WITH_PLUGIN_SUPPORT=0;IS_MONOLITHIC=1;IS_PROGRAM=1;PLATFORM_WINDOWS=1;PLATFORM_MICROSOFT=1;CORE_API=DLLIMPORT;CORE_NON_ATTRIBUTED_API=;COREUOBJECT_API=DLLIMPORT;COREUOBJECT_NON_ATTRIBUTED_API=;DATASMITHEXPORTER_API=DLLIMPORT;DATASMITHCORE_API=DLLIMPORT;DIRECTLINK_API=DLLIMPORT;UE_BUILD_DEVELOPMENT_WITH_DEBUGGAME=0;UBT_COMPILED_PLATFORM=Windows;WINVER=0x0601;PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS=1;%(PreprocessorDefinitions)</PreprocessorDefinitions>
      <AdditionalIncludeDirectories>$(DatasmithSDKPath)Public;$(GuidelinesSupportLibrary);$(DatasmithSDKPath)Private;$(DatasmithSDKPath)Private\Windows;%(AdditionalIncludeDirectories)</AdditionalIncludeDirectories>
    </ClCompile>
    <Link>
      <AdditionalDependencies>DatasmithSDK.lib;%(AdditionalDependencies)</AdditionalDependencies>
      <AdditionalLibraryDirectories>$(DatasmithSDKPath);%(AdditionalLibraryDirectories)</AdditionalLibraryDirectories>
    </Link>
  </ItemDefinitionGroup>
  <ItemGroup>
    <BuildMacro Include="DatasmithSDKPath">
      <Value>$(DatasmithSDKPath)</Value>
    </BuildMacro>
  </ItemGroup>
  <ItemGroup>
    <Natvis Include="$(DatasmithSDKPath)Unreal.natvis" />
  </ItemGroup>
</Project>