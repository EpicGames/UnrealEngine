// Copyright Epic Games, Inc. All Rights Reserved.

#include "pch.h"

#include "ScenesManager.h"

// datasmith-directlink
#include "DatasmithCore.h"
#include "DatasmithExporterManager.h"
#include "DirectLinkEndpoint.h"
#include "DirectLinkLog.h"

// unreal
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "UObject/UObjectGlobals.h"

// std
#include <iostream>
#include <sstream>
#include <string>
#include <thread>


TUniquePtr<DirectLink::FEndpoint> EndpointPtr;

DirectLink::FEndpoint& GetDirectLinkEndpoint()
{
	if (!EndpointPtr)
	{
		EndpointPtr = MakeUnique<DirectLink::FEndpoint>(TEXT("Datasmith SDK Sample"));
		std::cout << "create endpoint...";
		FPlatformProcess::Sleep(1.f);
		std::cout << " done\n";
	}

	return *EndpointPtr;
}

void ShutdownEndpoint()
{
	if (EndpointPtr)
	{
		EndpointPtr.Reset();
		FPlatformProcess::Sleep(1.f);
	}
}


void DirectlinkExposeScene(TSharedPtr<ISampleScene> Sample)
{
	if (Sample)
	{
		if (TSharedPtr<IDatasmithScene> Scene = Sample->Export())
		{
			std::wcout << "Expose scene [" << *Sample->GetName() << "] with directlink...\n";

			// fix the ResourcePath for directlink:
			// Datasmith needs to find the assets files, they are stored in the Export folder.
			Scene->SetResourcePath(*FPaths::ConvertRelativePathToFull(GetExportPath()));

			// Register a new source in directlink
			DirectLink::FEndpoint& DirectLinkEndpoint = GetDirectLinkEndpoint();
			DirectLink::FSourceHandle SourceHandle = DirectLinkEndpoint.AddSource(Sample->GetName(), DirectLink::EVisibility::Public);

			// Fill that source with a snapshot of the scene
			bool bSnapshotNow = true;
			DirectLinkEndpoint.SetSourceRoot(SourceHandle, Scene.Get(), bSnapshotNow);
		}
	}
}

bool Execute(const FString& Cmd, const FString& Line=TEXT(""))
{
	if (Cmd == "list" || Cmd == "l")
	{
		const auto& AllScenes = FScenesManager::GetAllScenes();
		int32 Index = 0;
		for (const auto& Scene : AllScenes)
		{
			std::wcout << "Scene [" << Index++ << "]: '" << *Scene->GetName() << "'\n";
			std::wcout << "    " << *Scene->GetDescription() << "\n";
			std::wcout << "    tags: " << *FString::Join(Scene->GetTags(), TEXT(", ")) << "\n";
			std::wcout << "\n";
		}
		return true;
	}

	else if (Cmd == "export" || Cmd == "e")
	{
		FString Names_str = Line.RightChop(Cmd.Len()+1);
		TArray<FString> Names;
		Names_str.ParseIntoArrayWS(Names, TEXT(","));

 		for (const FString& Name : Names)
		{
			if (TSharedPtr<ISampleScene> Scene = FScenesManager::GetScene(Name))
			{
				std::wcout << "Export scene [" << *Scene->GetName() << "]\n";
				Scene->Export();
			}
		}
		return true;
	}

	else if (Cmd == "exportall" || Cmd == "ea")
	{
		for (const TSharedPtr<ISampleScene>& Scene : FScenesManager::GetAllScenes())
		{
			std::wcout << "Export scene [" << *Scene->GetName() << "]\n";
			Scene->Export();
		}
		return true;
	}

	else if (Cmd == "source" || Cmd == "s")
	{
		FString Names_str = Line.RightChop(Cmd.Len()+1);
		TArray<FString> Names;
		Names_str.ParseIntoArrayWS(Names, TEXT(","));

		for (const FString& Name : Names)
		{
			DirectlinkExposeScene(FScenesManager::GetScene(Name));
		}
		return true;
	}

	else if (Cmd == "sourceall" || Cmd == "sa")
	{
		for (const TSharedPtr<ISampleScene>& Scene : FScenesManager::GetAllScenes())
		{
			DirectlinkExposeScene(Scene);
		}
		return true;
	}

	else if (Cmd == "dlstatus" || Cmd == "dls")
	{
		DirectLink::FEndpoint& DirectLinkEndpoint = GetDirectLinkEndpoint();
		DirectLink::FRawInfo RawInfo = DirectLinkEndpoint.GetRawInfoCopy();

		int EndpointIndex = 0;
		for (const TPair<FMessageAddress, DirectLink::FRawInfo::FEndpointInfo>& Pair : RawInfo.EndpointsInfo)
		{
			const DirectLink::FRawInfo::FEndpointInfo& EndpointInfo = Pair.Value;

			FString Str = FString::Printf(TEXT("- %s Endpoint [%d] '%s':\n  - machine: %s: %s\n  - exe: %s (%d)\n  - version: %s\n")
				, Pair.Key == RawInfo.ThisEndpointAddress ? TEXT("THIS") : TEXT("CONNECTED")
				, EndpointIndex++, *EndpointInfo.Name
				, EndpointInfo.bIsLocal ? TEXT("local") : TEXT("remote"), *EndpointInfo.ComputerName
				, *EndpointInfo.ExecutableName, EndpointInfo.ProcessId
				, *EndpointInfo.Version.ToString()
			);
			std::wcout << *Str;

			int ScrIndex = 0;
			std::cout << "  - " << EndpointInfo.Sources.Num() << " source(s)\n";
			for (const DirectLink::FRawInfo::FDataPointId& DataPointId : EndpointInfo.Sources)
			{
				Str = FString::Printf(TEXT("  - Source [%d] '%s':\n    - visibility: %s\n    - guid: %s\n")
					, ScrIndex++, *DataPointId.Name
					, DataPointId.bIsPublic ? TEXT("public") : TEXT("private")
					, *DataPointId.Id.ToString()
				);
				std::wcout << *Str;
			}

			int DstIndex = 0;
			std::cout << "  - " << EndpointInfo.Destinations.Num() << " destination(s)\n";
			for (const DirectLink::FRawInfo::FDataPointId& DataPointId : EndpointInfo.Destinations)
			{
				Str = FString::Printf(TEXT("  - Dest [%d] '%s':\n    - visibility: %s\n    - guid: %s\n")
					, DstIndex++, *DataPointId.Name
					, DataPointId.bIsPublic ? TEXT("public") : TEXT("private")
					, *DataPointId.Id.ToString()
				);
				std::wcout << *Str;
			}
		}
		// list other endpoint
		// list their source
		return true;
	}
	return false;
}

void InteractiveLoop()
{
	std::cout << R"(
Commands:
  list (l)                   -> list available scenes
  quit (q)                   -> exit the interactive loop

File based:
  export (e) name[,name2...] -> export specified scenes (names or indices) as udatasmith scenes
  exportall (ea)             -> export all scenes as files

Directlink:
  dlstatus (dls)             -> dump the directlink swarm
  source (s) name[,name2...] -> expose scenes (names or indices) as directlink sources
  sourceall (sa)             -> expose all the scenes as directlink sources
)";/*
//todo : remove sources
*/
	while (true) // if stream broken
	{
		std::cout << "\n>>> ";
		std::string line;
		std::getline(std::cin, line);
		if (!std::cin.good())
		{
			std::cout << "std input broken: exit...\n";
			break;
		}

		std::istringstream iss(line);
		std::string cmd;
		iss >> cmd;
		FString Cmd(cmd.c_str()); // for case independent comparison
		FString Line(line.c_str()); // for case independent comparison

		if (Execute(Cmd, Line))
		{
			continue;
		}

		else if (Cmd == "quit" || Cmd == "q")
		{
			std::cerr << "bye\n";
			break;
		}

		std::cerr << "unhandled cmd: " << cmd << "\n";
	}
}


// Quite hacky... Currently the main thread doesn't log anything, so the logs are never flushed. This loop ensure logs are flushed correctly.
static bool FlushLogsFlag = true;
void FlushLogLoop()
{
	GLog->SetCurrentThreadAsPrimaryThread();
	while (FlushLogsFlag)
	{
		FPlatformProcess::Sleep(0.1f);
		if(GLog)
		{
			GLog->Flush();
		}
	}
}


int main(int argc, char** argv)
{
	std::wcout << "== Datasmith SDK Sample ==\n";
	std::wcout << "cwd:" << *FPlatformProcess::GetCurrentWorkingDirectory() << std::endl;

	// #ue_todo print usage
	FString Cmd = FString::Join(TArrayView<char*, int32>{argv, argc}, TEXT(" "));
	bool bVerbose = FParse::Param(*Cmd, TEXT("verbose"));
	bool bIsInteractive = FParse::Param(*Cmd, TEXT("i"));

	// Initialize the Datasmith SDK and its dependencies. Required before any operation with datasmith.
	FDatasmithExporterManager::FInitOptions InitOptions;
	InitOptions.bSuppressLogs = !bVerbose;
	InitOptions.bEnableMessaging = true; // required in order to use DirectLink
	FDatasmithExporterManager::Initialize(InitOptions);

	// in this sample application, we manually flush the logs
	std::thread FlushLogThread(FlushLogLoop);

	// optionally set another export path
	FString ExplicitExportPath;
	if (FParse::Value(*Cmd, TEXT("-exportpath"), ExplicitExportPath))
	{
		SetExportPath(ExplicitExportPath);
	}

#if !NO_LOGGING
	// Increase verbosity for dbg purpose
	LogDatasmith.SetVerbosity(ELogVerbosity::All);
	if (bVerbose)
	{
		LogDirectLink.SetVerbosity(ELogVerbosity::All);
		LogDirectLinkNet.SetVerbosity(ELogVerbosity::All);
		StaticExec(nullptr, TEXT("log LogMessaging All"));
		StaticExec(nullptr, TEXT("log LogUdpMessaging All"));
	}
#endif

	// In this sample program, we run an optional interactive loop to play with various scenes.
	FString Exec;
	if (bIsInteractive)
	{
		GetDirectLinkEndpoint(); // This call makes a DirectLink Endpoint. This connects this process to other endpoints.
		InteractiveLoop();
	}
	else if (FParse::Value(*Cmd, TEXT("-exec"), Exec))
	{
		std::wcout << TEXT("execute command ") << *Exec << TEXT("\n");
		int32 Pos = Cmd.Find(TEXT("-exec"));
		FString Line = Cmd.RightChop(Pos+6);
		if (!Execute(Exec, Line))
		{
			std::wcerr << TEXT("unhandled cmd: ") << *Exec << TEXT("\n");
		}
	}
	else
	{
		// when the interactive mode is not specified, we export the Default scene as file
		if (TSharedPtr<ISampleScene> DefaultSampleScene = FScenesManager::GetScene("Default"))
		{
			DefaultSampleScene->Export();
			std::wcout << TEXT("Default scene exported to ") << *GetExportPath() << TEXT("\n");
		}
	}

	// end log hack thread
	FlushLogsFlag = false;
	FlushLogThread.join();
	GLog->SetCurrentThreadAsPrimaryThread();

	// Shutdown the SDK. Must be called before the dll/executable exits in order to prevent ungraceful exit
	ShutdownEndpoint();
	FDatasmithExporterManager::Shutdown();
}
