// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithMesh.h"

static void CreateSimpleBox(FDatasmithMesh& Mesh)
{
	int32 NumFaces = 6; //six quads on a box
	int32 NumTriangles = 12; //everything should be converted to triangles
	int32 NumVertex = 8; //eight shared vertex to create all the triangles

	//set the num of needed geometry vertex and faces
	Mesh.SetVerticesCount(NumVertex);
	Mesh.SetFacesCount(NumTriangles);

	// Set the position of each vertex
	Mesh.SetVertex(0, -50.0, -50.0, 0.0);
	Mesh.SetVertex(1, -50.0, 50.0, 0.0);
	Mesh.SetVertex(2, 50.0, 50.0, 0.0);
	Mesh.SetVertex(3, 50.0, -50.0, 0.0);
	Mesh.SetVertex(4, -50.0, -50.0, -100.0);
	Mesh.SetVertex(5, -50.0, 50.0, -100.0);
	Mesh.SetVertex(6, 50.0, 50.0, -100.0);
	Mesh.SetVertex(7, 50.0, -50.0, -100.0);


	Mesh.AddUVChannel();

	// Set the num of needed UV vertex
	Mesh.SetUVCount(0, 4);

	//set the actual position of each uv vertex
	Mesh.SetUV(0, 0, 0.0, 0.0);
	Mesh.SetUV(0, 1, 0.0, 1.0);
	Mesh.SetUV(0, 2, 1.0, 1.0);
	Mesh.SetUV(0, 3, 1.0, 0.0);


	// Create polygons. Assign texture and texture UV indices.

	//We'll create three quads with material 0 and three quads with material 1

	//SetFace: face index, vert1, vert2, vert3, FaceId
	int32 FaceId = 0;
	Mesh.SetFace(0, 1, 2, 0, FaceId); //top face
	Mesh.SetFace(1, 3, 0, 2, FaceId);
	Mesh.SetFace(2, 6, 5, 4, FaceId); //bottom face
	Mesh.SetFace(3, 4, 7, 6, FaceId);
	Mesh.SetFace(4, 4, 5, 1, FaceId); //left face
	Mesh.SetFace(5, 1, 0, 4, FaceId);

	FaceId = 1;
	Mesh.SetFace(6, 6, 7, 2, FaceId); //right face
	Mesh.SetFace(7, 3, 2, 7, FaceId);
	Mesh.SetFace(8, 7, 4, 0, FaceId); //front face
	Mesh.SetFace(9, 0, 3, 7, FaceId);
	Mesh.SetFace(10, 5, 6, 1, FaceId); //back face
	Mesh.SetFace(11, 2, 1, 6, FaceId);


	//set uv indexes for each face vertex
	//top faces
	Mesh.SetFaceUV(0, 0, 1, 2, 0);
	Mesh.SetFaceUV(1, 0, 3, 0, 2);
	//bottom faces
	Mesh.SetFaceUV(2, 0, 2, 1, 0);
	Mesh.SetFaceUV(3, 0, 0, 3, 2);
	//left faces
	Mesh.SetFaceUV(4, 0, 2, 1, 0);
	Mesh.SetFaceUV(5, 0, 0, 3, 2);
	//right faces
	Mesh.SetFaceUV(6, 0, 1, 2, 0);
	Mesh.SetFaceUV(7, 0, 3, 0, 2);
	//front faces
	Mesh.SetFaceUV(8, 0, 2, 1, 0);
	Mesh.SetFaceUV(9, 0, 0, 3, 2);
	//back faces
	Mesh.SetFaceUV(10, 0, 1, 2, 0);
	Mesh.SetFaceUV(11, 0, 3, 0, 2);

	//set normals
	Mesh.SetNormal(0, 0.0, 0.0, 1.0); //top faces
	Mesh.SetNormal(1, 0.0, 0.0, 1.0);
	Mesh.SetNormal(2, 0.0, 0.0, 1.0);
	Mesh.SetNormal(3, 0.0, 0.0, 1.0);
	Mesh.SetNormal(4, 0.0, 0.0, 1.0);
	Mesh.SetNormal(5, 0.0, 0.0, 1.0);
	Mesh.SetNormal(6, 0.0, 0.0, -1.0); //bottom faces
	Mesh.SetNormal(7, 0.0, 0.0, -1.0);
	Mesh.SetNormal(8, 0.0, 0.0, -1.0);
	Mesh.SetNormal(9, 0.0, 0.0, -1.0);
	Mesh.SetNormal(10, 0.0, 0.0, -1.0);
	Mesh.SetNormal(11, 0.0, 0.0, -1.0);
	Mesh.SetNormal(12, -1.0, 0.0, 0.0); //left faces
	Mesh.SetNormal(13, -1.0, 0.0, 0.0);
	Mesh.SetNormal(14, -1.0, 0.0, 0.0);
	Mesh.SetNormal(15, -1.0, 0.0, 0.0);
	Mesh.SetNormal(16, -1.0, 0.0, 0.0);
	Mesh.SetNormal(17, -1.0, 0.0, 0.0);
	Mesh.SetNormal(18, 1.0, 0.0, 0.0); //right faces
	Mesh.SetNormal(19, 1.0, 0.0, 0.0);
	Mesh.SetNormal(20, 1.0, 0.0, 0.0);
	Mesh.SetNormal(21, 1.0, 0.0, 0.0);
	Mesh.SetNormal(22, 1.0, 0.0, 0.0);
	Mesh.SetNormal(23, 1.0, 0.0, 0.0);
	Mesh.SetNormal(24, 0.0, -1.0, 0.0); //front faces
	Mesh.SetNormal(25, 0.0, -1.0, 0.0);
	Mesh.SetNormal(26, 0.0, -1.0, 0.0);
	Mesh.SetNormal(27, 0.0, -1.0, 0.0);
	Mesh.SetNormal(28, 0.0, -1.0, 0.0);
	Mesh.SetNormal(29, 0.0, -1.0, 0.0);
	Mesh.SetNormal(30, 0.0, 1.0, 0.0); //back faces
	Mesh.SetNormal(31, 0.0, 1.0, 0.0);
	Mesh.SetNormal(32, 0.0, 1.0, 0.0);
	Mesh.SetNormal(33, 0.0, 1.0, 0.0);
	Mesh.SetNormal(34, 0.0, 1.0, 0.0);
	Mesh.SetNormal(35, 0.0, 1.0, 0.0);
}

static void CreateSimplePyramid(FDatasmithMesh& Mesh)
{
	int32 NumFaces = 5; //six quads on a box
	int32 NumTriangles = 6; //everything should be converted to triangles
	int32 NumVertex = 5; //eight shared vertex to create all the triangles

	//set the num of needed geometry vertex and faces
	Mesh.SetVerticesCount(NumVertex);
	Mesh.SetFacesCount(NumTriangles);

	// Set the position of each vertex
	Mesh.SetVertex(0, 0.0, 0.0, 100.0);
	Mesh.SetVertex(1, -50.0, -50.0, 0.0);
	Mesh.SetVertex(2, -50.0, 50.0, 0.0);
	Mesh.SetVertex(3, 50.0, 50.0, 0.0);
	Mesh.SetVertex(4, 50.0, -50.0, 0.0);

	Mesh.SetUVChannelsCount(1);

	// Set the num of needed UV vertex
	Mesh.SetUVCount(0, 4);

	//set the actual position of each uv vertex
	Mesh.SetUV(0, 0, 0.0, 0.0);
	Mesh.SetUV(0, 1, 0.0, 4.0);
	Mesh.SetUV(0, 2, 1.0, 10.0);
	Mesh.SetUV(0, 3, 20.0, 0.0);


	// Create polygons. Assign texture and texture UV indices.

	//We'll create three quads with material 0 and three quads with material 1

	//SetFace: face index, vert1, vert2, vert3, FaceId
	int32 FaceId = 0;
	Mesh.SetFace(0, 3, 2, 1, FaceId); //bottom face
	Mesh.SetFace(1, 1, 4, 3, FaceId);
	Mesh.SetFace(2, 1, 2, 0, FaceId); //left face

	FaceId = 1;
	Mesh.SetFace(3, 2, 3, 0, FaceId); //right face
	Mesh.SetFace(4, 3, 4, 0, FaceId); //front face
	Mesh.SetFace(5, 4, 1, 0, FaceId); //back face


									  //set uv indexes for each face vertex
									  //bottom faces
	Mesh.SetFaceUV(0, 0, 2, 1, 0);
	Mesh.SetFaceUV(1, 0, 0, 3, 2);
	//left faces
	Mesh.SetFaceUV(2, 0, 2, 1, 0);
	//right faces
	Mesh.SetFaceUV(3, 0, 1, 2, 0);
	//front faces
	Mesh.SetFaceUV(4, 0, 2, 1, 0);
	//back faces
	Mesh.SetFaceUV(5, 0, 1, 2, 0);

	//set normals
	Mesh.SetNormal(0, 0.0, 0.0, -1.0); //bottom faces
	Mesh.SetNormal(1, 0.0, 0.0, -1.0);
	Mesh.SetNormal(2, 0.0, 0.0, -1.0);
	Mesh.SetNormal(3, 0.0, 0.0, -1.0);
	Mesh.SetNormal(4, 0.0, 0.0, -1.0);
	Mesh.SetNormal(5, 0.0, 0.0, -1.0);
	Mesh.SetNormal(6, -1.0, 0.0, 0.0); //left faces
	Mesh.SetNormal(7, -1.0, 0.0, 0.0);
	Mesh.SetNormal(8, 0.0, 0.0, 1.0);
	Mesh.SetNormal(9, 1.0, 0.0, 0.0); //right faces
	Mesh.SetNormal(10, 1.0, 0.0, 0.0);
	Mesh.SetNormal(11, 0.0, 0.0, 1.0);
	Mesh.SetNormal(12, 0.0, 1.0, 0.0); //front faces
	Mesh.SetNormal(13, 0.0, 1.0, 0.0);
	Mesh.SetNormal(14, 0.0, 0.0, 1.0);
	Mesh.SetNormal(15, 0.0, -1.0, 0.0); //back faces
	Mesh.SetNormal(16, 0.0, -1.0, 0.0);
	Mesh.SetNormal(17, 0.0, 0.0, 1.0);
}