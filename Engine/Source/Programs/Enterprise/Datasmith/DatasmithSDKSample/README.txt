== Datasmnith SDK Sample ==

Description
This sample project aims to provide a base from which a Datasmith exporter can be built.
In this project, we use the Datasmith SDK to generate some content, expose this content on directlink, and/or export a datasmith scene to disk.

Dependency
- In order to build this project, a Datasmith SDK must be provided. The default configuration searches for an SDK generated by the main Unreal solution (see project DatasmithSDK), but you could edit DatasmithSDK.props to point to another SDK.
- Unreal requires a c++20 compiler