// Copyright Epic Games, Inc. All Rights Reserved.

#include "pch.h"
#include "shared.h"

#include "ScenesManager.h"
#include "MeshUtils.h"

#include "DatasmithSceneExporter.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFilemanager.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithMesh.h"
#include "DatasmithMeshExporter.h"
#include "DatasmithExportOptions.h"
#include "GenericPlatform/GenericPlatformFile.h"

#include "MeshUtils.h"
#include "Containers\UnrealString.h"


struct FActorPropertiesScene : public ISampleScene
{
	virtual FString GetName() const override { return TEXT("ActorProperties");}
	virtual FString GetDescription() const override { return TEXT("Demonstrate the basic actor properties editable with Datasmith."); }
	virtual TArray<FString> GetTags() const override { return {"actors", "visibility", "cast-shadow", "custom-actor", "layer", "tag"}; }
	virtual TSharedPtr<IDatasmithScene> Export() override;
};
REGISTER_SCENE(FActorPropertiesScene);


TSharedRef<IDatasmithCustomActorElement> BuildTextActorElement(const TCHAR* Text)
{
	TSharedRef<IDatasmithCustomActorElement> TextActor = FDatasmithSceneFactory::CreateCustomActor(TEXT("text actor"));
	TextActor->SetClassOrPathName(TEXT("/Script/Engine.TextRenderActor"));
	TextActor->SetRotation(FRotator(90., 90., 0.).Quaternion()); // yzx
	TextActor->SetTranslation(20, 0, 10);

	auto AddProp = [&](const TCHAR* Key, EDatasmithKeyValuePropertyType Type, const TCHAR* Value)
	{
		TSharedPtr<IDatasmithKeyValueProperty> Property = FDatasmithSceneFactory::CreateKeyValueProperty(Key);
		Property->SetValue(Value);
		TextActor->AddProperty(Property);
	};

	AddProp(TEXT("TextRender.Text"), EDatasmithKeyValuePropertyType::String, Text);
	AddProp(TEXT("TextRender.HorizontalAlignment"), EDatasmithKeyValuePropertyType::String, TEXT("EHTA_Right")); // see EHorizTextAligment in \Engine\Source\Runtime\Engine\Classes\Components\TextRenderComponent.h
	AddProp(TEXT("TextRender.WorldSize"), EDatasmithKeyValuePropertyType::Float, TEXT("40"));
	AddProp(TEXT("SpriteScale"), EDatasmithKeyValuePropertyType::Float, TEXT("0"));

	FLinearColor TextColor(0,250,255,180);
	AddProp(TEXT("TextRender.TextRenderColor"), EDatasmithKeyValuePropertyType::Color, *TextColor.ToString());

	return TextActor;
}

TSharedPtr<IDatasmithScene> FActorPropertiesScene::Export()
{
	FDatasmithSceneExporter DatasmithSceneExporter;

	DatasmithSceneExporter.SetName(*GetName());
	DatasmithSceneExporter.SetOutputPath(*GetExportPath());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.CreateDirectoryTree(DatasmithSceneExporter.GetOutputPath()))
	{
		return nullptr;
	}

	TSharedRef<IDatasmithScene> DatasmithScene = FDatasmithSceneFactory::CreateScene(*GetName());
	SetupSharedSceneProperties(DatasmithScene);

	// Creates a mesh asset
	{
		FDatasmithMesh BaseMesh;
		CreateSimpleBox(BaseMesh);

		FDatasmithMeshExporter MeshExporter;
		TSharedPtr<IDatasmithMeshElement> MeshElement = MeshExporter.ExportToUObject(DatasmithSceneExporter.GetAssetsOutputPath(), TEXT("BoxMesh"), BaseMesh, nullptr, EDSExportLightmapUV::Always /** Not used */);

		// Add the mesh to the DatasmithScene
		DatasmithScene->AddMesh(MeshElement);
	}

	// Creates a red material
	{
		TSharedRef<IDatasmithUEPbrMaterialElement> RedMaterial = FDatasmithSceneFactory::CreateUEPbrMaterial(TEXT("Red Material"));
		IDatasmithMaterialExpressionColor* RedColorExpression = RedMaterial->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		RedColorExpression->GetColor() = FLinearColor(0.8f, 0.f, 0.f);
		RedColorExpression->ConnectExpression(RedMaterial->GetBaseColor(), 0);
		DatasmithScene->AddMaterial(RedMaterial);
	}

	SampleUtils::FGridLayout layout({-450, -450, 0});


	auto AddEmptyActor = [&](const TCHAR* Name)
	{
		TSharedRef<IDatasmithActorElement> Actor = FDatasmithSceneFactory::CreateActor(Name);
		Actor->SetTranslation(layout.GetCurrentVector());
		DatasmithScene->AddActor(Actor);
		layout.NextItem();
		return Actor;
	};

	// Creates the mesh actors
	auto AddMeshActor = [&](TSharedPtr<IDatasmithActorElement> Parent, const TCHAR* Name)
	{
		TSharedRef<IDatasmithMeshActorElement> Actor = FDatasmithSceneFactory::CreateMeshActor(Name);
		Actor->SetStaticMeshPathName(TEXT("BoxMesh")); // Assign the geometry with index 0 (the box) to this actor
		Actor->AddMaterialOverride(TEXT("Red Material"), -1); // Since we want a unique material on our actor we'll use -1 as id

		Actor->SetTranslation(layout.GetCurrentVector() + FVector(0, 0, 50));
		Actor->SetScale(FVector(0.5));
		Actor->SetRotation(FQuat::Identity);

		if (Parent)
		{
			Parent->AddChild(Actor);
		}
		layout.NextItem();
		return Actor;
	};

	TSharedPtr<IDatasmithMeshActorElement> Actor;
	TSharedPtr<IDatasmithActorElement> CategoryActor;

	// visibility
	CategoryActor = AddEmptyActor(TEXT("Visibility"));
	CategoryActor->AddChild(BuildTextActorElement(TEXT("Visibility")), EDatasmithActorAttachmentRule::KeepRelativeTransform);
	AddMeshActor(CategoryActor, TEXT("Visibility_true"))->SetVisibility(true);
	AddMeshActor(CategoryActor, TEXT("Visibility_false"))->SetVisibility(false);
	layout.NextLine();

	// shadows
	CategoryActor = AddEmptyActor(TEXT("CastShadow"));
	CategoryActor->AddChild(BuildTextActorElement(TEXT("Shadows")), EDatasmithActorAttachmentRule::KeepRelativeTransform);
	AddMeshActor(CategoryActor, TEXT("CastShadow_true"))->SetCastShadow(true);
	AddMeshActor(CategoryActor, TEXT("CastShadow_false"))->SetCastShadow(false);
	layout.NextLine();

	// layer
	CategoryActor = AddEmptyActor(TEXT("Layers"));
	CategoryActor->AddChild(BuildTextActorElement(TEXT("Layers")), EDatasmithActorAttachmentRule::KeepRelativeTransform);
	AddMeshActor(CategoryActor, TEXT("LayerA"))->SetLayer(TEXT("Layer A"));
	AddMeshActor(CategoryActor, TEXT("LayerA_bis"))->SetLayer(TEXT("Layer A"));
	AddMeshActor(CategoryActor, TEXT("LayerB"))->SetLayer(TEXT("Layer B"));
	layout.NextLine();

	// tag
	CategoryActor = AddEmptyActor(TEXT("Tags"));
	CategoryActor->AddChild(BuildTextActorElement(TEXT("Tags")), EDatasmithActorAttachmentRule::KeepRelativeTransform);
	Actor = AddMeshActor(CategoryActor, TEXT("Tags"));
	Actor->AddTag(TEXT("Tag 0"));
	Actor->AddTag(TEXT("Tag 1"));
	layout.NextLine();

	DatasmithSceneExporter.Export(DatasmithScene);

	return DatasmithScene;
}





