// Copyright Epic Games, Inc. All Rights Reserved.

#include "pch.h"
#include "shared.h"

#include "ScenesManager.h"
#include "MeshUtils.h"

#include "DatasmithSceneExporter.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFilemanager.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithMesh.h"
#include "DatasmithMeshExporter.h"
#include "DatasmithExportOptions.h"
#include "GenericPlatform/GenericPlatformFile.h"

#include "MeshUtils.h"


struct FDefaultScene : public ISampleScene
{
	virtual FString GetName() const override { return TEXT("Default");}
	virtual FString GetDescription() const override { return TEXT("Simple scene sample. Demonstrate the basic setup to export a scene in Datasmith."); }
	virtual TArray<FString> GetTags() const override { return {"static-mesh", "uepbr-material", "area-light"}; }
	virtual TSharedPtr<IDatasmithScene> Export() override;
};
REGISTER_SCENE(FDefaultScene)


TSharedPtr<IDatasmithScene> FDefaultScene::Export()
{
	FDatasmithSceneExporter DatasmithSceneExporter;

	DatasmithSceneExporter.SetName(*GetName());
	DatasmithSceneExporter.SetOutputPath(*GetExportPath());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.CreateDirectoryTree(DatasmithSceneExporter.GetOutputPath()))
	{
		return nullptr;
	}

	TSharedRef<IDatasmithScene> DatasmithScene = FDatasmithSceneFactory::CreateScene(*GetName());
	SetupSharedSceneProperties(DatasmithScene);

	// Set Geolocation to the Scene
	{
		DatasmithScene->SetGeolocationLatitude(10);
		DatasmithScene->SetGeolocationLongitude(20);
		// Skip setting Elevation to test it's absent
	}

	// Creates a mesh asset
	{
		FDatasmithMesh BaseMesh;
		CreateSimpleBox(BaseMesh);

		FDatasmithMeshExporter MeshExporter;

		TSharedPtr<IDatasmithMeshElement> MeshElement = MeshExporter.ExportToUObject(DatasmithSceneExporter.GetAssetsOutputPath(), TEXT("BoxMesh"), BaseMesh, nullptr, EDSExportLightmapUV::Always /** Not used */);

		// Add the mesh to the DatasmithScene
		DatasmithScene->AddMesh(MeshElement);
	}

	// Creates a red material
	{
		TSharedRef<IDatasmithUEPbrMaterialElement> RedMaterial = FDatasmithSceneFactory::CreateUEPbrMaterial(TEXT("Red Material"));

		IDatasmithMaterialExpressionColor* RedColorExpression = RedMaterial->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		RedColorExpression->GetColor() = FLinearColor(0.8f, 0.f, 0.f);

		RedColorExpression->ConnectExpression(RedMaterial->GetBaseColor(), 0);

		DatasmithScene->AddMaterial(RedMaterial);
	}

	// Creates a chrome material
	{
		TSharedRef<IDatasmithUEPbrMaterialElement> ChromeMaterial = FDatasmithSceneFactory::CreateUEPbrMaterial(TEXT("Chrome Material"));

		IDatasmithMaterialExpressionColor* WhiteColorExpression = ChromeMaterial->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		WhiteColorExpression->GetColor() = FLinearColor::White;

		WhiteColorExpression->ConnectExpression(ChromeMaterial->GetBaseColor(), 0);

		IDatasmithMaterialExpressionScalar* Metallic = ChromeMaterial->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		Metallic->GetScalar() = 1.f;

		Metallic->ConnectExpression(ChromeMaterial->GetMetallic(), 0);

		IDatasmithMaterialExpressionScalar* Roughness = ChromeMaterial->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		Roughness->GetScalar() = 0.f;

		Roughness->ConnectExpression(ChromeMaterial->GetRoughness(), 0);

		DatasmithScene->AddMaterial(ChromeMaterial);
	}

	// Creates a blue material
	{
		TSharedRef<IDatasmithUEPbrMaterialElement> BlueMaterial = FDatasmithSceneFactory::CreateUEPbrMaterial(TEXT("Blue Material"));

		IDatasmithMaterialExpressionColor* BlueColorExpression = BlueMaterial->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		BlueColorExpression->GetColor() = FLinearColor(0.f, 0.f, 0.8f);

		BlueColorExpression->ConnectExpression(BlueMaterial->GetBaseColor(), 0);

		IDatasmithMaterialExpressionScalar* Roughness = BlueMaterial->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		Roughness->GetScalar() = 0.2f;

		Roughness->ConnectExpression(BlueMaterial->GetRoughness(), 0);

		DatasmithScene->AddMaterial(BlueMaterial);
	}

	// Creates the mesh actors
	TSharedPtr<IDatasmithMeshActorElement> MeshActorRed;
	{
		MeshActorRed = FDatasmithSceneFactory::CreateMeshActor(TEXT("Sample Actor Red"));
		MeshActorRed->SetStaticMeshPathName(TEXT("BoxMesh")); // Assign the geometry with index 0 (the box) to this actor
		MeshActorRed->AddMaterialOverride(TEXT("Red Material"), -1); // Since we want a unique material on our actor we'll use -1 as id

		MeshActorRed->SetTranslation(FVector(0.f, 0.f, 100.f));
		MeshActorRed->SetScale(FVector::OneVector);
		MeshActorRed->SetRotation(FQuat::Identity);

		DatasmithScene->AddActor(MeshActorRed);

		TSharedRef<IDatasmithMeshActorElement> MeshActorChromeBlue = FDatasmithSceneFactory::CreateMeshActor(TEXT("Sample Actor Green & Blue"));
		MeshActorChromeBlue->SetStaticMeshPathName(TEXT("BoxMesh")); // Assign the geometry with index 0 (the box) to this actor
		MeshActorChromeBlue->AddMaterialOverride(TEXT("Chrome Material"), 0);
		MeshActorChromeBlue->AddMaterialOverride(TEXT("Blue Material"), 1);

		MeshActorChromeBlue->SetTranslation(FVector(0.f, 200.f, 100.f));
		MeshActorChromeBlue->SetScale(FVector(0.75f, 0.75f, 0.75f));
		MeshActorChromeBlue->SetRotation(FRotator(0.f, 45.f, 0.f).Quaternion());

		DatasmithScene->AddActor(MeshActorChromeBlue);
	}

	// Creates a light actor
	{
		TSharedRef<IDatasmithAreaLightElement> AreaLightActor = FDatasmithSceneFactory::CreateAreaLight(TEXT("AreaLight Actor"));
		AreaLightActor->SetTranslation(FVector(0.f, 0.f, 500.f));
		AreaLightActor->SetRotation(FRotator(-90.f, 0.f, 0.f).Quaternion());

		AreaLightActor->SetWidth(800.f);
		AreaLightActor->SetLength(400.f);

		AreaLightActor->SetTemperature(6500.f);
		AreaLightActor->SetUseTemperature(true);

		AreaLightActor->SetIntensity(1000.f);
		AreaLightActor->SetIntensityUnits(EDatasmithLightUnits::Lumens);

		AreaLightActor->SetLightShape(EDatasmithLightShape::Rectangle);

		DatasmithScene->AddActor(AreaLightActor);
	}

	// Add meta data to Sample Actor Red
	{
		TSharedRef<IDatasmithMetaDataElement> MetaData = FDatasmithSceneFactory::CreateMetaData(MeshActorRed->GetName());
		MetaData->SetAssociatedElement(MeshActorRed);

		TSharedRef<IDatasmithKeyValueProperty> ActorTypeProperty = FDatasmithSceneFactory::CreateKeyValueProperty(TEXT("ActorType"));
		ActorTypeProperty->SetValue(TEXT("Big Box"));
		ActorTypeProperty->SetPropertyType(EDatasmithKeyValuePropertyType::String);

		MetaData->AddProperty(ActorTypeProperty);

		DatasmithScene->AddMetaData(MetaData);
	}

	// Export
	DatasmithSceneExporter.Export(DatasmithScene);

	return DatasmithScene;
}

