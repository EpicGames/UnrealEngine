// Copyright Epic Games, Inc. All Rights Reserved.

#include "pch.h"

#include "MeshUtils.h"
#include "ScenesManager.h"
#include "shared.h"

#include "DatasmithExportOptions.h"
#include "DatasmithMesh.h"
#include "DatasmithMeshExporter.h"
#include "DatasmithSceneExporter.h"
#include "DatasmithSceneFactory.h"

#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFilemanager.h"
#include "Misc/Paths.h"


struct FLightsScene : public ISampleScene
{
	virtual FString GetName() const override { return TEXT("Lights");}
	virtual FString GetDescription() const override { return TEXT("A scene that demonstrates datasmith lights."); }
	virtual TArray<FString> GetTags() const override { return {"lights", "point-light", "spot-light", "area-light", "IES-light"}; }
	virtual TSharedPtr<IDatasmithScene> Export() override;
};
REGISTER_SCENE(FLightsScene)


TSharedPtr<IDatasmithScene> FLightsScene::Export()
{
	FDatasmithSceneExporter DatasmithSceneExporter;

	DatasmithSceneExporter.SetName(*GetName());
	DatasmithSceneExporter.SetOutputPath(*GetExportPath());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.CreateDirectoryTree(DatasmithSceneExporter.GetOutputPath()))
	{
		return nullptr;
	}

	TSharedRef<IDatasmithScene> DatasmithScene = FDatasmithSceneFactory::CreateScene(*GetName());
	SetupSharedSceneProperties(DatasmithScene);

	// setup a layout that help us placing actors regularly on the scene
	FVector Origin{-1200, -400, 120}; // our lights are positioned 1.2 meters high
	FVector Stride{600, 600, 0};      // we separate the lights by 6 meters (on X and Y)
	SampleUtils::FGridLayout Grid(Origin, Stride);

	{
		TSharedRef<IDatasmithPointLightElement> PointLightActor = FDatasmithSceneFactory::CreatePointLight(TEXT("Point"));
		DatasmithScene->AddActor(PointLightActor);
		PointLightActor->SetTranslation(Grid.GetCurrentVector());
		PointLightActor->SetEnabled(true);
		PointLightActor->SetIntensity(20.);
		PointLightActor->SetIntensityUnits(EDatasmithLightUnits::Candelas);

		PointLightActor->SetColor(FLinearColor{1.f, 1.f, 1.f});
		PointLightActor->SetUseTemperature(true);
		PointLightActor->SetTemperature(2500.);
	}

	Grid.NextItem();
	auto MakeSpot = [&](const TCHAR* Name, FLinearColor Color, FVector Position)
	{
		TSharedRef<IDatasmithSpotLightElement> SpotLightActor = FDatasmithSceneFactory::CreateSpotLight(Name);
		DatasmithScene->AddActor(SpotLightActor);
		SpotLightActor->SetTranslation(Position);
		SpotLightActor->SetRotation(FRotator(-90., 0., 0.).Quaternion()); // (yzx) we rotate spots down toward the floor
		SpotLightActor->SetIntensity(30000.);
		SpotLightActor->SetColor(Color);
		SpotLightActor->SetInnerConeAngle(38.);
		SpotLightActor->SetOuterConeAngle(40.);
	};

	MakeSpot(TEXT("Spot-R"), FLinearColor{1,0,0}, Grid.GetCurrentVector());
	MakeSpot(TEXT("Spot-G"), FLinearColor{0,1,0}, Grid.GetCurrentVector() + FVector(35, 50, 0));
	MakeSpot(TEXT("Spot-B"), FLinearColor{0,0,1}, Grid.GetCurrentVector() + FVector(70, 0, 0));

	Grid.NextItem();
	{
		TSharedRef<IDatasmithSpotLightElement> SpotLightActor = FDatasmithSceneFactory::CreateSpotLight(TEXT("Spot-soft"));
		DatasmithScene->AddActor(SpotLightActor);
		SpotLightActor->SetTranslation(Grid.GetCurrentVector());
		SpotLightActor->SetRotation(FRotator(-60., 0., 0.).Quaternion()); // yzx
		SpotLightActor->SetIntensity(20000.);
		SpotLightActor->SetColor(FLinearColor{1,1,1});
		SpotLightActor->SetInnerConeAngle(20.);
		SpotLightActor->SetOuterConeAngle(40.);
	}

	Grid.NextLine();
	{
		TSharedRef<IDatasmithAreaLightElement> AreaLight = FDatasmithSceneFactory::CreateAreaLight(TEXT("Area-Rect"));
		DatasmithScene->AddActor(AreaLight);
		AreaLight->SetTranslation(Grid.GetCurrentVector());
		AreaLight->SetRotation(FRotator(-90., 0., 0.).Quaternion()); // yzxe
		AreaLight->SetLightShape(EDatasmithLightShape::Rectangle);
		AreaLight->SetLength(200);
		AreaLight->SetWidth(20);

		AreaLight->SetIntensity(100.);
		AreaLight->SetIntensityUnits(EDatasmithLightUnits::Candelas);
		AreaLight->SetUseTemperature(true);
		AreaLight->SetTemperature(4000.);
	}

	Grid.NextItem();
	{
		TSharedRef<IDatasmithAreaLightElement> AreaLight = FDatasmithSceneFactory::CreateAreaLight(TEXT("Area-Sphere"));
		DatasmithScene->AddActor(AreaLight);
		AreaLight->SetTranslation(Grid.GetCurrentVector());
		AreaLight->SetRotation(FRotator(-90., 0., 0.).Quaternion()); // yzxe
		AreaLight->SetLightShape(EDatasmithLightShape::Sphere);
		AreaLight->SetLength(80);
		AreaLight->SetWidth(20);

		AreaLight->SetIntensity(100.);
		AreaLight->SetIntensityUnits(EDatasmithLightUnits::Candelas);
		AreaLight->SetColor(FLinearColor{1.f, 0.7f, 0.0f});
		AreaLight->SetUseTemperature(false);
	}

	Grid.NextItem();
	{
		TSharedRef<IDatasmithAreaLightElement> AreaLight = FDatasmithSceneFactory::CreateAreaLight(TEXT("Area-Cylinder"));
		DatasmithScene->AddActor(AreaLight);
		AreaLight->SetTranslation(Grid.GetCurrentVector());
		AreaLight->SetRotation(FRotator(-90., 0., 0.).Quaternion()); // yzxe
		AreaLight->SetLightShape(EDatasmithLightShape::Cylinder);
		AreaLight->SetLength(100);
		AreaLight->SetWidth(10);

		AreaLight->SetIntensity(100.);
		AreaLight->SetIntensityUnits(EDatasmithLightUnits::Candelas);
		AreaLight->SetUseTemperature(true);
		AreaLight->SetTemperature(7000);
	}

	Grid.NextLine();
	{
		// Create the IES texture element in the datasmith scene
		auto Tex = FDatasmithSceneFactory::CreateTexture(TEXT("IES"));
		DatasmithScene->AddTexture(Tex);
		Tex->SetFile(TEXT("Assets/Std_Lights_Assets/IESTextureFile.IES"));
		Tex->SetTextureMode(EDatasmithTextureMode::Ies);
	}

	auto SpawnIESLight = [&](float Temp, int32 Index)
	{
		FString Name = FString::Printf(TEXT("IES-%d"), Index); // generate unique names
		TSharedRef<IDatasmithPointLightElement> LightActor = FDatasmithSceneFactory::CreatePointLight(*Name);
		DatasmithScene->AddActor(LightActor);
		LightActor->SetTranslation(Grid.GetCurrentVector());
		LightActor->SetRotation(FRotator(0., 0., -90.).Quaternion()); // yzx, orient toward the ground

		LightActor->SetIntensity(50.);
		LightActor->SetIntensityUnits(EDatasmithLightUnits::Candelas);
		LightActor->SetUseTemperature(true);
		LightActor->SetTemperature(Temp);

		LightActor->SetIesTexturePathName(TEXT("IES"));
		LightActor->SetUseIes(true);

		LightActor->SetUseIesBrightness(true);
		LightActor->SetIesBrightnessScale(0.10);

		Grid.NextItem();
	};
	for (int32 Index = 0; Index < 6; ++Index)
	{
		SpawnIESLight(1000.f * (Index + 1), Index);
	}

	// Export
	DatasmithSceneExporter.Export(DatasmithScene, false);

	return DatasmithScene;
}

