// Copyright Epic Games, Inc. All Rights Reserved.

#include "pch.h"
#include "shared.h"

#include "ScenesManager.h"
#include "MeshUtils.h"

#include "DatasmithSceneExporter.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFilemanager.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithMesh.h"
#include "DatasmithMeshExporter.h"
#include "DatasmithExportOptions.h"
#include "GenericPlatform/GenericPlatformFile.h"


struct FMeshesScene : public ISampleScene
{
	virtual FString GetName() const override { return TEXT("Meshes");}
	virtual FString GetDescription() const override { return TEXT("A scene that details how to use the meshes API."); }
	virtual TArray<FString> GetTags() const override { return {"meshes"}; }
	virtual TSharedPtr<IDatasmithScene> Export() override;
};
REGISTER_SCENE(FMeshesScene)


// [ ] Basics
// [ ] Normal Tangentes
// [ ] UVs
// [ ] LODs
// [ ] Vertex color

enum EMeshOptions
{
	MO_WithUVs = 1 << 0,
	MO_WithHighMatIds = 1 << 1, // test purpose only
	MO_WithNormals = 1 << 2,

	MO_Default = MO_WithNormals | MO_WithUVs
};


static void CreateSimpleBox_(FDatasmithMesh& Mesh, EMeshOptions Flags=MO_Default)
{
	int32 NumFaces = 6; //six quads on a box
	int32 NumTriangles = 12; //everything should be converted to triangles
	int32 NumVertex = 8; //eight shared vertex to create all the triangles

	//set the num of needed geometry vertex and faces
	Mesh.SetVerticesCount(NumVertex);
	Mesh.SetFacesCount(NumTriangles);

	// Set the position of each vertex
	Mesh.SetVertex(0, -50.0, -50.0, 0.0);
	Mesh.SetVertex(1, -50.0, 50.0, 0.0);
	Mesh.SetVertex(2, 50.0, 50.0, 0.0);
	Mesh.SetVertex(3, 50.0, -50.0, 0.0);
	Mesh.SetVertex(4, -50.0, -50.0, -100.0);
	Mesh.SetVertex(5, -50.0, 50.0, -100.0);
	Mesh.SetVertex(6, 50.0, 50.0, -100.0);
	Mesh.SetVertex(7, 50.0, -50.0, -100.0);

	if (Flags & EMeshOptions::MO_WithUVs)
	{
		Mesh.AddUVChannel();

		// Set the num of needed UV vertex
		Mesh.SetUVCount(0, 4);

		//set the actual position of each uv vertex
		Mesh.SetUV(0, 0, 0.0, 0.0);
		Mesh.SetUV(0, 1, 0.0, 1.0);
		Mesh.SetUV(0, 2, 1.0, 1.0);
		Mesh.SetUV(0, 3, 1.0, 0.0);
	}

	// Create polygons. Assign texture and texture UV indices.

	// We'll create three quads with material 0 and three quads with material 1

	//SetFace: face index, vert1, vert2, vert3, FaceId
	int32 FaceId = Flags & EMeshOptions::MO_WithHighMatIds ? 1'234'567 : 0;
	Mesh.SetFace(0, 1, 2, 0, FaceId); //top face
	Mesh.SetFace(1, 3, 0, 2, FaceId);
	FaceId++;
	Mesh.SetFace(2, 6, 5, 4, FaceId); //bottom face
	Mesh.SetFace(3, 4, 7, 6, FaceId);
	Mesh.SetFace(4, 4, 5, 1, FaceId); //left face
	Mesh.SetFace(5, 1, 0, 4, FaceId);
	Mesh.SetFace(6, 6, 7, 2, FaceId); //right face
	Mesh.SetFace(7, 3, 2, 7, FaceId);
	Mesh.SetFace(8, 7, 4, 0, FaceId); //front face
	Mesh.SetFace(9, 0, 3, 7, FaceId);
	Mesh.SetFace(10, 5, 6, 1, FaceId); //back face
	Mesh.SetFace(11, 2, 1, 6, FaceId);

	if (Flags & EMeshOptions::MO_WithUVs)
	{
		//set uv indexes for each face vertex
		//top faces
		Mesh.SetFaceUV(0, 0, 1, 2, 0);
		Mesh.SetFaceUV(1, 0, 3, 0, 2);
		//bottom faces
		Mesh.SetFaceUV(2, 0, 2, 1, 0);
		Mesh.SetFaceUV(3, 0, 0, 3, 2);
		//left faces
		Mesh.SetFaceUV(4, 0, 2, 1, 0);
		Mesh.SetFaceUV(5, 0, 0, 3, 2);
		//right faces
		Mesh.SetFaceUV(6, 0, 1, 2, 0);
		Mesh.SetFaceUV(7, 0, 3, 0, 2);
		//front faces
		Mesh.SetFaceUV(8, 0, 2, 1, 0);
		Mesh.SetFaceUV(9, 0, 0, 3, 2);
		//back faces
		Mesh.SetFaceUV(10, 0, 1, 2, 0);
		Mesh.SetFaceUV(11, 0, 3, 0, 2);
	}

	if (Flags & EMeshOptions::MO_WithNormals)
	{
		Mesh.SetNormal(0, 0.0, 0.0, 1.0); //top faces
		Mesh.SetNormal(1, 0.0, 0.0, 1.0);
		Mesh.SetNormal(2, 0.0, 0.0, 1.0);
		Mesh.SetNormal(3, 0.0, 0.0, 1.0);
		Mesh.SetNormal(4, 0.0, 0.0, 1.0);
		Mesh.SetNormal(5, 0.0, 0.0, 1.0);
		Mesh.SetNormal(6, 0.0, 0.0, -1.0); //bottom faces
		Mesh.SetNormal(7, 0.0, 0.0, -1.0);
		Mesh.SetNormal(8, 0.0, 0.0, -1.0);
		Mesh.SetNormal(9, 0.0, 0.0, -1.0);
		Mesh.SetNormal(10, 0.0, 0.0, -1.0);
		Mesh.SetNormal(11, 0.0, 0.0, -1.0);
		Mesh.SetNormal(12, -1.0, 0.0, 0.0); //left faces
		Mesh.SetNormal(13, -1.0, 0.0, 0.0);
		Mesh.SetNormal(14, -1.0, 0.0, 0.0);
		Mesh.SetNormal(15, -1.0, 0.0, 0.0);
		Mesh.SetNormal(16, -1.0, 0.0, 0.0);
		Mesh.SetNormal(17, -1.0, 0.0, 0.0);
		Mesh.SetNormal(18, 1.0, 0.0, 0.0); //right faces
		Mesh.SetNormal(19, 1.0, 0.0, 0.0);
		Mesh.SetNormal(20, 1.0, 0.0, 0.0);
		Mesh.SetNormal(21, 1.0, 0.0, 0.0);
		Mesh.SetNormal(22, 1.0, 0.0, 0.0);
		Mesh.SetNormal(23, 1.0, 0.0, 0.0);
		Mesh.SetNormal(24, 0.0, -1.0, 0.0); //front faces
		Mesh.SetNormal(25, 0.0, -1.0, 0.0);
		Mesh.SetNormal(26, 0.0, -1.0, 0.0);
		Mesh.SetNormal(27, 0.0, -1.0, 0.0);
		Mesh.SetNormal(28, 0.0, -1.0, 0.0);
		Mesh.SetNormal(29, 0.0, -1.0, 0.0);
		Mesh.SetNormal(30, 0.0, 1.0, 0.0); //back faces
		Mesh.SetNormal(31, 0.0, 1.0, 0.0);
		Mesh.SetNormal(32, 0.0, 1.0, 0.0);
		Mesh.SetNormal(33, 0.0, 1.0, 0.0);
		Mesh.SetNormal(34, 0.0, 1.0, 0.0);
		Mesh.SetNormal(35, 0.0, 1.0, 0.0);
	}
}

TSharedPtr<IDatasmithScene> FMeshesScene::Export()
{
	FDatasmithSceneExporter DatasmithSceneExporter;

	DatasmithSceneExporter.SetName(*GetName());
	DatasmithSceneExporter.SetOutputPath(*GetExportPath());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.CreateDirectoryTree(DatasmithSceneExporter.GetOutputPath()))
	{
		return nullptr;
	}

	TSharedRef<IDatasmithScene> DatasmithScene = FDatasmithSceneFactory::CreateScene(*GetName());
	SetupSharedSceneProperties(DatasmithScene);

	// Creates a mesh asset
	{
		FDatasmithMesh BaseMesh;
		CreateSimpleBox_(BaseMesh, EMeshOptions(MO_WithHighMatIds));

		FDatasmithMeshExporter MeshExporter;
		TSharedPtr<IDatasmithMeshElement> MeshElement = MeshExporter.ExportToUObject(DatasmithSceneExporter.GetAssetsOutputPath(), TEXT("BoxMesh"), BaseMesh, nullptr, EDSExportLightmapUV::Always /** Not used */);

		// Meshes can be null if the export fails
		if (!MeshElement)
		{
			return nullptr;
		}

		// Add the mesh to the DatasmithScene
		DatasmithScene->AddMesh(MeshElement);
	}

	TSharedPtr<IDatasmithMeshActorElement> MeshActorRed;
	{
		MeshActorRed = FDatasmithSceneFactory::CreateMeshActor(TEXT("Sample Actor"));
		MeshActorRed->SetStaticMeshPathName(TEXT("BoxMesh")); // Assign the geometry with index 0 (the box) to this actor

		MeshActorRed->SetTranslation(FVector(0.f, 0.f, 100.f));
		MeshActorRed->SetScale(FVector::OneVector);
		MeshActorRed->SetRotation(FQuat::Identity);

		DatasmithScene->AddActor(MeshActorRed);
	}

	// Export
	DatasmithSceneExporter.Export(DatasmithScene, false);

	return DatasmithScene;
}

