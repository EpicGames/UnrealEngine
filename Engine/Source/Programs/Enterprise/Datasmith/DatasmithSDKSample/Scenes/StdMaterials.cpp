// Copyright Epic Games, Inc. All Rights Reserved.

#include "pch.h"
#include "shared.h"

#include "ScenesManager.h"
#include "MeshUtils.h"

#include "DatasmithSceneExporter.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFilemanager.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithMesh.h"
#include "DatasmithMeshExporter.h"
#include "DatasmithExportOptions.h"
#include "GenericPlatform/GenericPlatformFile.h"


struct FStdMaterialsScene : public ISampleScene
{
	virtual FString GetName() const override { return TEXT("StdMaterials");}
	virtual FString GetDescription() const override { return TEXT("A scene that demonstrates how to use the datasmith material library."); }
	virtual TArray<FString> GetTags() const override { return {"uepbr-material", "material-library"}; }
	virtual TSharedPtr<IDatasmithScene> Export() override;
};
REGISTER_SCENE(FStdMaterialsScene)


// see //UE5/Release-5.0/Sandbox/Enterprise/QAEnterprise/DatasmithSamples

TSharedPtr<IDatasmithScene> FStdMaterialsScene::Export()
{
	FDatasmithSceneExporter DatasmithSceneExporter;

	DatasmithSceneExporter.SetName(*GetName());
	DatasmithSceneExporter.SetOutputPath(*GetExportPath());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.CreateDirectoryTree(DatasmithSceneExporter.GetOutputPath()))
	{
		return nullptr;
	}

	TSharedRef<IDatasmithScene> DatasmithScene = FDatasmithSceneFactory::CreateScene(*GetName());
	SetupSharedSceneProperties(DatasmithScene);

	// Defines materials

	const TCHAR* MaterialRed = TEXT("Red Material");
	{
		TSharedRef<IDatasmithUEPbrMaterialElement> Material = FDatasmithSceneFactory::CreateUEPbrMaterial(MaterialRed);
		IDatasmithMaterialExpressionColor* BaseColorExpression = Material->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		BaseColorExpression->GetColor() = FLinearColor(0.8f, 0.f, 0.f);
		BaseColorExpression->ConnectExpression(Material->GetBaseColor());
		DatasmithScene->AddMaterial(Material);
	}

	const TCHAR* MaterialBlue = TEXT("Blue Material");
	{
		TSharedRef<IDatasmithUEPbrMaterialElement> Material = FDatasmithSceneFactory::CreateUEPbrMaterial(MaterialBlue);
		IDatasmithMaterialExpressionColor* BaseColorExpression = Material->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		BaseColorExpression->GetColor() = FLinearColor(0.f, 0.f, 0.8f);
		BaseColorExpression->ConnectExpression(Material->GetBaseColor());
		DatasmithScene->AddMaterial(Material);
	}

	// Defines Meshes. Here, we reference an existing asset that ship with the Engine
	const TCHAR* DemoMesh_Path = TEXT("/Engine/EngineMeshes/SM_MatPreviewMesh_01");
	const uint32 DemoMesh_MatSlotOuter = 0; // Material index of the outer sphere
	const uint32 DemoMesh_MatSlotInner = 1; // material index of the inner / engraved parts

	// Defines Actors
	{
		TSharedRef<IDatasmithMeshActorElement> Actor = FDatasmithSceneFactory::CreateMeshActor(TEXT("Mat Preview actor"));
		Actor->SetStaticMeshPathName(DemoMesh_Path);
		Actor->AddMaterialOverride(MaterialBlue, DemoMesh_MatSlotInner);
		Actor->AddMaterialOverride(MaterialRed, DemoMesh_MatSlotOuter);
		DatasmithScene->AddActor(Actor);
	}

	// Export
	DatasmithSceneExporter.Export(DatasmithScene, false);

	return DatasmithScene;
}

