// Copyright Epic Games, Inc. All Rights Reserved.

#include "pch.h"
#include "shared.h"

#include "ScenesManager.h"
#include "MeshUtils.h"

#include "DatasmithSceneExporter.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFilemanager.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithMesh.h"
#include "DatasmithMeshExporter.h"
#include "DatasmithExportOptions.h"
#include "GenericPlatform/GenericPlatformFile.h"


struct FTexturesScene : public ISampleScene
{
	virtual FString GetName() const override { return TEXT("Textures");}
	virtual FString GetDescription() const override { return TEXT("A scene that demonstrates how to use textures, with UV transforms."); }
	virtual TArray<FString> GetTags() const override { return {"texture", "uv"}; }
	virtual TSharedPtr<IDatasmithScene> Export() override;
};
REGISTER_SCENE(FTexturesScene)


// see //UE5/Release-5.0/Sandbox/Enterprise/QAEnterprise/DatasmithSamples

TSharedPtr<IDatasmithScene> FTexturesScene::Export()
{
	FDatasmithSceneExporter DatasmithSceneExporter;

	DatasmithSceneExporter.SetName(*GetName());
	DatasmithSceneExporter.SetOutputPath(*GetExportPath());

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.CreateDirectoryTree(DatasmithSceneExporter.GetOutputPath()))
	{
		return nullptr;
	}

	TSharedRef<IDatasmithScene> DatasmithScene = FDatasmithSceneFactory::CreateScene(*GetName());
	SetupSharedSceneProperties(DatasmithScene);

	const TCHAR* MaterialBlack = TEXT("Black Material");
	{
		TSharedRef<IDatasmithUEPbrMaterialElement> Material = FDatasmithSceneFactory::CreateUEPbrMaterial(MaterialBlack);
		IDatasmithMaterialExpressionColor* BaseColorExpression = Material->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		BaseColorExpression->GetColor() = FLinearColor(0.f, 0.f, 0.f);
		BaseColorExpression->ConnectExpression(Material->GetBaseColor());
		DatasmithScene->AddMaterial(Material);
	}

	const TCHAR* TextureName = TEXT("TextureName");
	{
		TSharedRef<IDatasmithTextureElement> TextureElement = FDatasmithSceneFactory::CreateTexture(TextureName);
		DatasmithScene->AddTexture(TextureElement);
		TextureElement->SetFile(TEXT("Assets/Textures/UV_Layout_Test_Grid.jpg")); // source file
		TextureElement->SetTextureMode(EDatasmithTextureMode::Diffuse);
	}

	const TCHAR* MaterialTexturedDiffuse = TEXT("TexturedDiffuse");
	{
		TSharedRef<IDatasmithUEPbrMaterialElement> Material = FDatasmithSceneFactory::CreateUEPbrMaterial(MaterialTexturedDiffuse);
		IDatasmithMaterialExpressionTexture* TextureExpression = Material->AddMaterialExpression<IDatasmithMaterialExpressionTexture>();
		TextureExpression->SetTexturePathName(TextureName);
		TextureExpression->ConnectExpression(Material->GetBaseColor());
		DatasmithScene->AddMaterial(Material);
	}

// 	// #ue_ds_sdk_todo : Material->AddMaterialExpression<IDatasmithMaterialExpressionTextureCoordinate>();
// 	const TCHAR* MaterialTexturedDiffuseScaled = TEXT("TexturedDiffuseScaled");
// 	{
// 		TSharedRef<IDatasmithUEPbrMaterialElement> Material = FDatasmithSceneFactory::CreateUEPbrMaterial(MaterialTexturedDiffuse);
// 		IDatasmithMaterialExpressionTexture* TextureExpression = Material->AddMaterialExpression<IDatasmithMaterialExpressionTexture>();
// 		TextureExpression->SetTexturePathName(TextureName);
// 		TextureExpression->ConnectExpression(Material->GetBaseColor());
// 		DatasmithScene->AddMaterial(Material);
// 	}

	// References to an Engine predefined mesh
	const TCHAR* DemoMesh_Path = TEXT("/Engine/EngineMeshes/SM_MatPreviewMesh_01"); // Path of an existing asset that ship with the Engine
	const uint32 DemoMesh_MatSlotMain = 0; // Material index of the outer sphere
	const uint32 DemoMesh_MatSlotInner = 1; // material index of the inner / engraved parts

	// Defines Actors
	{
		TSharedRef<IDatasmithMeshActorElement> Actor = FDatasmithSceneFactory::CreateMeshActor(TEXT("Test actor"));
		Actor->SetStaticMeshPathName(DemoMesh_Path);
		Actor->AddMaterialOverride(MaterialBlack, DemoMesh_MatSlotInner);
		Actor->AddMaterialOverride(MaterialTexturedDiffuse, DemoMesh_MatSlotMain);
		DatasmithScene->AddActor(Actor);
	}

// 	{
// 		TSharedRef<IDatasmithMeshActorElement> Actor = FDatasmithSceneFactory::CreateMeshActor(TEXT("Background actor"));
// 		Actor->SetStaticMeshPathName(TEXT("/Engine/EngineMeshes/BackgroundCube"));
// 		DatasmithScene->AddActor(Actor);
// 	}
// 	{
// 		TSharedRef<IDatasmithMeshActorElement> Actor = FDatasmithSceneFactory::CreateMeshActor(TEXT("Ball actor"));
// 		Actor->SetStaticMeshPathName(TEXT("/Engine/VREditor/BasicMeshes/SM_Ball_01"));
// 		Actor->SetTranslation(FVector(1000.f, 22.f, 33.f));
// 		DatasmithScene->AddActor(Actor);
// 	}

	// Export
	DatasmithSceneExporter.Export(DatasmithScene, false);

	return DatasmithScene;
}

