// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDatasmithSceneElements.h"

namespace SampleUtils
{
/**
 * a little helper struct to layout stuff in a grid pattern fashion
 */
struct FGridLayout
{
	FGridLayout(FVector Offset=FVector::ZeroVector, FVector Stride=FVector(100.))
		: Stride(Stride)
		, Offset(Offset)
	{}

	void NextItem() { ++XIndex; }
	void NextLine() { ++YIndex; XIndex = 0; }
	FVector GetCurrentVector() { return Offset + FVector(Stride.X * XIndex, Stride.Y * YIndex, 0); }

private:
	int32 XIndex = 0, YIndex = 0;
	FVector Offset;
	FVector Stride;
	double XStride = 100.0;
	double YStride = 100.0;
};

}

