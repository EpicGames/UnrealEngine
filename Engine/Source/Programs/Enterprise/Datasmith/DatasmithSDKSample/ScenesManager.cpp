// Copyright Epic Games, Inc. All Rights Reserved.

#include "pch.h"

#include "ScenesManager.h"
#include "DatasmithUtils.h"


static FString ExportPath = TEXT("Export");

const FString& GetExportPath()
{
	return ExportPath;
}

void SetExportPath(const FString& Path)
{
	ExportPath = Path;
}

void SetupSharedSceneProperties(TSharedRef<IDatasmithScene> DatasmithScene)
{
	DatasmithScene->SetVendor(TEXT("Epic"));
	DatasmithScene->SetProductName(TEXT("DatasmithSDK"));
	DatasmithScene->SetProductVersion(*FDatasmithUtils::GetEnterpriseVersionAsString());
	DatasmithScene->SetHost(TEXT("DatasmithSDKSample"));
}

TArray<TSharedPtr<ISampleScene>>& GetRegisteredScenes()
{
	static TArray<TSharedPtr<ISampleScene>> RegisteredScenes;
	return RegisteredScenes;
}

void FScenesManager::Register(TSharedPtr<ISampleScene> SampleScene)
{
	GetRegisteredScenes().Add(SampleScene);
	GetRegisteredScenes().StableSort([](auto&& L, auto&& R){ return L->GetName() < R->GetName();});
}

const TArray<TSharedPtr<ISampleScene>>& FScenesManager::GetAllScenes()
{
	return GetRegisteredScenes();
}

TSharedPtr<ISampleScene> FScenesManager::GetScene(const FString& Name)
{
	int32 Index = 0;
	for (const TSharedPtr<ISampleScene>& Scene : GetRegisteredScenes())
	{
		if (Scene->GetName() == Name || FString::FromInt(Index) == Name)
		{
			return Scene;
		}
		Index++;
	}
	return nullptr;
}


