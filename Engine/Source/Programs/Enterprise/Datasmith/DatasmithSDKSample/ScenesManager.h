// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "pch.h"

#include "IDatasmithSceneElements.h"

#include "Templates/SharedPointer.h"


const FString& GetExportPath();
void SetExportPath(const FString& Path);

/**
* For this sample application, some parameters are shared by all the scene. To avoid repetition, this is setup there.
*/
void SetupSharedSceneProperties(TSharedRef<IDatasmithScene> DatasmithScene);


struct ISampleScene
{
	// a simple name to identify that scene
	virtual FString GetName() const = 0;

	// a longer, optionnal description
	virtual FString GetDescription() const { return {}; }

	// list the themes used in this scene (eg. Instancing, LOD, Collision, etc.)
	virtual TArray<FString> GetTags() const { return {}; }

	// export a udatasmith scene
	virtual TSharedPtr<IDatasmithScene> Export() { return nullptr; }
};


#define REGISTER_SCENE(Type) \
static struct AutoRegister ## Type {\
	AutoRegister ## Type() { FScenesManager::Register(MakeShared<Type>()); }\
} AutoRegisterInstance ## Type;



class FScenesManager
{
public:
	// Register a new scene. Prefer the REGISTER_SCENE macro
	static void Register(TSharedPtr<ISampleScene> SampleScene);

	static const TArray<TSharedPtr<ISampleScene>>& GetAllScenes();

	static TSharedPtr<ISampleScene> GetScene(const FString& Name);
};
