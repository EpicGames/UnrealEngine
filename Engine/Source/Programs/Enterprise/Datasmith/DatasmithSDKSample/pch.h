// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef PCH_H
#define PCH_H

#include <strsafe.h>

// DatasmithSDK project can be built from the main solution, we check that datasmith headers are accessible
#if !__has_include("DatasmithCore.h")
#error Datasmith SDK is not accessible. Make sure the SDK has been built from the main solution, and verify that DatasmithSDK.props correctly points to the SDK.
#endif


#endif //PCH_H
